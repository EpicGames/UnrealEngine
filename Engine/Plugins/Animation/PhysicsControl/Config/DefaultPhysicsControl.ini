[CoreRedirects]
+PropertyRedirects=(OldName="PhysicsControlComponent.CreateControl.ParentMeshComponent",NewName="ParentComponent")
+PropertyRedirects=(OldName="PhysicsControlComponent.CreateControl.ChildMeshComponent",NewName="ChildComponent")

+PropertyRedirects=(OldName="PhysicsControlComponent.CreateNamedControl.ParentMeshComponent",NewName="ParentComponent")
+PropertyRedirects=(OldName="PhysicsControlComponent.CreateNamedControl.ChildMeshComponent",NewName="ChildComponent")

+PropertyRedirects=(OldName="PhysicsControlComponent.SetControlParent.ParentMeshComponent",NewName="ParentComponent")
+PropertyRedirects=(OldName="PhysicsControlComponent.SetControlParents.ParentMeshComponent",NewName="ParentComponent")
+PropertyRedirects=(OldName="PhysicsControlComponent.SetControlParentsInSet.ParentMeshComponent",NewName="ParentComponent")

+PropertyRedirects=(OldName="PhysicsControlComponent.CreateBodyModifier.MeshComponent",NewName="Component")
+PropertyRedirects=(OldName="PhysicsControlComponent.CreateNamedBodyModifier.MeshComponent",NewName="Component")

+FunctionRedirects=(OldName="PhysicsControlComponent.CreateControlsAndBodyModifiersFromControlProfileAsset",NewName="CreateControlsAndBodyModifiersFromPhysicsControlAsset")

+PropertyRedirects=(OldName="PhysicsControlProfileAsset",NewName="PhysicsControlAsset")
+ClassRedirects=(OldName="PhysicsControlProfileAsset",NewName="PhysicsControlAsset")
