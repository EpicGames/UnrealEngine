{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "PhysicsControl",
	"Description": "Physically control static and skeletal meshes through the Physics Control Component and the Rigid Body With Control animation graph node.",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"CanContainVerse": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "PhysicsControl",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "PhysicsControlUncookedOnly",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "PhysicsControlEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	]
}