// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PhysicsControl : ModuleRules
{
	public PhysicsControl(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AnimationCore",
				"AnimGraphRuntime",
				"Core",
				"CoreUObject",
				"Engine"
			}
		);

		SetupModulePhysicsSupport(Target);
	}
}
