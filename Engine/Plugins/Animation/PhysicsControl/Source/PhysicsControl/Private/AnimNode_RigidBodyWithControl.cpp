// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_RigidBodyWithControl.h"
#include "Animation/AnimInstance.h"
#include "PhysicsControlLog.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"
#include "ClothCollisionSource.h"
#include "Engine/OverlapResult.h"
#include "Engine/SkeletalMesh.h"
#include "GameFramework/Pawn.h"
#include "HAL/Event.h"
#include "HAL/LowLevelMemTracker.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "GameFramework/PawnMovementComponent.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsActorHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsAdapters.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsJointHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsSimulation.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsStats.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/Collision/CollisionContext.h"
#include "PhysicsField/PhysicsFieldComponent.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Logging/MessageLog.h"

#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

#include "Chaos/PBDJointConstraintTypes.h"

#if WITH_CHAOS_VISUAL_DEBUGGER
#include "ChaosVDRuntimeModule.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RigidBodyWithControl)

LLM_DEFINE_TAG(Animation_RigidBodyWithControl);

/////////////////////////////////////////////////////
// FAnimNode_RigidBodyWithControl

#define LOCTEXT_NAMESPACE "ImmediatePhysicsWithControl"

DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_InitPhysics"), STAT_RigidBodyWithControlInitPhysicsTime, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_SetupControls"), STAT_RigidBodyWithControlSetupControlsTime, STATGROUP_Anim);

DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_Eval"), STAT_RigidBodyNodeWithControl_Eval, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_Simulation"), STAT_RigidBodyNodeWithControl_Simulation, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_UpdateWorldObjects"), STAT_RigidBodyNodeWithControl_UpdateWorldObjects, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_SimulationWait"), STAT_RigidBodyNodeWithControl_SimulationWait, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_PreUpdate"), STAT_RigidBodyNodeWithControl_PreUpdate, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_Update"), STAT_RigidBodyNodeWithControl_Update, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_PoseUpdate"), STAT_RigidBodyNodeWithControl_PoseUpdate, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("FAnimNode_RigidBodyWithControl::EvaluateSkeletalControl_AnyThread"), STAT_ImmediateEvaluateSkeletalControl, STATGROUP_ImmediatePhysics);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(ENGINE_API, Animation);

TAutoConsoleVariable<int32> CVarEnableRigidBodyNodeWithControl(TEXT("p.RigidBodyNodeWithControl"), 1, TEXT("Enables/disables the whole rigid body node system. When disabled, avoids all allocations and runtime costs. Can be used to disable RB Nodes on low-end platforms."), ECVF_Scalability);
TAutoConsoleVariable<int32> CVarEnableRigidBodyNodeWithControlSimulation(TEXT("p.RigidBodyNodeWithControl.EnableSimulation"), 1, TEXT("Runtime Enable/Disable RB Node Simulation for debugging and testing (node is initialized and bodies and constraints are created, even when disabled.)"), ECVF_Default);
TAutoConsoleVariable<int32> CVarEnableRigidBodyNodeWithControlMatchingConstraintsToSkeleton(TEXT("p.RigidBodyNodeWithControl.EnableMatchingConstraintsToSkeleton"), 1, TEXT("Enables/disables the code that modifies physics asset constraint transforms to match the Skeleton at runtime."), ECVF_Scalability);
TAutoConsoleVariable<int32> CVarRigidBodyNodeWithControlLODThreshold(TEXT("p.RigidBodyWithControlLODThreshold"), -1, TEXT("Max LOD that rigid body node is allowed to run on. Provides a global threshold that overrides per-node the LODThreshold property. -1 means no override."), ECVF_Scalability);

int32 RBANWithControl_MaxSubSteps = 4;
bool bRBANWithControl_EnableTimeBasedReset = true;
bool bRBANWithControl_EnableComponentAcceleration = true;
int32 RBANWithControl_WorldObjectExpiry = 4;
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlMaxSteps(TEXT("p.RigidBodyNodeWithControl.MaxSubSteps"), RBANWithControl_MaxSubSteps, TEXT("Set the maximum number of simulation steps in the update loop"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlEnableTimeBasedReset(TEXT("p.RigidBodyNodeWithControl.EnableTimeBasedReset"), bRBANWithControl_EnableTimeBasedReset, TEXT("If true, Rigid Body nodes are reset when they have not been updated for a while (default true)"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlEnableComponentAcceleration(TEXT("p.RigidBodyNodeWithControl.EnableComponentAcceleration"), bRBANWithControl_EnableComponentAcceleration, TEXT("Enable/Disable the simple acceleration transfer system for component- or bone-space simulation"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlWorldObjectExpiry(TEXT("p.RigidBodyNodeWithControl.WorldObjectExpiry"), RBANWithControl_WorldObjectExpiry, TEXT("World objects are removed from the simulation if not detected after this many tests"), ECVF_Default);

bool bRBANWithControl_IncludeClothColliders = true;
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlIncludeClothColliders(TEXT("p.RigidBodyNodeWithControl.IncludeClothColliders"), bRBANWithControl_IncludeClothColliders, TEXT("Include cloth colliders as kinematic bodies in the immediate physics simulation."), ECVF_Default);

// FSimSpaceSettingsWithControl forced overrides for testing
bool bRBANWithControl_SimSpace_EnableOverride = false;
FSimSpaceSettings RBANWithControl_SimSpaceOverride;
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlSpaceOverride(TEXT("p.RigidBodyNodeWithControl.Space.Override"), bRBANWithControl_SimSpace_EnableOverride, TEXT("Force-enable the advanced simulation space movement forces"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlSpaceWorldAlpha(TEXT("p.RigidBodyNodeWithControl.Space.WorldAlpha"), RBANWithControl_SimSpaceOverride.WorldAlpha, TEXT("RBANWithControl SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlSpaceVelScaleZ(TEXT("p.RigidBodyNodeWithControl.Space.VelocityScaleZ"), RBANWithControl_SimSpaceOverride.VelocityScaleZ, TEXT("RBANWithControl SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlSpaceMaxCompLinVel(TEXT("p.RigidBodyNodeWithControl.Space.MaxLinearVelocity"), RBANWithControl_SimSpaceOverride.MaxLinearVelocity, TEXT("RBANWithControl SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlSpaceMaxCompAngVel(TEXT("p.RigidBodyNodeWithControl.Space.MaxAngularVelocity"), RBANWithControl_SimSpaceOverride.MaxAngularVelocity, TEXT("RBANWithControl SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlSpaceMaxCompLinAcc(TEXT("p.RigidBodyNodeWithControl.Space.MaxLinearAcceleration"), RBANWithControl_SimSpaceOverride.MaxLinearAcceleration, TEXT("RBANWithControl SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlSpaceMaxCompAngAcc(TEXT("p.RigidBodyNodeWithControl.Space.MaxAngularAcceleration"), RBANWithControl_SimSpaceOverride.MaxAngularAcceleration, TEXT("RBANWithControl SimSpaceSettings overrides"), ECVF_Default);
// LWC_TODO: Double support for console variables
#if 0
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlSpaceExternalLinearDragX(TEXT("p.RigidBodyNodeWithControl.Space.ExternalLinearDrag.X"), RBANWithControl_SimSpaceOverride.ExternalLinearDragV.X, TEXT("RBANWithControl SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlSpaceExternalLinearDragY(TEXT("p.RigidBodyNodeWithControl.Space.ExternalLinearDrag.Y"), RBANWithControl_SimSpaceOverride.ExternalLinearDragV.Y, TEXT("RBANWithControl SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlSpaceExternalLinearDragZ(TEXT("p.RigidBodyNodeWithControl.Space.ExternalLinearDrag.Z"), RBANWithControl_SimSpaceOverride.ExternalLinearDragV.Z, TEXT("RBANWithControl SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlSpaceExternalLinearVelocityX(TEXT("p.RigidBodyNodeWithControl.Space.ExternalLinearVelocity.X"), RBANWithControl_SimSpaceOverride.ExternalLinearVelocity.X, TEXT("RBANWithControl SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlSpaceExternalLinearVelocityY(TEXT("p.RigidBodyNodeWithControl.Space.ExternalLinearVelocity.Y"), RBANWithControl_SimSpaceOverride.ExternalLinearVelocity.Y, TEXT("RBANWithControl SimSpaceSettings overrides"), ECVF_Default);
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlSpaceExternalLinearVelocityZ(TEXT("p.RigidBodyNodeWithControl.Space.ExternalLinearVelocity.Z"), RBANWithControl_SimSpaceOverride.ExternalLinearVelocity.Z, TEXT("RBANWithControl SimSpaceSettings overrides"), ECVF_Default);
#endif

bool bRBANWithControl_DeferredSimulationDefault = false;
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlDeferredSimulationDefault(
	TEXT("p.RigidBodyNodeWithControl.DeferredSimulationDefault"),
	bRBANWithControl_DeferredSimulationDefault,
	TEXT("Whether rigid body simulations are deferred one frame for assets that don't opt into a specific simulation timing"),
	ECVF_Default);

bool bRBANWithControl_DebugDraw = false;
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlDebugDraw(TEXT("p.RigidBodyNodeWithControl.DebugDraw"), bRBANWithControl_DebugDraw, TEXT("Whether to debug draw the rigid body simulation state. Requires p.Chaos.DebugDraw.Enabled 1 to function as well."), ECVF_Default);

// Array of priorities that can be indexed into with CVars, since task priorities cannot be set from scalability .ini
static UE::Tasks::ETaskPriority GRigidBodyNodeWithControlTaskPriorities[] =
{
	UE::Tasks::ETaskPriority::High,
	UE::Tasks::ETaskPriority::Normal,
	UE::Tasks::ETaskPriority::BackgroundHigh,
	UE::Tasks::ETaskPriority::BackgroundNormal,
	UE::Tasks::ETaskPriority::BackgroundLow
};

static int32 GRigidBodyNodeWithControlSimulationTaskPriority = 0;
FAutoConsoleVariableRef CVarRigidBodyNodeWithControlSimulationTaskPriority(
	TEXT("p.RigidBodyNodeWithControl.TaskPriority.Simulation"),
	GRigidBodyNodeWithControlSimulationTaskPriority,
	TEXT("Task priority for running the rigid body node simulation task (0 = foreground/high, 1 = foreground/normal, 2 = background/high, 3 = background/normal, 4 = background/low)."),
	ECVF_Default
);

FAnimNode_RigidBodyWithControl::FAnimNode_RigidBodyWithControl()
	: OverridePhysicsAsset(nullptr)
	, PreviousCompWorldSpaceTM()
	, CurrentTransform()
	, PreviousTransform()
	, PhysicsAssetToUse(nullptr)
	, OverrideWorldGravity(0.0f)
	, ExternalForce(0.0f)
	, ComponentLinearAccScale(0.0f)
	, ComponentLinearVelScale(0.0f)
	, ComponentAppliedLinearAccClamp(10000.0f)
	, SimSpaceSettings()
	, CachedBoundsScale(1.2f)
	, bUpdateCacheEveryFrame(true)
	, BaseBoneRef()
	, OverlapChannel(ECC_WorldStatic)
	, SimulationSpace(ESimulationSpace::ComponentSpace)
	, bCalculateVelocitiesForWorldGeometry(true)
	, bForceDisableCollisionBetweenConstraintBodies(false)
	, bUseExternalClothCollision(false)
	, bMakeKinematicConstraints(true)
	, ResetSimulatedTeleportType(ETeleportType::None)
	, bEnableWorldGeometry(false)
	, bWorldGeometryCollisionEnabledHasPhysics(true)
	, bWorldGeometryCollisionEnabledHasQuery(false)
	, bWorldGeometryCollisionEnabledHasProbe(false)
	, bOverrideWorldGravity(false)
	, bTransferBoneVelocities(false)
	, bFreezeIncomingPoseOnStart(false)
	, PhysicsAssetConditioningConstraintPosition(MapConstraintsBehaviorType::None)
	, PhysicsAssetConditioningConstraintOrientation(MapConstraintsBehaviorType::None)
	, WorldSpaceMinimumScale(0.01f)
	, EvaluationResetTime(0.01f)
	, bEnableControls(false)
	, PhysicsAssetAuthoredSkeletalMesh(nullptr)
	, bEnabled(false)
	, bSimulationStarted(false)
	, bCheckForBodyTransformInit(false)
	, bHaveSetupControls(false)
	, SimulationTiming(ESimulationTiming::Synchronous)
	, WorldTimeSeconds(0.0f)
	, LastEvalTimeSeconds(0.0f)
	, AccumulatedDeltaTime(0.0f)
	, AnimPhysicsMinDeltaTime(0.0f)
	, bSimulateAnimPhysicsAfterReset(false)
	, SkelMeshCompWeakPtr()
	, Simulation(nullptr)
	, SolverSettings()
	, SolverIterations()
	, SimulationTask()
	, OutputBoneData()
	, Bodies()
	, SkeletonBoneIndexToBodyIndex()
	, BodyAnimData()
	, PendingRadialForces()
	, PerSolverField()
	, ComponentsInSim()
	, ComponentsInSimTick(0)
	, WorldSpaceGravity(0.0f)
	, TotalMass(0.0f)
	, CachedBounds(FVector::ZeroVector, 0.0f)
	, QueryParams(NAME_None, FCollisionQueryParams::GetUnknownStatId())
	, UnsafeWorld(nullptr)
	, UnsafeOwner(nullptr)
	, SkeletalMeshAtLastPhysicsInit(nullptr)
	, CapturedBoneVelocityBoneContainer()
	, CapturedBoneVelocityPose()
	, CapturedFrozenPose()
	, CapturedFrozenCurves()
	, PreviousComponentLinearVelocity(0.0f)
	, PreviousSimulationSpaceTransform()
	, PreviousPreviousSimulationSpaceTransform()
{
}

FAnimNode_RigidBodyWithControl::~FAnimNode_RigidBodyWithControl()
{
	DestroyPhysicsSimulation();
}

void FAnimNode_RigidBodyWithControl::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	DebugLine += ")";

	DebugData.AddDebugItem(DebugLine);

	const bool bUsingFrozenPose = bFreezeIncomingPoseOnStart && bSimulationStarted && (CapturedFrozenPose.GetPose().GetNumBones() > 0);
	if (!bUsingFrozenPose)
	{
		ComponentPose.GatherDebugData(DebugData);
	}
}

void FAnimNode_RigidBodyWithControl::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	Super::Initialize_AnyThread(Context);

#if WITH_EDITOR
	if(GIsReinstancing)
	{
		InitPhysics(Cast<UAnimInstance>(Context.GetAnimInstanceObject()));
	}
#endif
}


void FAnimNode_RigidBodyWithControl::UpdateComponentPose_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(UpdateComponentPose_AnyThread)
	// Only freeze update graph after initial update, as we want to get that pose through.
	if (bFreezeIncomingPoseOnStart && bSimulationStarted && ResetSimulatedTeleportType == ETeleportType::None)
	{
		// If we have a Frozen Pose captured, 
		// then we don't need to update the rest of the graph.
		if (CapturedFrozenPose.GetPose().GetNumBones() > 0)
		{
		}
		else
		{
			// Create a new context with zero deltatime to freeze time in rest of the graph.
			// This will be used to capture a frozen pose.
			FAnimationUpdateContext FrozenContext = Context.FractionalWeightAndTime(1.f, 0.f);

			Super::UpdateComponentPose_AnyThread(FrozenContext);
		}
	}
	else
	{
		Super::UpdateComponentPose_AnyThread(Context);
	}
}

void FAnimNode_RigidBodyWithControl::EvaluateComponentPose_AnyThread(FComponentSpacePoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateComponentPose_AnyThread)
	if (bFreezeIncomingPoseOnStart && bSimulationStarted)
	{
		// If we have a Frozen Pose captured, use it.
		// Only after our intialize setup. As we need new pose for that.
		if (ResetSimulatedTeleportType == ETeleportType::None && (CapturedFrozenPose.GetPose().GetNumBones() > 0))
		{
			Output.Pose.CopyPose(CapturedFrozenPose);
			Output.Curve.CopyFrom(CapturedFrozenCurves);
		}
		// Otherwise eval graph to capture it.
		else
		{
			Super::EvaluateComponentPose_AnyThread(Output);
			CapturedFrozenPose.CopyPose(Output.Pose);
			CapturedFrozenCurves.CopyFrom(Output.Curve);
		}
	}
	else
	{
		Super::EvaluateComponentPose_AnyThread(Output);
	}

	// Capture incoming pose if 'bTransferBoneVelocities' is set.
	// That is, until simulation starts.
	if (bTransferBoneVelocities && !bSimulationStarted)
	{
		CapturedBoneVelocityPose.CopyPose(Output.Pose);
		CapturedBoneVelocityPose.CopyAndAssignBoneContainer(CapturedBoneVelocityBoneContainer);
	}
}

void FAnimNode_RigidBodyWithControl::InitializeNewBodyTransformsDuringSimulation(
	FComponentSpacePoseContext& Output, const FTransform& ComponentTransform, const FTransform& BaseBoneTM)
{
	for (const UE::PhysicsControl::FOutputBoneData& OutputData : OutputBoneData)
	{
		const int32 BodyIndex = OutputData.BodyIndex;
		FBodyAnimData& BodyData = BodyAnimData[BodyIndex];
		if (!BodyData.bBodyTransformInitialized)
		{
			BodyData.bBodyTransformInitialized = true;

			// If we have a parent body, we need to grab relative transforms to it.
			if (OutputData.ParentBodyIndex != INDEX_NONE)
			{
				ensure(BodyAnimData[OutputData.ParentBodyIndex].bBodyTransformInitialized);

				FTransform BodyRelativeTransform = FTransform::Identity;
				for (const FCompactPoseBoneIndex& CompactBoneIndex : OutputData.BoneIndicesToParentBody)
				{
					const FTransform& LocalSpaceTM = Output.Pose.GetLocalSpaceTransform(CompactBoneIndex);
					BodyRelativeTransform = BodyRelativeTransform * LocalSpaceTM;
				}

				const FTransform WSBodyTM = BodyRelativeTransform * Bodies[OutputData.ParentBodyIndex]->GetWorldTransform();
				Bodies[BodyIndex]->InitWorldTransform(WSBodyTM);
			}
			// If we don't have a parent body, then we can just grab the incoming pose in component space.
			else
			{
				const FTransform& ComponentSpaceTM = Output.Pose.GetComponentSpaceTransform(OutputData.CompactPoseBoneIndex);
				const FTransform BodyTM = ConvertCSTransformToSimSpace(SimulationSpace, ComponentSpaceTM, ComponentTransform, BaseBoneTM);

				Bodies[BodyIndex]->InitWorldTransform(BodyTM);
			}
		}
	}
}

void FAnimNode_RigidBodyWithControl::InitSimulationSpace(
	const FTransform& ComponentToWorld,
	const FTransform& BoneToComponent)
{
	SimulationSpaceTransform = SpaceToWorldTransform(SimulationSpace, ComponentToWorld, BoneToComponent);
	PreviousSimulationSpaceTransform = SimulationSpaceTransform;
	PreviousPreviousSimulationSpaceTransform = SimulationSpaceTransform;
}

void FAnimNode_RigidBodyWithControl::CalculateSimulationSpaceMotion(
	const float              Dt,
	const FSimSpaceSettings& Settings,
	FVector&                 SpaceLinearVel, 
	FVector&                 SpaceAngularVel, 
	FVector&                 SpaceLinearAcc, 
	FVector&                 SpaceAngularAcc)
{
	SpaceLinearVel = FVector::ZeroVector;
	SpaceAngularVel = FVector::ZeroVector;
	SpaceLinearAcc = FVector::ZeroVector;
	SpaceAngularAcc = FVector::ZeroVector;

	// If the system is disabled, nothing else to do
	if ((Settings.WorldAlpha == 0.0f) || (Dt < UE_SMALL_NUMBER))
	{
		return;
	}

	// If the simulation is in world space, the simulation space is stationary
	if (SimulationSpace == ESimulationSpace::WorldSpace)
	{
		SpaceLinearVel = Settings.ExternalLinearVelocity;
		SpaceAngularVel = Settings.ExternalAngularVelocity;
		return;
	}

	// World-space component linear velocity and acceleration
	FVector PrevSpaceLinearVel = Chaos::FVec3::CalculateVelocity(
		PreviousPreviousSimulationSpaceTransform.GetTranslation(), PreviousSimulationSpaceTransform.GetTranslation(), Dt);
	SpaceLinearVel = Chaos::FVec3::CalculateVelocity(
		PreviousSimulationSpaceTransform.GetTranslation(), SimulationSpaceTransform.GetTranslation(), Dt);
	SpaceLinearAcc = (SpaceLinearVel - PrevSpaceLinearVel) / Dt;

	// World-space component angular velocity and acceleration
	FVector PrevSpaceAngularVel = Chaos::FRotation3::CalculateAngularVelocity(
		PreviousPreviousSimulationSpaceTransform.GetRotation(), PreviousSimulationSpaceTransform.GetRotation(), Dt);
	SpaceAngularVel = Chaos::FRotation3::CalculateAngularVelocity(
		PreviousSimulationSpaceTransform.GetRotation(), SimulationSpaceTransform.GetRotation(), Dt);
	SpaceAngularAcc = (SpaceAngularVel - PrevSpaceAngularVel) / Dt;

	// Apply Z scale
	SpaceLinearVel.Z *= Settings.VelocityScaleZ;
	SpaceLinearAcc.Z *= Settings.VelocityScaleZ;
	
	// Clamped world-space motion of the simulation space
	SpaceLinearVel = SpaceLinearVel.GetClampedToMaxSize(Settings.MaxLinearVelocity) + Settings.ExternalLinearVelocity;
	SpaceAngularVel = SpaceAngularVel.GetClampedToMaxSize(Settings.MaxAngularVelocity) + Settings.ExternalAngularVelocity;
	SpaceLinearAcc = SpaceLinearAcc.GetClampedToMaxSize(Settings.MaxLinearAcceleration);
	SpaceAngularAcc = SpaceAngularAcc.GetClampedToMaxSize(Settings.MaxAngularAcceleration);

	// Transform world-space motion into simulation space
	SpaceLinearVel = SimulationSpaceTransform.InverseTransformVector(SpaceLinearVel);
	SpaceAngularVel = SimulationSpaceTransform.InverseTransformVectorNoScale(SpaceAngularVel);
	SpaceLinearAcc = SimulationSpaceTransform.InverseTransformVector(SpaceLinearAcc);
	SpaceAngularAcc = SimulationSpaceTransform.InverseTransformVectorNoScale(SpaceAngularAcc);

	// Apply WorldAlpha to simulation space motion (usually to reduce it)
	SpaceLinearVel *= Settings.WorldAlpha;
	SpaceLinearAcc *= Settings.WorldAlpha;
	SpaceAngularVel *= Settings.WorldAlpha;
	SpaceAngularAcc *= Settings.WorldAlpha;

	PreviousPreviousSimulationSpaceTransform = PreviousSimulationSpaceTransform;
	PreviousSimulationSpaceTransform = SimulationSpaceTransform;
}


bool FAnimNode_RigidBodyWithControl::ShouldApplyPhysicsAssetConditioning() const
{
	return (PhysicsAssetConditioningConstraintPosition != MapConstraintsBehaviorType::None)
		&& (PhysicsAssetConditioningConstraintOrientation != MapConstraintsBehaviorType::None)
		&& (CVarEnableRigidBodyNodeWithControlMatchingConstraintsToSkeleton.GetValueOnAnyThread() > 0);
}

void FAnimNode_RigidBodyWithControl::RunPhysicsSimulation(float DeltaSeconds, const FVector& SimSpaceGravity)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigidBodyWithControl"));
	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_Simulation);
	CSV_SCOPED_TIMING_STAT(Animation, RigidBodyNodeWithControlSimulation);
	FScopeCycleCounterUObject AdditionalScope(PhysicsAssetToUse, GET_STATID(STAT_RigidBodyNodeWithControl_Simulation));

	const int32 MaxSteps = RBANWithControl_MaxSubSteps;
	const float MaxDeltaSeconds = 1.f / 30.f;

	Simulation->Simulate_AssumesLocked(DeltaSeconds, MaxDeltaSeconds, MaxSteps, SimSpaceGravity);
	bSimulationStarted = true;
}

void FAnimNode_RigidBodyWithControl::FlushDeferredSimulationTask()
{
	if (SimulationTask.IsValid() && !SimulationTask.IsCompleted())
	{
		SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_SimulationWait);
		CSV_SCOPED_TIMING_STAT(Animation, RigidBodyNodeWithControlSimulationWait);
		SimulationTask.Wait();
	}
}

void FAnimNode_RigidBodyWithControl::DestroyPhysicsSimulation()
{
	ClothColliders.Reset();
	FlushDeferredSimulationTask();
	delete Simulation;
	Simulation = nullptr;
	CurrentConstraintProfile = FName();
}

void FAnimNode_RigidBodyWithControl::SetupControls(USkeletalMeshComponent* const SkeletalMeshComponent)
{
	SCOPE_CYCLE_COUNTER(STAT_RigidBodyWithControlSetupControlsTime);

	bool bSuccess = false;

	if (SkeletalMeshComponent)
	{
		if (USkeletalMesh* const SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			CreateWorldSpaceControlRootBody(PhysicsAssetToUse);
			InitControlsAndBodyModifiers(SkeletalMesh->GetRefSkeleton());
			bHaveSetupControls = true;
		}
		else
		{
			UE_LOGF(LogPhysicsControl, Warning, "Invalid Skeletal Mesh");
		}
	}
	else
	{
		UE_LOGF(LogPhysicsControl, Warning, "Invalid Skeletal Mesh Component");
	}
}


void FAnimNode_RigidBodyWithControl::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigidBodyWithControl"));
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(EvaluateSkeletalControl_AnyThread)
	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_Eval);
	CSV_SCOPED_TIMING_STAT(Animation, RigidBodyEval);
	SCOPE_CYCLE_COUNTER(STAT_ImmediateEvaluateSkeletalControl);
	//SCOPED_NAMED_EVENT_TEXT("FAnimNode_RigidBodyWithControl::EvaluateSkeletalControl_AnyThread", FColor::Magenta);

	if (CVarEnableRigidBodyNodeWithControlSimulation.GetValueOnAnyThread() == 0)
	{
		return;
	}

	const float DeltaSeconds = AccumulatedDeltaTime;
	AccumulatedDeltaTime = 0.f;

	if (bEnabled && Simulation)	
	{
		FlushDeferredSimulationTask();

		// Handle deferred control creation
		if (!bHaveSetupControls && bEnableControls)
		{
			SetupControls(Output.AnimInstanceProxy->GetSkelMeshComponent());
		}
		else if (bHaveSetupControls && !bEnableControls)
		{
			DestroyControlsAndBodyModifiers();
		}

		const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
		const FTransform CompWorldSpaceTM = Output.AnimInstanceProxy->GetComponentTransform();

		bool bFirstEvalSinceReset = !Output.AnimInstanceProxy->GetEvaluationCounter().HasEverBeenUpdated();

		// First-frame initialization
		if (bFirstEvalSinceReset)
		{
			PreviousCompWorldSpaceTM = CompWorldSpaceTM;
			ResetSimulatedTeleportType = ETeleportType::ResetPhysics;
		}

		// See if we need to reset physics because too much time passed since our last update (e.g., because we we off-screen for a while), 
		// in which case the current sim state may be too far from the current anim pose. This is mostly a problem with world-space 
		// simulation, whereas bone- and component-space sims can be fairly robust against missing updates.
		// Don't do this on first frame or if time-based reset is disabled. 
		if ((EvaluationResetTime > 0.0f) && !bFirstEvalSinceReset)
		{
			// NOTE: under normal conditions, when this anim node is being serviced at the usual rate (which may not be every frame
			// if URO is enabled), we expect that WorldTimeSeconds == (LastEvalTimeSeconds + DeltaSeconds). DeltaSeconds is the 
			// accumulated time since the last update, including frames dropped by URO, but not frames dropped because of
			// being off-screen or LOD changes.
			if (WorldTimeSeconds - (LastEvalTimeSeconds + DeltaSeconds) > EvaluationResetTime)
			{
				UE_LOGF(LogPhysicsControl, Verbose, "%ls Time-Based Reset", *Output.AnimInstanceProxy->GetAnimInstanceName());
				ResetSimulatedTeleportType = ETeleportType::ResetPhysics;
			}
		}

		// Update the evaluation time to the current time
		LastEvalTimeSeconds = WorldTimeSeconds;

		// Disable simulation below minimum scale in world space mode. World space sim doesn't play nice with scale anyway - we do not scale joint offets or collision shapes.
		if ((SimulationSpace == ESimulationSpace::WorldSpace) && (CompWorldSpaceTM.GetScale3D().SizeSquared() < WorldSpaceMinimumScale * WorldSpaceMinimumScale))
		{
			return;
		}

		const FTransform BaseBoneTM = Output.Pose.GetComponentSpaceTransform(BaseBoneRef.GetCompactPoseIndex(BoneContainer));

		// Now we know the base bone, we can calculate the simulation space transform
		SimulationSpaceTransform = SpaceToWorldTransform(SimulationSpace, CompWorldSpaceTM, BaseBoneTM);

		// Initialize potential new bodies because of LOD change.
		if (ResetSimulatedTeleportType == ETeleportType::None && bCheckForBodyTransformInit)
		{
			bCheckForBodyTransformInit = false;
			InitializeNewBodyTransformsDuringSimulation(Output, CompWorldSpaceTM, BaseBoneTM);
		}

		// If time advances, update simulation
		// Reset if necessary
		const bool bResetOrTeleportBodies = (ResetSimulatedTeleportType != ETeleportType::None);
		if (bResetOrTeleportBodies)
		{
			// Capture bone velocities if we have captured a bone velocity pose.
			if (bTransferBoneVelocities && (CapturedBoneVelocityPose.GetPose().GetNumBones() > 0))
			{
				for (const UE::PhysicsControl::FOutputBoneData& OutputData : OutputBoneData)
				{
					const int32 BodyIndex = OutputData.BodyIndex;
					FBodyAnimData& BodyData = BodyAnimData[BodyIndex];

					if (BodyData.bIsSimulated)
					{
						const FCompactPoseBoneIndex NextCompactPoseBoneIndex = OutputData.CompactPoseBoneIndex;
						// Convert CompactPoseBoneIndex to SkeletonBoneIndex...
						const FSkeletonPoseBoneIndex PoseSkeletonBoneIndex = BoneContainer.GetSkeletonPoseIndexFromCompactPoseIndex(NextCompactPoseBoneIndex);
						// ... So we can convert to the captured pose CompactPoseBoneIndex. 
						// In case there was a LOD change, and poses are not compatible anymore.
						const FCompactPoseBoneIndex PrevCompactPoseBoneIndex = CapturedBoneVelocityBoneContainer.GetCompactPoseIndexFromSkeletonPoseIndex(PoseSkeletonBoneIndex);

						if (PrevCompactPoseBoneIndex.IsValid())
						{
							const FTransform PrevCSTM = CapturedBoneVelocityPose.GetComponentSpaceTransform(PrevCompactPoseBoneIndex);
							const FTransform NextCSTM = Output.Pose.GetComponentSpaceTransform(NextCompactPoseBoneIndex);

							const FTransform PrevSSTM = ConvertCSTransformToSimSpace(SimulationSpace, PrevCSTM, CompWorldSpaceTM, BaseBoneTM);
							const FTransform NextSSTM = ConvertCSTransformToSimSpace(SimulationSpace, NextCSTM, CompWorldSpaceTM, BaseBoneTM);

							if(DeltaSeconds > 0.0f)
							{
								// Linear Velocity
								BodyData.TransferedBoneLinearVelocity = ((NextSSTM.GetLocation() - PrevSSTM.GetLocation()) / DeltaSeconds);
								
								// Angular Velocity
								const FQuat DeltaRotation = (NextSSTM.GetRotation().Inverse() * PrevSSTM.GetRotation()).GetShortestArcWith(FQuat::Identity);
								BodyData.TransferedBoneAngularVelocity = DeltaRotation.ToRotationVector() / DeltaSeconds;
							}
							else
							{
								BodyData.TransferedBoneLinearVelocity = FVector::ZeroVector;
								BodyData.TransferedBoneAngularVelocity = FVector::ZeroVector;
							}

						}
					}
				}
			}

			
			switch(ResetSimulatedTeleportType)
			{
				case ETeleportType::TeleportPhysics:
				{
					UE_LOGF(LogPhysicsControl, Verbose, "%ls TeleportPhysics (Scale: %f %f %f)", *Output.AnimInstanceProxy->GetAnimInstanceName(), CompWorldSpaceTM.GetScale3D().X, CompWorldSpaceTM.GetScale3D().Y, CompWorldSpaceTM.GetScale3D().Z);

					// Teleport bodies.
					for (const UE::PhysicsControl::FOutputBoneData& OutputData : OutputBoneData)
					{
						const int32 BodyIndex = OutputData.BodyIndex;
						BodyAnimData[BodyIndex].bBodyTransformInitialized = true;

						FTransform BodyTM = Bodies[BodyIndex]->GetWorldTransform();
						FTransform ComponentSpaceTM;

						switch(SimulationSpace)
						{
							case ESimulationSpace::ComponentSpace: ComponentSpaceTM = BodyTM; break;
							case ESimulationSpace::WorldSpace: ComponentSpaceTM = BodyTM.GetRelativeTransform(PreviousCompWorldSpaceTM); break;
							case ESimulationSpace::BaseBoneSpace: ComponentSpaceTM = BodyTM * BaseBoneTM; break;
							default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); ComponentSpaceTM = BodyTM;
						}

						BodyTM = ConvertCSTransformToSimSpace(SimulationSpace, ComponentSpaceTM, CompWorldSpaceTM, BaseBoneTM);
						Bodies[BodyIndex]->SetWorldTransform(BodyTM);
					}
				}
				break;

				case ETeleportType::ResetPhysics:
				{
					UE_LOGF(LogPhysicsControl, Verbose, "%ls ResetPhysics (Scale: %f %f %f)", *Output.AnimInstanceProxy->GetAnimInstanceName(), CompWorldSpaceTM.GetScale3D().X, CompWorldSpaceTM.GetScale3D().Y, CompWorldSpaceTM.GetScale3D().Z);

					InitSimulationSpace(CompWorldSpaceTM, BaseBoneTM);

					// Completely reset bodies.
					for (const UE::PhysicsControl::FOutputBoneData& OutputData : OutputBoneData)
					{
						const int32 BodyIndex = OutputData.BodyIndex;
						BodyAnimData[BodyIndex].bBodyTransformInitialized = true;

						const FTransform& ComponentSpaceTM = Output.Pose.GetComponentSpaceTransform(OutputData.CompactPoseBoneIndex);
						const FTransform BodyTM = ConvertCSTransformToSimSpace(SimulationSpace, ComponentSpaceTM, CompWorldSpaceTM, BaseBoneTM);
						Bodies[BodyIndex]->InitWorldTransform(BodyTM);
					}
				}
				break;
			}

			// Always reset after a teleport
			PreviousCompWorldSpaceTM = CompWorldSpaceTM;
			PreviousComponentLinearVelocity = FVector::ZeroVector;
		}

		// Assets can override config for deferred simulation
		const bool bUseDeferredSimulationTask =
			(SimulationTiming == ESimulationTiming::Deferred) ||
			((SimulationTiming == ESimulationTiming::Default) && bRBANWithControl_DeferredSimulationDefault);
		FVector SimSpaceGravity(0.f);

		// Only need to tick physics if we didn't reset and we have some time to simulate
		const bool bNeedsSimulationTick = 
			((bSimulateAnimPhysicsAfterReset || (ResetSimulatedTeleportType != ETeleportType::ResetPhysics)) && 
				DeltaSeconds > AnimPhysicsMinDeltaTime);
		if (bNeedsSimulationTick)
		{
			// Update the pose data
			{
				SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_PoseUpdate);
				PoseData.Update(
					Output, OutputBoneData, SimulationSpace, BaseBoneRef, Output.AnimInstanceProxy->GetUpdateCounter());
			}

			// Transfer bone velocities previously captured.
			if (bTransferBoneVelocities && (CapturedBoneVelocityPose.GetPose().GetNumBones() > 0))
			{
				for (const UE::PhysicsControl::FOutputBoneData& OutputData : OutputBoneData)
				{
					const int32 BodyIndex = OutputData.BodyIndex;
					const FBodyAnimData& BodyData = BodyAnimData[BodyIndex];

					if (BodyData.bIsSimulated)
					{
						ImmediatePhysics::FActorHandle* Body = Bodies[BodyIndex];
						Body->SetLinearVelocity(BodyData.TransferedBoneLinearVelocity);
						Body->SetAngularVelocity(BodyData.TransferedBoneAngularVelocity);
					}
				}

				// Free up our captured pose after it's been used.
				CapturedBoneVelocityPose.Empty();
			}
			else if ((SimulationSpace != ESimulationSpace::WorldSpace) && bRBANWithControl_EnableComponentAcceleration)
			{
				if (!ComponentLinearVelScale.IsNearlyZero() || !ComponentLinearAccScale.IsNearlyZero())
				{
					// Calc linear velocity
					const FVector ComponentDeltaLocation = CurrentTransform.GetTranslation() - PreviousTransform.GetTranslation();
					const FVector ComponentLinearVelocity = ComponentDeltaLocation / DeltaSeconds;
					// Apply acceleration that opposed velocity (basically 'drag')
					FVector LinearAccelToApply = WorldVectorToSpaceNoScale(SimulationSpace, -ComponentLinearVelocity, CompWorldSpaceTM, BaseBoneTM) * ComponentLinearVelScale;

					// Calc linear acceleration
					const FVector ComponentLinearAcceleration = (ComponentLinearVelocity - PreviousComponentLinearVelocity) / DeltaSeconds;
					PreviousComponentLinearVelocity = ComponentLinearVelocity;
					// Apply opposite acceleration to bodies
					LinearAccelToApply += WorldVectorToSpaceNoScale(SimulationSpace, -ComponentLinearAcceleration, CompWorldSpaceTM, BaseBoneTM) * ComponentLinearAccScale;

					// Clamp if desired
					if (!ComponentAppliedLinearAccClamp.IsNearlyZero())
					{
						LinearAccelToApply = LinearAccelToApply.BoundToBox(-ComponentAppliedLinearAccClamp, ComponentAppliedLinearAccClamp);
					}

					// Iterate over bodies
					for (const UE::PhysicsControl::FOutputBoneData& OutputData : OutputBoneData)
					{
						const int32 BodyIndex = OutputData.BodyIndex;
						const FBodyAnimData& BodyData = BodyAnimData[BodyIndex];

						if (BodyData.bIsSimulated)
						{
							ImmediatePhysics::FActorHandle* Body = Bodies[BodyIndex];

							// Apply 
							const float BodyInvMass = (float) Body->GetInverseMass();
							if (BodyInvMass > 0.f)
							{
								// Apply to body
								Body->AddForce(LinearAccelToApply / BodyInvMass);
							}
						}
					}
				}
			}

			UpdateWorldForces(CompWorldSpaceTM, BaseBoneTM, DeltaSeconds);
			SimSpaceGravity = WorldVectorToSpaceNoScale(SimulationSpace, WorldSpaceGravity, CompWorldSpaceTM, BaseBoneTM);

			// Adjust the constraint profiles when it is changed
			if (ConstraintProfile != CurrentConstraintProfile)
			{
				CurrentConstraintProfile = ConstraintProfile;
				ApplyCurrentConstraintProfile();
			}

			if (bHaveSetupControls)
			{
				if (ControlProfile != CurrentControlProfile)
				{
					CurrentControlProfile = ControlProfile;
					ApplyCurrentControlProfile();
				}

				// Apply the controls. Note that these won't set kinematic targets - we'll do that afterwards as they
				// can apply to bodies that aren't under the influence of a modifier
				ApplyControlAndModifierUpdatesAndParametersToRecords(ControlAndModifierUpdates, ControlAndModifierParameters);
				ApplyControlsAndModifiers(SimSpaceGravity, DeltaSeconds);
				ApplyForcesAndTorques(CompWorldSpaceTM, BaseBoneTM);
			}

			// Note that the simulation interpolates kinematic targets to handle substepping
			for (const UE::PhysicsControl::FOutputBoneData& OutputData : OutputBoneData)
			{
				const int32 BodyIndex = OutputData.BodyIndex;
				BodyAnimData[BodyIndex].bIsSimulated = !Bodies[BodyIndex]->GetIsKinematic();
				if (!BodyAnimData[BodyIndex].bIsSimulated)
				{
					// Custom kinematic targets are set subsequently, overriding these.
					if (ensure(PoseData.IsValidIndex(BodyIndex)))
					{
						Bodies[BodyIndex]->SetKinematicTarget(PoseData.GetTM(BodyIndex).ToTransform());
					}
				}
			}

			// Note that this must be called after the "regular" kinematic targets above, as it will
			// overwrite any that have an explicit target set (which is probably quite unusual, so
			// the duplication is not likely to be expensive)
			ApplyKinematicTargets(CompWorldSpaceTM, BaseBoneTM);

			FSimSpaceSettings* SimSpaceSettingsToUse = &SimSpaceSettings;
			if (bRBANWithControl_SimSpace_EnableOverride)
			{
				SimSpaceSettingsToUse = &RBANWithControl_SimSpaceOverride;
			}

			FVector SimulationSpaceLinearVelocity;
			FVector SimulationSpaceAngularVelocity;
			FVector SimulationSpaceLinearAcceleration;
			FVector SimulationSpaceAngularAcceleration;
			CalculateSimulationSpaceMotion(
				DeltaSeconds,
				*SimSpaceSettingsToUse,
				SimulationSpaceLinearVelocity,
				SimulationSpaceAngularVelocity,
				SimulationSpaceLinearAcceleration,
				SimulationSpaceAngularAcceleration);

			UpdateWorldObjects(SimulationSpaceLinearVelocity, SimulationSpaceAngularVelocity, DeltaSeconds);
			UpdateClothColliderObjects();

			Simulation->UpdateSimulationSpace(
				SimulationSpaceTransform, SimulationSpaceLinearVelocity, SimulationSpaceAngularVelocity,
				SimulationSpaceLinearAcceleration, SimulationSpaceAngularAcceleration);

			Simulation->SetSimulationSpaceSettings(
				(SimSpaceSettingsToUse->WorldAlpha > 0.0f), SimSpaceSettingsToUse->DampingAlpha, SimSpaceSettingsToUse->ExternalLinearDragV);

			Simulation->SetSolverSettings(
				SolverSettings.FixedTimeStep,
				SolverSettings.CullDistance,
				SolverSettings.MaxDepenetrationVelocity,
				SolverSettings.bUseLinearJointSolver,
				SolverSettings.PositionIterations,
				SolverSettings.VelocityIterations,
				SolverSettings.ProjectionIterations,
				SolverSettings.bUseManifolds);

			Chaos::FCollisionDetectorSettings CollisionDetectorSettings = Simulation->GetCollisionDetectorSettings();
			CollisionDetectorSettings.BoundsVelocityInflation = BoundsVelocityMultiplier;
			CollisionDetectorSettings.MaxVelocityBoundsExpansion = MaxVelocityBoundsExpansion;
			CollisionDetectorSettings.bAllowCCD = bAllowCCD;
			Simulation->SetCollisionDetectorSettings(CollisionDetectorSettings);

			if (!bUseDeferredSimulationTask)
			{
				RunPhysicsSimulation(DeltaSeconds, SimSpaceGravity);
			}

			// Draw here even if the simulation is deferred since we want the shapes drawn relative to the current transform
			if (bRBANWithControl_DebugDraw)
			{
				Simulation->DebugDraw();
			}
		}
		
		//write back to animation system
		const FTransform& SimulationWorldSpaceTM = bUseDeferredSimulationTask ? PreviousCompWorldSpaceTM : CompWorldSpaceTM;
		for (const UE::PhysicsControl::FOutputBoneData& OutputData : OutputBoneData)
		{
			const int32 BodyIndex = OutputData.BodyIndex;
			// Note that we always read back, whether kinematic or simulated
			FTransform BodyTM = Bodies[BodyIndex]->GetWorldTransform();
			if (ensure(!BodyTM.ContainsNaN()))
			{
				FTransform ComponentSpaceTM;
				switch(SimulationSpace)
				{
					case ESimulationSpace::ComponentSpace: ComponentSpaceTM = BodyTM; break;
					case ESimulationSpace::WorldSpace: ComponentSpaceTM = BodyTM.GetRelativeTransform(SimulationWorldSpaceTM); break;
					case ESimulationSpace::BaseBoneSpace: ComponentSpaceTM = BodyTM * BaseBoneTM; break;
					default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); ComponentSpaceTM = BodyTM;
				}
					
				OutBoneTransforms.Add(FBoneTransform(OutputData.CompactPoseBoneIndex, ComponentSpaceTM));
			}
		}

		// Deferred task must be started after we read actor poses to avoid a race
		if (bNeedsSimulationTick && bUseDeferredSimulationTask)
		{
			// FlushDeferredSimulationTask() should have already ensured task is done.
			ensure(SimulationTask.IsCompleted());
			const int32 PriorityIndex = FMath::Clamp<int32>(GRigidBodyNodeWithControlSimulationTaskPriority, 0, UE_ARRAY_COUNT(GRigidBodyNodeWithControlTaskPriorities) - 1);
			const UE::Tasks::ETaskPriority TaskPriority = GRigidBodyNodeWithControlTaskPriorities[PriorityIndex];
			SimulationTask = UE::Tasks::Launch(
				TEXT("RigidBodyNodeWithControlSimulationTask"),
				[this, DeltaSeconds, SimSpaceGravity] { RunPhysicsSimulation(DeltaSeconds, SimSpaceGravity); },
				TaskPriority);
		}

		PreviousCompWorldSpaceTM = CompWorldSpaceTM;
		ResetSimulatedTeleportType = ETeleportType::None;
	}
}

void ComputeBodyInsertionOrderWithControl(TArray<FBoneIndexType>& InsertionOrder, const USkeletalMeshComponent& SKC)
{
	//We want to ensure simulated bodies are sorted by LOD so that the first simulated bodies are at the highest LOD.
	//Since LOD2 is a subset of LOD1 which is a subset of LOD0 we can change the number of simulated bodies without any reordering
	//For this to work we must first insert all simulated bodies in the right order. We then insert all the kinematic bodies in the right order

	InsertionOrder.Reset();

	if (SKC.GetSkeletalMeshAsset() == nullptr)
	{
		return;
	}

	const int32 NumLODs = SKC.GetNumLODs();
	if(NumLODs > 0)
	{
		TArray<FBoneIndexType> RequiredBones0;
		TArray<FBoneIndexType> ComponentSpaceTMs0;
		SKC.ComputeRequiredBones(RequiredBones0, ComponentSpaceTMs0, 0, /*bIgnorePhysicsAsset=*/ true);

		TArray<bool> InSortedOrder;
		InSortedOrder.AddZeroed(SKC.GetSkeletalMeshAsset()->GetRefSkeleton().GetNum());

		auto MergeIndices = [&InsertionOrder, &InSortedOrder](const TArray<FBoneIndexType>& RequiredBones) -> void
		{
			for (FBoneIndexType BoneIdx : RequiredBones)
			{
				if (!InSortedOrder[BoneIdx])
				{
					InsertionOrder.Add(BoneIdx);
				}

				InSortedOrder[BoneIdx] = true;
			}
		};


		for(int32 LodIdx = NumLODs - 1; LodIdx > 0; --LodIdx)
		{
			TArray<FBoneIndexType> RequiredBones;
			TArray<FBoneIndexType> ComponentSpaceTMs;
			SKC.ComputeRequiredBones(RequiredBones, ComponentSpaceTMs, LodIdx, /*bIgnorePhysicsAsset=*/ true);
			MergeIndices(RequiredBones);
		}

		MergeIndices(RequiredBones0);
	}
}

UPhysicsAsset* FAnimNode_RigidBodyWithControl::GetPhysicsAssetToBeUsed(const UAnimInstance* InAnimInstance) const
{
	if (IsValid(OverridePhysicsAsset))
	{
		return ToRawPtr(OverridePhysicsAsset);
	}

	if (bDefaultToSkeletalMeshPhysicsAsset && ensure(InAnimInstance))
	{
		const USkeletalMeshComponent* SkeletalMeshComp = InAnimInstance->GetSkelMeshComponent();
		if (SkeletalMeshComp)
		{
			return SkeletalMeshComp->GetPhysicsAsset();
		}
	}

	return nullptr;
}

void FAnimNode_RigidBodyWithControl::InitPhysics(const UAnimInstance* InAnimInstance)
{
	SCOPE_CYCLE_COUNTER(STAT_RigidBodyWithControlInitPhysicsTime);

	DestroyControlsAndBodyModifiers();
	DestroyPhysicsSimulation();

	const USkeletalMeshComponent* SkeletalMeshComp = InAnimInstance->GetSkelMeshComponent();
	if (!SkeletalMeshComp)
	{
		return;
	}
	const USkeletalMesh* SkeletalMeshAsset = SkeletalMeshComp->GetSkeletalMeshAsset();
	USkeleton* SkeletonAsset = InAnimInstance->CurrentSkeleton;

	if(!SkeletalMeshAsset || !SkeletonAsset)
	{
		// Without both the skeleton and the mesh we can't create a new simulation.
		// The previous simulation has just been cleaned up above so we can return early here and not instantiate a new one
		return;
	}

	// Record the Skeletal Mesh used to initialize physics so that it can be re-initialized if necessary when the Skeletal Mesh changes.
	SkeletalMeshAtLastPhysicsInit = SkeletalMeshAsset;

	const FReferenceSkeleton& SkelMeshRefSkel = SkeletalMeshAsset->GetRefSkeleton();
	PhysicsAssetToUse = GetPhysicsAssetToBeUsed(InAnimInstance);

	ensure(SkeletonAsset == SkeletalMeshAsset->GetSkeleton());

	const FSkeletonToMeshLinkup& SkeletonToMeshLinkupTable = SkeletonAsset->FindOrAddMeshLinkupData(SkeletalMeshAsset);
	const TArray<int32>& MeshToSkeletonBoneIndex = SkeletonToMeshLinkupTable.MeshToSkeletonTable;
	
	const int32 NumSkeletonBones = SkeletonAsset->GetReferenceSkeleton().GetNum();
	SkeletonBoneIndexToBodyIndex.Reset(NumSkeletonBones);
	SkeletonBoneIndexToBodyIndex.Init(INDEX_NONE, NumSkeletonBones);

	PreviousTransform = SkeletalMeshComp->GetComponentToWorld();

	ComponentsInSim.Reset();
	ComponentsInSimTick = 0;

	if (UPhysicsSettings* Settings = UPhysicsSettings::Get())
	{
		AnimPhysicsMinDeltaTime = Settings->AnimPhysicsMinDeltaTime;
		bSimulateAnimPhysicsAfterReset = Settings->bSimulateAnimPhysicsAfterReset;
	}
	else
	{
		AnimPhysicsMinDeltaTime = 0.f;
		bSimulateAnimPhysicsAfterReset = false;
	}
	
	bEnabled = PhysicsAssetToUse && SkeletalMeshComp->GetAllowRigidBodyAnimNode() && CVarEnableRigidBodyNodeWithControl.GetValueOnAnyThread() != 0;
	if(bEnabled)
	{
		Simulation = new ImmediatePhysics::FSimulation();

#if WITH_CHAOS_VISUAL_DEBUGGER
		Simulation->GetChaosVDContextData().Id = FChaosVDRuntimeModule::Get().GenerateUniqueID();
		Simulation->GetChaosVDContextData().Type = static_cast<int32>(EChaosVDContextType::Solver);
#endif

		Simulation->SetRewindVelocities(true);

		const int32 NumBodies = PhysicsAssetToUse->SkeletalBodySetups.Num();
		BodyNameToIndexMap.Reset();
		Bodies.Empty(NumBodies);
		Joints.Empty(NumBodies);
		WorldSpaceControlActorHandle = nullptr;
		BodyAnimData.Reset(NumBodies);
		BodyAnimData.AddDefaulted(NumBodies);
		TotalMass = 0.f;

		DestroyControlsAndBodyModifiers();

		// Instantiate a FBodyInstance/FConstraintInstance set that will be cloned into the Immediate Physics sim.
		// NOTE: We do not have a skeleton at the moment, so we have to use the ref pose
		TArray<FBodyInstance*> HighLevelBodyInstances;
		TArray<FConstraintInstance*> HighLevelConstraintInstances;

		// Chaos relies on the initial pose to set up constraint positions
		constexpr bool bCreateBodiesInRefPose = true;
		SkeletalMeshComp->InstantiatePhysicsAssetRefPose(
			*PhysicsAssetToUse, 
			SimulationSpace == ESimulationSpace::WorldSpace ? SkeletalMeshComp->GetComponentToWorld().GetScale3D() : FVector(1.f), 
			HighLevelBodyInstances, 
			HighLevelConstraintInstances, 
			nullptr, 
			nullptr, 
			INDEX_NONE, 
			FPhysicsAggregateHandle(),
			bCreateBodiesInRefPose);

		if (ShouldApplyPhysicsAssetConditioning())
		{
			TransformConstraintsToMatchSkeletalMesh(SkeletalMeshAsset, PhysicsAssetConditioningConstraintPosition, PhysicsAssetConditioningConstraintOrientation, HighLevelConstraintInstances);
		}

		TMap<FName, ImmediatePhysics::FActorHandle*> NamesToHandles;
		TArray<ImmediatePhysics::FActorHandle*> IgnoreCollisionActors;

		// Map bone names to the joint connecting them to the bone's parent
		TMap<FName, ImmediatePhysics::FJointHandle*> NamesToJointHandles;

		TArray<FBoneIndexType> InsertionOrder;
		ComputeBodyInsertionOrderWithControl(InsertionOrder, *SkeletalMeshComp);

		// NOTE: NumBonesLOD0 may be less than NumBonesTotal, and it may be middle bones that are
		// missing from LOD0. In this case, LOD0 bone indices may be >= NumBonesLOD0, but always <
		// NumBonesTotal. Arrays indexed by bone index must be size NumBonesTotal.
		const int32 NumBonesLOD0 = InsertionOrder.Num();
		const int32 NumBonesTotal = SkelMeshRefSkel.GetNum();

		// If our skeleton is not the one that was used to build the PhysicsAsset, some bodies may
		// be missing, or rearranged. We need to map the original indices to the new bodies for use
		// by the CollisionDisableTable.
		// NOTE: This array is indexed by the original BodyInstance body index (BodyInstance->InstanceBodyIndex)
		TArray<ImmediatePhysics::FActorHandle*> BodyIndexToActorHandle;
		BodyIndexToActorHandle.AddZeroed(HighLevelBodyInstances.Num());

		TArray<FBodyInstance*> BodiesSorted;
		BodiesSorted.AddZeroed(NumBonesTotal);

		for (FBodyInstance* BI : HighLevelBodyInstances)
		{
			if(BI->IsValidBodyInstance())
			{
				BodiesSorted[BI->InstanceBoneIndex] = BI;
			}
		}

		// Create the immediate physics bodies
		for (FBoneIndexType InsertBone : InsertionOrder)
		{
			if (FBodyInstance* BodyInstance = BodiesSorted[InsertBone])
			{
				const UBodySetup* BodySetup = PhysicsAssetToUse->SkeletalBodySetups[BodyInstance->InstanceBodyIndex];

				// Note that we create actors as dynamics, and then set them to kinematic if desired. Creating them
				// as kinematic prevents them being subsequently made dynamic.
				bool bSimulated = (BodySetup->PhysicsType == EPhysicsType::PhysType_Simulated);
				ImmediatePhysics::EActorType ActorType = ImmediatePhysics::EActorType::DynamicActor;
				ImmediatePhysics::FActorHandle* ActorHandle = Simulation->CreateActor(
					ImmediatePhysics::MakeActorSetup(ActorType, BodyInstance, BodyInstance->GetUnrealWorldTransform()));
				if (ActorHandle)
				{
					ActorHandle->SetCCDEnabled(BodySetup->DefaultInstance.bUseCCD);

					ActorHandle->InitWorldTransform(BodyInstance->GetUnrealWorldTransform());

					const float InvMass = (float) ActorHandle->GetInverseMass();
					TotalMass += InvMass > 0.f ? 1.f / InvMass : 0.f;
					if (!bSimulated)
					{
						// Note that particles are always created disabled (why?). However,
						// SetEnabled only operates on dyanmics, so we need to enable the particle
						// before making it kinematic, otherwise we end up with particles that are
						// disabled, but still simulate.
						ActorHandle->SetEnabled(true);
						Simulation->SetIsKinematic(ActorHandle, true);
					}

					ActorHandle->SetName(BodySetup->BoneName);
					const int32 BodyIndex = AddBody(ActorHandle);
					const int32 SkeletonBoneIndex = MeshToSkeletonBoneIndex[InsertBone];
					if (ensure(SkeletonBoneIndex >= 0))
					{
						SkeletonBoneIndexToBodyIndex[SkeletonBoneIndex] = BodyIndex;
					}
					BodyAnimData[BodyIndex].bIsSimulated = bSimulated;
					NamesToHandles.Add(BodySetup->BoneName, ActorHandle);
					BodyIndexToActorHandle[BodyInstance->InstanceBodyIndex] = ActorHandle;

					if (BodySetup->CollisionReponse == EBodyCollisionResponse::BodyCollision_Disabled)
					{
						IgnoreCollisionActors.Add(ActorHandle);
					}
				}
			}
		}

		//Insert joints so that they coincide body order. That is, if we stop simulating all bodies past some index, we can simply ignore joints past a corresponding index without any re-order
		//For this to work we consider the most last inserted bone in each joint
		TArray<int32> InsertionOrderPerBone;
		InsertionOrderPerBone.AddUninitialized(NumBonesTotal);

		for(int32 Position = 0; Position < NumBonesLOD0; ++Position)
		{
			InsertionOrderPerBone[InsertionOrder[Position]] = Position;
		}

		HighLevelConstraintInstances.Sort([&InsertionOrderPerBone, &SkelMeshRefSkel](const FConstraintInstance& LHS, const FConstraintInstance& RHS)
		{
			if(LHS.IsValidConstraintInstance() && RHS.IsValidConstraintInstance())
			{
				const int32 BoneIdxLHS1 = SkelMeshRefSkel.FindBoneIndex(LHS.ConstraintBone1);
				const int32 BoneIdxLHS2 = SkelMeshRefSkel.FindBoneIndex(LHS.ConstraintBone2);

				const int32 BoneIdxRHS1 = SkelMeshRefSkel.FindBoneIndex(RHS.ConstraintBone1);
				const int32 BoneIdxRHS2 = SkelMeshRefSkel.FindBoneIndex(RHS.ConstraintBone2);

				const int32 MaxPositionLHS = FMath::Max(InsertionOrderPerBone[BoneIdxLHS1], InsertionOrderPerBone[BoneIdxLHS2]);
				const int32 MaxPositionRHS = FMath::Max(InsertionOrderPerBone[BoneIdxRHS1], InsertionOrderPerBone[BoneIdxRHS2]);

				return MaxPositionLHS < MaxPositionRHS;
			}
			
			return false;
		});

		Joints.SetNumZeroed(Bodies.Num());

		TArray<ImmediatePhysics::FSimulation::FIgnorePair> IgnorePairs;
		if(NamesToHandles.Num() > 0)
		{
			//constraints
			for(int32 ConstraintIdx = 0; ConstraintIdx < HighLevelConstraintInstances.Num(); ++ConstraintIdx)
			{
				FConstraintInstance* CI = HighLevelConstraintInstances[ConstraintIdx];
				ImmediatePhysics::FActorHandle* Body1Handle = NamesToHandles.FindRef(CI->ConstraintBone1);
				ImmediatePhysics::FActorHandle* Body2Handle = NamesToHandles.FindRef(CI->ConstraintBone2);

				if(Body1Handle && Body2Handle)
				{
					if (bMakeKinematicConstraints || (Body1Handle->IsSimulated() || Body2Handle->IsSimulated()))
					{
						ImmediatePhysics::FJointHandle* JointHandle =
							Simulation->CreateJoint(ImmediatePhysics::MakeJointSetup(CI, Body1Handle, Body2Handle));

						// Record the joint handle under the child bone (i.e. more leaf-ward bone),
						// since each bone may have multiple children, but only one parent.
						NamesToJointHandles.Add(CI->ConstraintBone1, JointHandle);

						if (bForceDisableCollisionBetweenConstraintBodies)
						{
							int32 BodyIndex1 = PhysicsAssetToUse->FindBodyIndex(CI->ConstraintBone1);
							int32 BodyIndex2 = PhysicsAssetToUse->FindBodyIndex(CI->ConstraintBone2);
							if (BodyIndex1 != INDEX_NONE && BodyIndex2 != INDEX_NONE)
							{
								PhysicsAssetToUse->DisableCollision(BodyIndex1, BodyIndex2);
							}
						}

						int32 BodyIndex;
						if (Bodies.Find(Body1Handle, BodyIndex))
						{
							Joints[BodyIndex] = JointHandle;
						}

						if (CI->IsCollisionDisabled())
						{
							ImmediatePhysics::FSimulation::FIgnorePair Pair;
							Pair.A = Body1Handle;
							Pair.B = Body2Handle;
							IgnorePairs.Add(Pair);
						}
					}
				}
			}

			ResetSimulatedTeleportType = ETeleportType::ResetPhysics;
		}

		if (!bHaveSetupControls && bEnableControls)
		{
			SetupControls(InAnimInstance->GetSkelMeshComponent());
		}

		// Terminate all the constraint instances
		for (FConstraintInstance* CI : HighLevelConstraintInstances)
		{
			CI->TermConstraint();
			delete CI;
		}

		// Terminate all of the instances, cannot be done during insert or we may break constraint chains
		for(FBodyInstance* Instance : HighLevelBodyInstances)
		{
			if(Instance->IsValidBodyInstance())
			{
				Instance->TermBody(true);
			}

			delete Instance;
		}

		HighLevelConstraintInstances.Empty();
		HighLevelBodyInstances.Empty();
		BodiesSorted.Empty();

		const TMap<FRigidBodyIndexPair, bool>& DisableTable = PhysicsAssetToUse->CollisionDisableTable;
		for(auto ConstItr = DisableTable.CreateConstIterator(); ConstItr; ++ConstItr)
		{
			int32 IndexA = ConstItr.Key().Indices[0];
			int32 IndexB = ConstItr.Key().Indices[1];
			if ((IndexA < BodyIndexToActorHandle.Num()) && (IndexB < BodyIndexToActorHandle.Num()))
			{
				if ((BodyIndexToActorHandle[IndexA] != nullptr) && (BodyIndexToActorHandle[IndexB] != nullptr))
				{
					ImmediatePhysics::FSimulation::FIgnorePair Pair;
					Pair.A = BodyIndexToActorHandle[IndexA];
					Pair.B = BodyIndexToActorHandle[IndexB];
					IgnorePairs.Add(Pair);
				}
			}
		}

		Simulation->SetIgnoreCollisionPairTable(IgnorePairs);
		Simulation->SetIgnoreCollisionActors(IgnoreCollisionActors);

		CollectClothColliderObjects(SkeletalMeshComp);

		SolverSettings = PhysicsAssetToUse->SolverSettings;
		Simulation->SetSolverSettings(
			SolverSettings.FixedTimeStep,
			SolverSettings.CullDistance,
			SolverSettings.MaxDepenetrationVelocity,
			SolverSettings.bUseLinearJointSolver,
			SolverSettings.PositionIterations,
			SolverSettings.VelocityIterations,
			SolverSettings.ProjectionIterations,
			SolverSettings.bUseManifolds);

		SolverIterations = PhysicsAssetToUse->SolverIterations;

		ApplyCurrentConstraintProfile();
	}
}

DECLARE_CYCLE_STAT(TEXT("FAnimNode_RigidBodyWithControl::UpdateWorldGeometry"), STAT_ImmediateUpdateWorldGeometry, STATGROUP_ImmediatePhysics);

void FAnimNode_RigidBodyWithControl::UpdateWorldGeometry(const UWorld& World, const USkeletalMeshComponent& SKC)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigidBodyWithControl")); 
	
	SCOPE_CYCLE_COUNTER(STAT_ImmediateUpdateWorldGeometry);
	QueryParams = FCollisionQueryParams(SCENE_QUERY_STAT(RagdollNodeFindGeometry), /*bTraceComplex=*/false);
	// The Mobility type is ignored anyway - see UE-168341
	// When that's fixed, we could expose this, as we might not want to collide with dynamic objects etc
	QueryParams.MobilityType = EQueryMobilityType::Any;	
	QueryParams.AddIgnoredComponent(&SKC);

	// Check for deleted world objects and flag for removal (later in anim task)
	ExpireWorldObjects();

	// If we have moved outside of the bounds we checked for world objects we need to gather new world objects
	FSphere Bounds = SKC.CalcBounds(SKC.GetComponentToWorld()).GetSphere();
	if (!Bounds.IsInside(CachedBounds) || bUpdateCacheEveryFrame)
	{
		// Since the cached bounds are no longer valid, update them.
		CachedBounds = Bounds;
		CachedBounds.W *= CachedBoundsScale;

		// Cache the World for use in UpdateWorldForces and CollectWorldObjects. When
		// these are non-null it is an indicator that we need to update the collected world objects
		// list in CollectWorldObjects
		UnsafeWorld = &World;
		UnsafeOwner = SKC.GetOwner();

		// A timer to track objects we haven't detected in a while
		++ComponentsInSimTick;
	}
}

DECLARE_CYCLE_STAT(TEXT("FAnimNode_RigidBodyWithControl::UpdateWorldForces"), STAT_ImmediateUpdateWorldForces, STATGROUP_ImmediatePhysics);

void FAnimNode_RigidBodyWithControl::UpdateWorldForces(const FTransform& ComponentToWorld, const FTransform& BaseBoneTM, const float DeltaSeconds)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigidBodyWithControl")); 
	SCOPE_CYCLE_COUNTER(STAT_ImmediateUpdateWorldForces);

	if(TotalMass > 0.f)
	{
		for (const USkeletalMeshComponent::FPendingRadialForces& PendingRadialForce : PendingRadialForces)
		{
			const FVector RadialForceOrigin = WorldPositionToSpace(SimulationSpace, PendingRadialForce.Origin, ComponentToWorld, BaseBoneTM);
			for(ImmediatePhysics::FActorHandle* Body : Bodies)
			{
				const float InvMass = (float) Body->GetInverseMass();
				if(InvMass > 0.f)
				{
					const float StrengthPerBody = PendingRadialForce.bIgnoreMass ? PendingRadialForce.Strength : PendingRadialForce.Strength / (TotalMass * InvMass);
					ImmediatePhysics::EForceType ForceType;
					if (PendingRadialForce.Type == USkeletalMeshComponent::FPendingRadialForces::AddImpulse)
					{
						ForceType = PendingRadialForce.bIgnoreMass ? ImmediatePhysics::EForceType::AddVelocity : ImmediatePhysics::EForceType::AddImpulse;
					}
					else
					{
						ForceType = PendingRadialForce.bIgnoreMass ? ImmediatePhysics::EForceType::AddAcceleration : ImmediatePhysics::EForceType::AddForce;
					}
					
					Body->AddRadialForce(RadialForceOrigin, StrengthPerBody, PendingRadialForce.Radius, PendingRadialForce.Falloff, ForceType);
				}
			}
		}

		if(!ExternalForce.IsNearlyZero())
		{
			const FVector ExternalForceInSimSpace = WorldVectorToSpaceNoScale(SimulationSpace, ExternalForce, ComponentToWorld, BaseBoneTM);
			for (ImmediatePhysics::FActorHandle* Body : Bodies)
			{
				const float InvMass = (float) Body->GetInverseMass();
				if (InvMass > 0.f)
				{
					Body->AddForce(ExternalForceInSimSpace);
				}
			}
		}
		if(DeltaSeconds != 0.0)
		{
			if(!PerSolverField.IsEmpty())
			{
				TArray<FVector>& SamplePositions = PerSolverField.GetSamplePositions();
				TArray<FFieldContextIndex>& SampleIndices = PerSolverField.GetSampleIndices();

				SamplePositions.SetNum(Bodies.Num(),EAllowShrinking::No);
				SampleIndices.SetNum(Bodies.Num(), EAllowShrinking::No);

				int32 Index = 0;
				for (ImmediatePhysics::FActorHandle* Body : Bodies)
				{
					SamplePositions[Index] = (Body->GetWorldTransform() * SpaceToWorldTransform(SimulationSpace, ComponentToWorld, BaseBoneTM)).GetLocation();
					SampleIndices[Index] = FFieldContextIndex(Index, Index);
					++Index;
				}
				PerSolverField.ComputeFieldRigidImpulse(WorldTimeSeconds);

				const TArray<FVector>& LinearVelocities = PerSolverField.GetOutputResults(EFieldCommandOutputType::LinearVelocity);
				const TArray<FVector>& LinearForces = PerSolverField.GetOutputResults(EFieldCommandOutputType::LinearForce);
				const TArray<FVector>& AngularVelocities = PerSolverField.GetOutputResults(EFieldCommandOutputType::AngularVelocity);
				const TArray<FVector>& AngularTorques = PerSolverField.GetOutputResults(EFieldCommandOutputType::AngularTorque);

				if (LinearVelocities.Num() == Bodies.Num())
				{
					Index = 0;
					for (ImmediatePhysics::FActorHandle* Body : Bodies)
					{
						const FVector ExternalForceInSimSpace = WorldVectorToSpaceNoScale(SimulationSpace, LinearVelocities[Index++], ComponentToWorld, BaseBoneTM) * Body->GetMass() / DeltaSeconds;
						Body->AddForce(ExternalForceInSimSpace);
					}
				}
				if (LinearForces.Num() == Bodies.Num())
				{
					Index = 0;
					for (ImmediatePhysics::FActorHandle* Body : Bodies)
					{
						const FVector ExternalForceInSimSpace = WorldVectorToSpaceNoScale(SimulationSpace, LinearForces[Index++], ComponentToWorld, BaseBoneTM);
						Body->AddForce(ExternalForceInSimSpace);
					}
				}
				if (AngularVelocities.Num() == Bodies.Num())
				{
					Index = 0;
					for (ImmediatePhysics::FActorHandle* Body : Bodies)
					{
						const FVector ExternalTorqueInSimSpace = WorldVectorToSpaceNoScale(SimulationSpace, AngularVelocities[Index++], ComponentToWorld, BaseBoneTM) * Body->GetInertia() / DeltaSeconds;
						Body->AddTorque(ExternalTorqueInSimSpace);
					}
				}
				if (AngularTorques.Num() == Bodies.Num())
				{
					Index = 0;
					for (ImmediatePhysics::FActorHandle* Body : Bodies)
					{
						const FVector ExternalTorqueInSimSpace = WorldVectorToSpaceNoScale(SimulationSpace, AngularTorques[Index++], ComponentToWorld, BaseBoneTM);
						Body->AddTorque(ExternalTorqueInSimSpace);
					}
				}
			}
		}
	}
}

bool FAnimNode_RigidBodyWithControl::NeedsDynamicReset() const
{
	return true;
}

void FAnimNode_RigidBodyWithControl::ResetDynamics(ETeleportType InTeleportType)
{
	// This will be picked up next evaluate and reset our simulation.
	// Teleport type can only go higher - i.e. if we have requested a reset, then a teleport will still reset fully
	ResetSimulatedTeleportType = ((InTeleportType > ResetSimulatedTeleportType) ? InTeleportType : ResetSimulatedTeleportType);
}

void FAnimNode_RigidBodyWithControl::SetOverridePhysicsAsset(UPhysicsAsset* PhysicsAsset)
{
	OverridePhysicsAsset = PhysicsAsset;
}

void FAnimNode_RigidBodyWithControl::PreUpdate(const UAnimInstance* InAnimInstance)
{
	USkeletalMeshComponent* const SKC = InAnimInstance->GetSkelMeshComponent();

	// Re-initialize physics if the physics asset to be used has changed (This can happen when 
	// using the override physics asset feature) or if the Skeletal Mesh has changed and Physics
	// Asset Conditioning is required.
	UPhysicsAsset* PhysicsAssetToBeUsed = GetPhysicsAssetToBeUsed(InAnimInstance);
	const USkeletalMesh* const SkeletalMeshAsset = (SKC) ? SKC->GetSkeletalMeshAsset() : nullptr;

	if ((GetPhysicsAsset() != PhysicsAssetToBeUsed) || ((SkeletalMeshAsset != SkeletalMeshAtLastPhysicsInit) && ShouldApplyPhysicsAssetConditioning()))
	{
		InitPhysics(InAnimInstance);

		// Update the bone references after a change in the physics asset. This needs to happen
		// after initializing physics as the Bodies set up in InitPhysics() need to be up to date.
		InitializeBoneReferences(InAnimInstance->GetRequiredBones());
	}

	// Don't update geometry if RBN is disabled
	if(!bEnabled)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_PreUpdate);

	APawn* PawnOwner = InAnimInstance->TryGetPawnOwner();
	UPawnMovementComponent* MovementComp = PawnOwner ? PawnOwner->GetMovementComponent() : nullptr;

#if WITH_EDITOR
	if (bEnableWorldGeometry && SimulationSpace != ESimulationSpace::WorldSpace && SKC && SKC->GetRelativeScale3D() != FVector(1.f, 1.f, 1.f))
	{
		FMessageLog("PIE").Warning(FText::Format(LOCTEXT("WorldCollisionComponentSpace", "Trying to use world collision without world space simulation for scaled ''{0}''. This is not supported, please change SimulationSpace to WorldSpace"),
			FText::FromString(GetPathNameSafe(SKC))));
	}
#endif

	UWorld* World = InAnimInstance->GetWorld();
	if (World)
	{
		WorldSpaceGravity = bOverrideWorldGravity ? OverrideWorldGravity : (MovementComp ? FVector(0.f, 0.f, MovementComp->GetGravityZ()) : FVector(0.f, 0.f, World->GetGravityZ()));

		if(SKC)
		{
			// Store game time for use in parallel evaluation. This may be the totol time (inc pauses) or the time the game has been unpaused.
			WorldTimeSeconds = (float) (SKC->PrimaryComponentTick.bTickEvenWhenPaused ? World->UnpausedTimeSeconds : World->TimeSeconds);

			if (Simulation && bEnableWorldGeometry)
			{ 
				// @todo: this logic can be simplified now. We used to run PurgeExpiredWorldObjects and CollectWorldObjects
				// in UpdateAnimation, but we can't access the world actor's geometry there
				UpdateWorldGeometry(*World, *SKC);

				// Remove expired objects from the sim
				PurgeExpiredWorldObjects();

				// Find nearby world objects to add to the sim (gated on UnsafeWorld - see UpdateWorldGeometry)
				CollectWorldObjects();
			}

			PendingRadialForces = SKC->GetPendingRadialForces();

			PreviousTransform = CurrentTransform;
			CurrentTransform = SKC->GetComponentToWorld();

			if (World->PhysicsField)
			{
				const FBox BoundingBox = SKC->CalcBounds(SKC->GetComponentTransform()).GetBox();

				World->PhysicsField->FillTransientCommands(false, BoundingBox, WorldTimeSeconds, PerSolverField.GetTransientCommands());
				World->PhysicsField->FillPersistentCommands(false, BoundingBox, WorldTimeSeconds, PerSolverField.GetPersistentCommands());
			}
		}
	}
}

int32 FAnimNode_RigidBodyWithControl::GetLODThreshold() const
{
	if(CVarRigidBodyNodeWithControlLODThreshold.GetValueOnAnyThread() != -1)
	{
		if(LODThreshold != -1)
		{
			return FMath::Min(LODThreshold, CVarRigidBodyNodeWithControlLODThreshold.GetValueOnAnyThread());
		}
		else
		{
			return CVarRigidBodyNodeWithControlLODThreshold.GetValueOnAnyThread();
		}
	}
	else
	{
		return LODThreshold;
	}
}

void FAnimNode_RigidBodyWithControl::UpdateInternal(const FAnimationUpdateContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigidBodyWithControl")); 
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(UpdateInternal)
	// Avoid this work if RBN is disabled, as the results would be discarded
	if(!bEnabled)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_Update);
	
	// Must flush the simulation since we may be making changes to the scene
	FlushDeferredSimulationTask();

	// Accumulate deltatime elapsed during update. To be used during evaluation.
	AccumulatedDeltaTime += Context.AnimInstanceProxy->GetDeltaSeconds();

	if (UnsafeWorld != nullptr)
	{
		// Node is valid to evaluate. Simulation is starting.
		bSimulationStarted = true;
	}

	// These get set again if our bounds change. Subsequent calls to CollectWorldObjects will early-out until then
	UnsafeWorld = nullptr;
	UnsafeOwner = nullptr;
}

void FAnimNode_RigidBodyWithControl::CollectClothColliderObjects(const USkeletalMeshComponent* SkeletalMeshComp)
{
	if (bUseExternalClothCollision && bRBANWithControl_IncludeClothColliders && SkeletalMeshComp && Simulation)
	{
		const TArray<FClothCollisionSource>& SkeletalMeshClothCollisionSources = SkeletalMeshComp->GetClothCollisionSources();
		
		for (const FClothCollisionSource& ClothCollisionSource : SkeletalMeshClothCollisionSources)
		{
			const USkeletalMeshComponent* const SourceComponent = ClothCollisionSource.SourceComponent.Get();
			const UPhysicsAsset* const PhysicsAsset = ClothCollisionSource.SourcePhysicsAsset.Get();

			if (SourceComponent && PhysicsAsset)
			{
				TArray<FBodyInstance*> BodyInstances;
				SourceComponent->InstantiatePhysicsAssetBodies(*PhysicsAsset, BodyInstances);

				for (uint32 BodyInstanceIndex = 0, BodyInstanceMax = BodyInstances.Num(); BodyInstanceIndex < BodyInstanceMax; ++BodyInstanceIndex)
				{
					FBodyInstance* const BodyInstance = BodyInstances[BodyInstanceIndex];

					ImmediatePhysics::FActorHandle* const ActorHandle = Simulation->CreateActor(ImmediatePhysics::MakeKinematicActorSetup(BodyInstance, BodyInstance->GetUnrealWorldTransform()));
					Simulation->AddToCollidingPairs(ActorHandle); // <-allow collision between this actor and all dynamic actors.
					ClothColliders.Add(FClothCollider(ActorHandle, SourceComponent, BodyInstance->InstanceBoneIndex));

					// Terminate the instance.
					if (BodyInstance->IsValidBodyInstance())
					{
						BodyInstance->TermBody(true);
					}

					delete BodyInstance;
					BodyInstances[BodyInstanceIndex] = nullptr;
				}

				BodyInstances.Reset();
			}
		}
	}
}

void FAnimNode_RigidBodyWithControl::RemoveClothColliderObjects()
{
	for (const FClothCollider& ClothCollider : ClothColliders)
	{
		Simulation->DestroyActor(ClothCollider.ActorHandle);
	}
	
	ClothColliders.Reset();
}

void FAnimNode_RigidBodyWithControl::UpdateClothColliderObjects()
{
	for (FClothCollider& ClothCollider : ClothColliders)
	{
		const USkeletalMeshComponent* const SkeletalMeshComp = ClothCollider.SkeletalMeshComponent.Get();
		if (ClothCollider.ActorHandle && SkeletalMeshComp)
		{
			// Calculate the sim-space transform of this object
			const FTransform CompWorldTransform = SkeletalMeshComp->GetBoneTransform(ClothCollider.BoneIndex);
			FTransform CompSpaceTransform;
			CompSpaceTransform.SetTranslation(SimulationSpaceTransform.InverseTransformPosition(
				CompWorldTransform.GetLocation()));
			CompSpaceTransform.SetRotation(SimulationSpaceTransform.InverseTransformRotation(
				CompWorldTransform.GetRotation()));
			CompSpaceTransform.SetScale3D(FVector::OneVector);	// TODO - sort out scale for world objects in local sim

			// Update the sim's copy of the world object
			ClothCollider.ActorHandle->SetWorldTransform(CompSpaceTransform);
		}
	}
}

inline bool IsComponentCollisionEnabled(const FAnimNode_RigidBodyWithControl* Node, const UPrimitiveComponent* const Component)
{
	if (Component->GetCollisionResponseToChannel(ECC_PhysicsBody) != ECollisionResponse::ECR_Block)
	{
		return false;
	}

	const ECollisionEnabled::Type CollisionEnabled = Component->GetBodyInstance()->GetCollisionEnabled();

	return (Node->bWorldGeometryCollisionEnabledHasPhysics && CollisionEnabledHasPhysics(CollisionEnabled))
		|| (Node->bWorldGeometryCollisionEnabledHasQuery && CollisionEnabledHasQuery(CollisionEnabled))
		|| (Node->bWorldGeometryCollisionEnabledHasProbe && CollisionEnabledHasProbe(CollisionEnabled));
}

inline bool IsComponentDesiredInSim(const FAnimNode_RigidBodyWithControl* Node, const UPrimitiveComponent* Component)
{
	if (!IsValid(Component))
	{
		return false;
	}
	if (!Component->GetBodyInstance())
	{
		return false;
	}
	if (!Component->GetBodyInstance()->IsValidBodyInstance())
	{
		return false;
	}
	if(!IsComponentCollisionEnabled(Node, Component))
	{
		return false;
	}
	return true;
}

// TODO I think this can crash when running deferred, since it accesses UnsafeWorld from a worker thread
void FAnimNode_RigidBodyWithControl::CollectWorldObjects()
{
	if (UnsafeWorld)
	{
		// @todo(ccaulfield): should this use CachedBounds?
		TArray<FOverlapResult> Overlaps;
		UnsafeWorld->OverlapMultiByChannel(
			Overlaps, CachedBounds.Center, FQuat::Identity, OverlapChannel, 
			FCollisionShape::MakeSphere((float) CachedBounds.W), 
			QueryParams, FCollisionResponseParams(ECR_Overlap));

		for (const FOverlapResult& Overlap : Overlaps)
		{
			if (UPrimitiveComponent* OverlapComp = Overlap.GetComponent())
			{
				FWorldObject* WorldObject = ComponentsInSim.Find(OverlapComp);
				if (WorldObject != nullptr)
				{
					// Existing object - reset its age
					WorldObject->LastSeenTick = ComponentsInSimTick;
				}
				else
				{
					// New object - add it to the sim
					const bool bIsSelf = (UnsafeOwner == OverlapComp->GetOwner());

					if (!bIsSelf && IsComponentDesiredInSim(this, OverlapComp))
					{
						// Note that the TM here isn't correct unless we are actually using
						// world-space simulation, but it will be updated.
						FTransform TM = OverlapComp->GetComponentTransform();
						// Create a kinematic actor. Not using Static as world-static objects may
						// move in the simulation's frame of reference. Note that TM may be in the wrong space here
						ImmediatePhysics::FActorHandle* ActorHandle = Simulation->CreateActor(
							ImmediatePhysics::MakeKinematicActorSetup(&OverlapComp->BodyInstance, TM));
						ActorHandle->SetName(FName(OverlapComp->GetName()));
						Simulation->AddToCollidingPairs(ActorHandle);
						ComponentsInSim.Add(
							OverlapComp, FWorldObject(ActorHandle, ComponentsInSimTick, Overlap.Component));
						// We would need this: 
						// OverlapComp->BodyInstance.bUpdateKinematicFromSimulation = true;
						// in order to get velocities from kinematic objects, but we can't as there's no way of 
						// reliably undoing it (multiple RBWC nodes might be interacting with the same world object).
						// So for now accept that we won't be able to pull velocities from kinematics.
					}
				}
			}
		}
	}
}

// Flag invalid objects for purging
void FAnimNode_RigidBodyWithControl::ExpireWorldObjects()
{
	// Invalidate deleted and expired world objects
	TArray<const UPrimitiveComponent*> PrunedEntries;
	for (auto& WorldEntry : ComponentsInSim)
	{
		FWorldObject& WorldObject = WorldEntry.Value;
		const UPrimitiveComponent* WorldComp = WorldObject.Component.Get();

		// Do we need to expire this object?
		const int32 ExpireTickCount = RBANWithControl_WorldObjectExpiry;
		if (((ComponentsInSimTick - WorldObject.LastSeenTick) > ExpireTickCount)	// Haven't seen this object for a while
			|| !IsComponentDesiredInSim(this, WorldComp) )
		{
			WorldObject.bExpired = true;
		}
	}
}

void FAnimNode_RigidBodyWithControl::PurgeExpiredWorldObjects()
{
	// Destroy expired simulated objects
	TArray<const UPrimitiveComponent*> PurgedEntries;
	for (auto& WorldEntry : ComponentsInSim)
	{
		FWorldObject& WorldObject = WorldEntry.Value;

		if (WorldObject.bExpired)
		{
			Simulation->DestroyActor(WorldObject.ActorHandle);
			WorldObject.ActorHandle = nullptr;

			PurgedEntries.Add(WorldEntry.Key);
		}
	}

	// Remove purged map entries
	for (const UPrimitiveComponent* PurgedEntry : PurgedEntries)
	{
		ComponentsInSim.Remove(PurgedEntry);
	}
}

// Update the transforms of the world objects we added to the sim. This could be because we're a
// component-based simulation and need to update stationary objects into our space, or simply
// because the objects are moving.
void FAnimNode_RigidBodyWithControl::UpdateWorldObjects(
	const FVector& LinearVel, const FVector& AngularVel, const float DeltaSeconds)
{
	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_UpdateWorldObjects);
	LLM_SCOPE_BYNAME(TEXT("Animation/RigidBodyWithControl"));

	if (ComponentsInSim.IsEmpty())
	{
		return;
	}

	if (bCalculateVelocitiesForWorldGeometry)
	{
		// We will want the "previous" simulation space to calculate velocities. Get it from the
		// velocity, so we can take advantage of that being zero at initialisation/reset times etc.
		const FVector PrevSimSpacePosition = 
			SimulationSpaceTransform.GetLocation() - LinearVel * DeltaSeconds;
		const FQuat SimSpaceDeltaOrientation = FQuat::MakeFromRotationVector(AngularVel * DeltaSeconds);
		const FQuat PrevSimSpaceOrientation = 
			SimSpaceDeltaOrientation.Inverse() * SimulationSpaceTransform.GetRotation();
		const FTransform PrevSimSpaceTransform(PrevSimSpaceOrientation, PrevSimSpacePosition, FVector::OneVector);

		for (const auto& WorldEntry : ComponentsInSim)
		{ 
			const UPrimitiveComponent* OverlapComp = WorldEntry.Value.Component.Get();
			if (OverlapComp != nullptr)
			{
				ImmediatePhysics::FActorHandle* ActorHandle = WorldEntry.Value.ActorHandle;

				// Calculate the sim-space transform of this object
				const FTransform CompWorldTransform = OverlapComp->BodyInstance.GetUnrealWorldTransform();
				const FVector CompWorldPosition = CompWorldTransform.GetLocation();
				const FQuat CompWorldOrientation = CompWorldTransform.GetRotation();

				const FVector CompSimSpacePosition = SimulationSpaceTransform.InverseTransformPosition(CompWorldPosition);
				const FQuat CompSimSpaceOrientation = SimulationSpaceTransform.InverseTransformRotation(CompWorldOrientation);

				// TODO - sort out scale for world objects in local sim
				const FTransform CompSimSpaceTransform(CompSimSpaceOrientation, CompSimSpacePosition, FVector::OneVector);

				// We need to set the velocity - either because the object or the space is
				// moving. Ideally would do this by setting the velocity (with transformations), but
				// it's not possible to set the velocity of kinematics. Use the kinematic target to
				// force the velocities. This should handle initialization and teleportation too.

				// We also have the problem that velocities are not reported from kinematics, unless
				// OverlapComp->BodyInstance.bUpdateKinematicFromSimulation = true;

				const FVector Velocity = OverlapComp->BodyInstance.GetUnrealWorldVelocity();
				const FVector AngularVelocity = OverlapComp->BodyInstance.GetUnrealWorldAngularVelocityInRadians();

				// To get the previous component space transform, we estimate the previous world space
				// transform, and then put that into the previous sim space.

				const FVector PrevCompWorldPosition = CompWorldPosition - Velocity * DeltaSeconds;
				const FQuat DeltaWorldOrientation = FQuat::MakeFromRotationVector(AngularVelocity * DeltaSeconds);
				const FQuat PrevCompWorldOrientation = DeltaWorldOrientation.Inverse() * CompWorldOrientation;

				const FVector PrevCompSimSpacePosition = PrevSimSpaceTransform.InverseTransformPosition(CompWorldPosition);
				const FQuat PrevCompSimSpaceOrientation = PrevSimSpaceTransform.InverseTransformRotation(CompWorldOrientation);
				const FTransform PrevCompSimSpaceTransform(PrevCompSimSpaceOrientation, PrevCompSimSpacePosition, FVector::OneVector);

				ActorHandle->InitWorldTransform(PrevCompSimSpaceTransform);
				ActorHandle->SetKinematicTarget(CompSimSpaceTransform);
			}
		}
	}
	else
	{
		for (const auto& WorldEntry : ComponentsInSim)
		{
			const UPrimitiveComponent* OverlapComp = WorldEntry.Value.Component.Get();
			if (OverlapComp != nullptr)
			{
				ImmediatePhysics::FActorHandle* ActorHandle = WorldEntry.Value.ActorHandle;

				// Calculate the sim-space transform of this object
				const FTransform CompWorldTransform = OverlapComp->BodyInstance.GetUnrealWorldTransform();
				const FVector CompWorldPosition = CompWorldTransform.GetLocation();
				const FQuat CompWorldOrientation = CompWorldTransform.GetRotation();

				const FVector CompSimSpacePosition = SimulationSpaceTransform.InverseTransformPosition(CompWorldPosition);
				const FQuat CompSimSpaceOrientation = SimulationSpaceTransform.InverseTransformRotation(CompWorldOrientation);

				// TODO - sort out scale for world objects in local sim
				const FTransform CompSimSpaceTransform(CompSimSpaceOrientation, CompSimSpacePosition, FVector::OneVector);

				ActorHandle->InitWorldTransform(CompSimSpaceTransform);
			}
		}
	}
}

void FAnimNode_RigidBodyWithControl::InitializeBoneReferences(const FBoneContainer& RequiredBones) 
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(InitializeBoneReferences)
	/** We only need to update simulated bones and children of simulated bones*/
	const int32 NumBodies = Bodies.Num();
	OutputBoneData.Empty(NumBodies);

	if (!RequiredBones.IsValid())
	{
		UE_LOGF(LogPhysicsControl, Warning, "FAnimNode_RigidBodyWithControl::InitializeBoneReferences: Required bones is invalid for Skeleton %ls / SkeletalMesh %ls.",
			*GetNameSafe(RequiredBones.GetSkeletonAsset()), *GetNameSafe(RequiredBones.GetSkeletalMeshAsset()));
		return;
	}
	const FReferenceSkeleton& RefSkeleton = RequiredBones.GetReferenceSkeleton();
	const TArray<FBoneIndexType>& RequiredBoneIndices = RequiredBones.GetBoneIndicesArray();
	const int32 NumRequiredBoneIndices = RequiredBoneIndices.Num();

	int32 OutputBoneDataMaxBodyIndex = -1;

	int32 NumSimulatedBodies = 0;
	TArray<int32> SimulatedBodyIndices;
	// if no name is entered, use root
	if (BaseBoneRef.BoneName == NAME_None)
	{
		BaseBoneRef.BoneName = RefSkeleton.GetBoneName(0);
	}

	if (BaseBoneRef.BoneName != NAME_None)
	{
		BaseBoneRef.Initialize(RequiredBones);
	}

	if (!BaseBoneRef.HasValidSetup())
	{
		// If the user specified a simulation root that is not used by the skelmesh, issue a warning 
		// (FAnimNode_RigidBodyWithControl::IsValidToEvaluate will return false and the simulation will not run)
		UE_LOGF(LogPhysicsControl, Log, "FAnimNode_RigidBodyWithControl: RigidBodyWithControl Simulation Base Bone \'%ls\' does not exist on SkeletalMesh %ls.", *BaseBoneRef.BoneName.ToString(), *GetNameSafe(RequiredBones.GetSkeletalMeshAsset()));
	}

	bool bHasInvalidBoneReference = false;
	for (int32 Index = 0; Index < NumRequiredBoneIndices; ++Index)
	{
		const FCompactPoseBoneIndex CompactPoseBoneIndex(Index);
		const FBoneIndexType SkeletonBoneIndex = (FBoneIndexType) RequiredBones.GetSkeletonIndex(CompactPoseBoneIndex);
		const FBoneIndexType IndexToBodyNum = (FBoneIndexType) SkeletonBoneIndexToBodyIndex.Num();

		// If we have a missing bone in our skeleton, we don't want to have an out of bounds access.
		if (SkeletonBoneIndex >= IndexToBodyNum)
		{
			bHasInvalidBoneReference = true;
			break;
		}

		const int32 BodyIndex = SkeletonBoneIndexToBodyIndex[SkeletonBoneIndex];

		if (BodyIndex != INDEX_NONE)
		{
			//If we have a body we need to save it for later
			UE::PhysicsControl::FOutputBoneData& OutputData = OutputBoneData.AddDefaulted_GetRef();
			OutputData.BodyIndex = BodyIndex;
			OutputBoneDataMaxBodyIndex = FMath::Max(OutputBoneDataMaxBodyIndex, BodyIndex);
			OutputData.CompactPoseBoneIndex = CompactPoseBoneIndex;

			if (BodyAnimData[BodyIndex].bIsSimulated)
			{
				++NumSimulatedBodies;
				SimulatedBodyIndices.AddUnique(BodyIndex);
			}

			OutputData.BoneIndicesToParentBody.Add(CompactPoseBoneIndex);

			// Walk up parent chain until we find parent body.
			OutputData.ParentBodyIndex = INDEX_NONE;
			FCompactPoseBoneIndex CompactParentIndex = RequiredBones.GetParentBoneIndex(CompactPoseBoneIndex);
			OutputData.CompactPoseParentBoneIndex = CompactParentIndex;
			while (CompactParentIndex != INDEX_NONE)
			{
				const FBoneIndexType SkeletonParentBoneIndex = (FBoneIndexType ) RequiredBones.GetSkeletonIndex(CompactParentIndex);

				// Must check our parent as well for a missing bone.
				if (SkeletonParentBoneIndex >= IndexToBodyNum)
				{
					bHasInvalidBoneReference = true;
					break;
				}

				OutputData.ParentBodyIndex = SkeletonBoneIndexToBodyIndex[SkeletonParentBoneIndex];
				OutputData.CompactPoseParentBoneIndex = CompactParentIndex;
				if (OutputData.ParentBodyIndex != INDEX_NONE)
				{
					break;
				}

				OutputData.BoneIndicesToParentBody.Add(CompactParentIndex);
				CompactParentIndex = RequiredBones.GetParentBoneIndex(CompactParentIndex);
			}

			if (bHasInvalidBoneReference)
			{
				break;
			}
		}
	}

	if (bHasInvalidBoneReference)
	{
		// If a bone was missing, let us know which asset it happened on, and clear our bone container to make the bad asset visible.
		ensureMsgf(false, TEXT("FAnimNode_RigidBodyWithControl::InitializeBoneReferences: The Skeleton %s, is missing bones that SkeletalMesh %s needs. Skeleton might need to be resaved."),
			*GetNameSafe(RequiredBones.GetSkeletonAsset()), *GetNameSafe(RequiredBones.GetSkeletalMeshAsset()));
		OutputBoneData.Empty();
	}
	else
	{
		// The Pose data cache will only be indexed using the body indices from OutputBoneData
		PoseData.SetSize(OutputBoneDataMaxBodyIndex + 1);

		// New bodies potentially introduced with new LOD
		// We'll have to initialize their transform.
		bCheckForBodyTransformInit = true;

		if (Simulation)
		{
			Simulation->SetNumActiveBodies(NumSimulatedBodies, SimulatedBodyIndices);
		}

		// We're switching to a new LOD, this invalidates our captured poses.
		CapturedFrozenPose.Empty();
		CapturedFrozenCurves.Empty();
	}
}

void FAnimNode_RigidBodyWithControl::AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName)
{
	// Find the body. This is currently only used in the editor and will need optimizing if used in game
	for (int32 BodyIndex = 0; BodyIndex < Bodies.Num(); ++BodyIndex)
	{
		ImmediatePhysics::FActorHandle* Body = Bodies[BodyIndex];
		if (Body->GetName() == BoneName)
		{
			Body->AddImpulseAtLocation(Impulse, Location);
		}
	}
}

void FAnimNode_RigidBodyWithControl::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
	InitPhysics(InAnimInstance);
}

#if WITH_EDITORONLY_DATA
void FAnimNode_RigidBodyWithControl::PostSerialize(const FArchive& Ar)
{
}
#endif

bool FAnimNode_RigidBodyWithControl::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones)
{
	return BaseBoneRef.IsValidToEvaluate(RequiredBones);
}

const int32 FAnimNode_RigidBodyWithControl::GetNumBodies() const
{
	return Bodies.Num();
}



#undef LOCTEXT_NAMESPACE

