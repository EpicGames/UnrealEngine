// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlActor.h"
#include "PhysicsControlComponent.h"
#include "PhysicsControlLimbData.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/MeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsControlActor)

FInitialCharacterControls::FInitialCharacterControls() = default;
FInitialCharacterControls::~FInitialCharacterControls() = default;

//======================================================================================================================
void UPhysicsControlInitializerComponent::CreateInitialCharacterControls(UPhysicsControlComponent* ControlComponent)
{
	USkeletalMeshComponent* SkeletalMeshComponent = nullptr;

	if (InitialCharacterControls.CharacterActor.IsValid())
	{
		if (InitialCharacterControls.SkeletalMeshComponentName != NAME_None)
		{
			for (UActorComponent* Component : InitialCharacterControls.CharacterActor->GetComponents())
			{
				if (Component->GetFName() == InitialCharacterControls.SkeletalMeshComponentName)
				{
					SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Component);
					break;
				}
			}
		}
		else
		{
			SkeletalMeshComponent = InitialCharacterControls.CharacterActor->GetComponentByClass<USkeletalMeshComponent>();
		}
	}

	if (SkeletalMeshComponent)
	{
		FPhysicsControlNames AllWorldSpaceControls;
		TMap<FName, FPhysicsControlNames> LimbWorldSpaceControls;
		FPhysicsControlNames AllParentSpaceControls;
		TMap<FName, FPhysicsControlNames> LimbParentSpaceControls;
		TMap<FName, FPhysicsControlNames> LimbBodyModifiers;
		FPhysicsControlNames AllBodyModifiers;

		ControlComponent->CreateControlsAndBodyModifiersFromLimbBones(
			AllWorldSpaceControls, LimbWorldSpaceControls,
			AllParentSpaceControls, LimbParentSpaceControls,
			AllBodyModifiers, LimbBodyModifiers,
			SkeletalMeshComponent,
			InitialCharacterControls.LimbSetupData,
			InitialCharacterControls.WorldSpaceControlData,
			InitialCharacterControls.ParentSpaceControlData,
			InitialCharacterControls.BodyModifierData
		);
	}
}

//======================================================================================================================
void UPhysicsControlInitializerComponent::CreateOrUpdateInitialControls(UPhysicsControlComponent* ControlComponent)
{
	// Create any individual controls
	for (const TPair<FName, FInitialPhysicsControl>& InitialPhysicsControlPair : InitialControls)
	{
		const FName Name = InitialPhysicsControlPair.Key;
		const FInitialPhysicsControl& InitialPhysicsControl = InitialPhysicsControlPair.Value;

		UMeshComponent* ChildMeshComponent = nullptr;
		if (InitialPhysicsControl.ChildActor.IsValid())
		{
			if (InitialPhysicsControl.ChildMeshComponentName != NAME_None)
			{
				for (UActorComponent* Component : InitialPhysicsControl.ChildActor->GetComponents())
				{
					if (Component->GetFName() == InitialPhysicsControl.ChildMeshComponentName)
					{
						ChildMeshComponent = Cast<UMeshComponent>(Component);
						break;
					}
				}
			}
			else
			{
				ChildMeshComponent = Cast<UMeshComponent>(InitialPhysicsControl.ChildActor->GetRootComponent());
			}
		}

		if (!ChildMeshComponent)
		{
			continue;
		}

		UMeshComponent* ParentMeshComponent = nullptr;
		if (InitialPhysicsControl.ParentActor.IsValid())
		{
			if (InitialPhysicsControl.ParentMeshComponentName != NAME_None)
			{
				for (UActorComponent* Component : InitialPhysicsControl.ParentActor->GetComponents())
				{
					if (Component->GetFName() == InitialPhysicsControl.ParentMeshComponentName)
					{
						ParentMeshComponent = Cast<UMeshComponent>(Component);
						break;
					}
				}
			}
			else
			{
				ParentMeshComponent = Cast<UMeshComponent>(InitialPhysicsControl.ParentActor->GetRootComponent());
			}
		}

		// If we get here then we will be removing the original and making a new one in its place.
		TArray<FName> Sets = ControlComponent->GetSetsContainingControl(Name);
		if (ControlComponent->GetControlExists(Name))
		{
			ControlComponent->DestroyControl(Name);
		}

		if (ControlComponent->CreateNamedControl(
			Name,
			ParentMeshComponent,
			InitialPhysicsControl.ParentBoneName,
			ChildMeshComponent,
			InitialPhysicsControl.ChildBoneName,
			InitialPhysicsControl.ControlData,
			InitialPhysicsControl.ControlTarget,
			"All"))
		{
			// If we were replacing a control then add it to the original sets
			FPhysicsControlNames Names;
			for (FName Set : Sets)
			{
				ControlComponent->AddControlToSet(Names, Name, Set);
			}
		}
	}
}

//======================================================================================================================
void UPhysicsControlInitializerComponent::CreateOrUpdateInitialBodyModifiers(UPhysicsControlComponent* ControlComponent)
{
	// Create any individual body modifiers
	for (const TPair<FName, FInitialBodyModifier>& InitialBodyModifierPair : InitialBodyModifiers)
	{
		const FName Name = InitialBodyModifierPair.Key;
		const FInitialBodyModifier& InitialBodyModifier = InitialBodyModifierPair.Value;

		UMeshComponent* MeshComponent = nullptr;
		if (InitialBodyModifier.Actor.IsValid())
		{
			if (InitialBodyModifier.MeshComponentName != NAME_None)
			{
				for (UActorComponent* Component : InitialBodyModifier.Actor->GetComponents())
				{
					if (Component->GetFName() == InitialBodyModifier.MeshComponentName)
					{
						MeshComponent = Cast<UMeshComponent>(Component);
						break;
					}
				}
			}
			else
			{
				MeshComponent = Cast<UMeshComponent>(InitialBodyModifier.Actor->GetRootComponent());
			}
		}

		if (!MeshComponent)
		{
			continue;
		}

		// If we get here then we will be removing the original and making a new one in its place.
		TArray<FName> Sets = ControlComponent->GetSetsContainingBodyModifier(Name);
		if (ControlComponent->GetBodyModifierExists(Name))
		{
			ControlComponent->DestroyBodyModifier(Name);
		}

		if (ControlComponent->CreateNamedBodyModifier(
			Name, MeshComponent, InitialBodyModifier.BoneName, "All", InitialBodyModifier.BodyModifierData))
		{
			// If we were replacing a control then add it to the original sets
			FPhysicsControlNames Names;
			for (FName Set : Sets)
			{
				ControlComponent->AddBodyModifierToSet(Names, Name, Set);
			}
		}
	}
}

//======================================================================================================================
void UPhysicsControlInitializerComponent::CreateControls(UPhysicsControlComponent* PhysicsControlComponent)
{
	if (!PhysicsControlComponent)
	{
		return;
	}
	PhysicsControlComponent->RegisterAllComponentTickFunctions(true);

	CreateInitialCharacterControls(PhysicsControlComponent);
	CreateOrUpdateInitialControls(PhysicsControlComponent);
	CreateOrUpdateInitialBodyModifiers(PhysicsControlComponent);
}

//======================================================================================================================
void UPhysicsControlInitializerComponent::BeginPlay()
{
	Super::BeginPlay();
	if (bCreateControlsAtBeginPlay)
	{
		UPhysicsControlComponent* ControlComponent = GetOwner()->GetComponentByClass<UPhysicsControlComponent>();
		if (ControlComponent)
		{
			CreateControls(ControlComponent);
		}
	}
}

//======================================================================================================================
UPhysicsControlInitializerComponent::UPhysicsControlInitializerComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//======================================================================================================================
APhysicsControlActor::APhysicsControlActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinderOptional<UTexture2D> KBSJointTexture;
		FName NAME_Physics;
		FConstructorStatics()
			: KBSJointTexture(TEXT("/Engine/EditorResources/S_KBSJoint"))
			, NAME_Physics(TEXT("Physics"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	ControlComponent = CreateDefaultSubobject<UPhysicsControlComponent>(TEXT("MyControlComponent"));
	RootComponent = ControlComponent;

	ControlInitializerComponent = CreateDefaultSubobject<UPhysicsControlInitializerComponent>(TEXT("MyControlInitializerComponent"));

	PrimaryActorTick.bCanEverTick = true;

	SetHidden(true);
}
