// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAsset.h"

#include "Engine/SkeletalMesh.h"
#include "PhysicsEngine/PhysicsAsset.h"

//======================================================================================================================

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsControlAsset)
UPhysicsControlAsset::UPhysicsControlAsset()
{
}

#if WITH_EDITOR
//======================================================================================================================
void UPhysicsControlAsset::ShowCompiledData() const
{
	UE_LOGF(LogTemp, Log, "Character setup data:");
	for (const FPhysicsControlLimbSetupData& LimbSetupData : CharacterSetupData.LimbSetupData)
	{
		UE_LOGF(LogTemp, Log, "Limb %ls", *LimbSetupData.LimbName.ToString());
		UE_LOGF(LogTemp, Log, "  Start bone %ls", *LimbSetupData.StartBone.ToString());
		UE_LOGF(LogTemp, Log, "  Include parent bone %d", LimbSetupData.bIncludeParentBone);
		UE_LOGF(LogTemp, Log, "  Create world space controls %d", LimbSetupData.bCreateWorldSpaceControls);
		UE_LOGF(LogTemp, Log, "  Create parent space controls %d", LimbSetupData.bCreateParentSpaceControls);
		UE_LOGF(LogTemp, Log, "  Create body modifiers %d", LimbSetupData.bCreateBodyModifiers);
	}
	UE_LOGF(LogTemp, Log, "Additional controls and modifiers:");
	UE_LOGF(LogTemp, Log, "  Additional controls:");
	for (TMap<FName, FPhysicsControlCreationData>::ElementType ControlPair : AdditionalControlsAndModifiers.Controls)
	{
		UE_LOGF(LogTemp, Log, "    %ls:", *ControlPair.Key.ToString());
		UE_LOGF(LogTemp, Log, "      Parent bone %ls:", *ControlPair.Value.Control.ParentBoneName.ToString());
		UE_LOGF(LogTemp, Log, "      Child bone %ls:", *ControlPair.Value.Control.ChildBoneName.ToString());
	}
	UE_LOGF(LogTemp, Log, "  Additional modifiers:");
	for (TMap<FName, FPhysicsBodyModifierCreationData>::ElementType ModifierPair : AdditionalControlsAndModifiers.Modifiers)
	{
		UE_LOGF(LogTemp, Log, "    %ls:", *ModifierPair.Key.ToString());
		UE_LOGF(LogTemp, Log, "      Bone %ls:", *ModifierPair.Value.Modifier.BoneName.ToString());
	}
	UE_LOGF(LogTemp, Log, "Profiles:");
	for (TMap<FName, FPhysicsControlControlAndModifierUpdates>::ElementType ProfilePair : Profiles)
	{
		UE_LOGF(LogTemp, Log, "  %ls:", *ProfilePair.Key.ToString());
	}

}

//======================================================================================================================
void UPhysicsControlAsset::Compile()
{
	CharacterSetupData = GetCharacterSetupData();
	AdditionalControlsAndModifiers = GetAdditionalControlsAndModifiers();
	AdditionalSets = GetAdditionalSets();
	InitialControlAndModifierUpdates = GetInitialControlAndModifierUpdates();

	TArray<FName> OrigKeys;
	Profiles.GetKeys(OrigKeys);

	Profiles = GetProfiles();

	TArray<FName> NewKeys;
	Profiles.GetKeys(NewKeys);

	bool bProfileListChanged = (OrigKeys != NewKeys);

	OnControlAssetCompiledDelegate.Broadcast(bProfileListChanged);

	Modify();
}

//======================================================================================================================
bool UPhysicsControlAsset::IsCompilationNeeded() const
{
	if (CharacterSetupData != GetCharacterSetupData())
	{
		return true;
	}
	if (AdditionalControlsAndModifiers != GetAdditionalControlsAndModifiers())
	{
		return true;
	}
	if (AdditionalSets != GetAdditionalSets())
	{
		return true;
	}
	if (InitialControlAndModifierUpdates != GetInitialControlAndModifierUpdates())
	{
		return true;
	}
	if (!Profiles.OrderIndependentCompareEqual(GetProfiles()))
	{
		return true;
	}
	return false;
}

//======================================================================================================================
bool UPhysicsControlAsset::IsSetupDirty() const
{
	if (CharacterSetupData != GetCharacterSetupData())
	{
		return true;
	}
	if (AdditionalControlsAndModifiers != GetAdditionalControlsAndModifiers())
	{
		return true;
	}
	if (AdditionalSets != GetAdditionalSets())
	{
		return true;
	}
	if (InitialControlAndModifierUpdates != GetInitialControlAndModifierUpdates())
	{
		return true;
	}
	return false;
}

//======================================================================================================================
TArray<FName> UPhysicsControlAsset::GetDirtyProfiles() const
{
	TArray<FName> DirtyProfiles;

	TMap<FName, FPhysicsControlControlAndModifierUpdates> CompiledProfiles = GetProfiles();

	for (TMap<FName, FPhysicsControlControlAndModifierUpdates>::ElementType ProfilePair : Profiles)
	{
		FPhysicsControlControlAndModifierUpdates* Compiled = CompiledProfiles.Find(ProfilePair.Key);
		if (Compiled && *Compiled == ProfilePair.Value)
		{
			continue;
		}
		DirtyProfiles.Push(ProfilePair.Key);
	}
	return DirtyProfiles;
}


//======================================================================================================================
FPhysicsControlCharacterSetupData UPhysicsControlAsset::GetCharacterSetupData() const
{
	FPhysicsControlCharacterSetupData CompiledCharacterSetupData;
	if (ParentAsset.LoadSynchronous())
	{
		CompiledCharacterSetupData = ParentAsset->GetCharacterSetupData();
	}
	CompiledCharacterSetupData += MyCharacterSetupData;
	return CompiledCharacterSetupData;
}

//======================================================================================================================
FPhysicsControlAndBodyModifierCreationDatas UPhysicsControlAsset::GetAdditionalControlsAndModifiers() const
{
	FPhysicsControlAndBodyModifierCreationDatas CompiledAdditionalControlsAndModifiers;
	if (ParentAsset.LoadSynchronous())
	{
		CompiledAdditionalControlsAndModifiers = ParentAsset->GetAdditionalControlsAndModifiers();
	}
	// this will overwrite duplicates with our value
	CompiledAdditionalControlsAndModifiers += MyAdditionalControlsAndModifiers; 
	return CompiledAdditionalControlsAndModifiers;
}

//======================================================================================================================
FPhysicsControlSetUpdates UPhysicsControlAsset::GetAdditionalSets() const
{
	FPhysicsControlSetUpdates CompiledAdditionalSets;
	if (ParentAsset.LoadSynchronous())
	{
		CompiledAdditionalSets = ParentAsset->GetAdditionalSets();
	}
	CompiledAdditionalSets += MyAdditionalSets;
	return CompiledAdditionalSets;
}

//======================================================================================================================
TArray<FPhysicsControlControlAndModifierUpdates> UPhysicsControlAsset::GetInitialControlAndModifierUpdates() const
{
	TArray<FPhysicsControlControlAndModifierUpdates> CompiledInitialControlAndModifierUpdates;
	if (ParentAsset.LoadSynchronous())
	{
		CompiledInitialControlAndModifierUpdates = ParentAsset->GetInitialControlAndModifierUpdates();
	}
	CompiledInitialControlAndModifierUpdates.Append(MyInitialControlAndModifierUpdates);
	return CompiledInitialControlAndModifierUpdates;
}

//======================================================================================================================
TMap<FName, FPhysicsControlControlAndModifierUpdates> UPhysicsControlAsset::GetProfiles() const
{
	TMap<FName, FPhysicsControlControlAndModifierUpdates> CompiledProfiles;
	if (ParentAsset.LoadSynchronous())
	{
		CompiledProfiles = ParentAsset->GetProfiles();
	}
	for (const TSoftObjectPtr<UPhysicsControlAsset>& AdditionalProfileAsset : AdditionalProfileAssets)
	{
		if (AdditionalProfileAsset.LoadSynchronous())
		{
			CompiledProfiles.Append(AdditionalProfileAsset->GetProfiles());
		}
	}
	CompiledProfiles.Append(MyProfiles); // this will overwrite duplicates with our value
	return CompiledProfiles;
}

#endif

#if WITH_EDITOR
//======================================================================================================================
UPhysicsAsset* UPhysicsControlAsset::GetPhysicsAsset() const
{
	return PhysicsAsset.LoadSynchronous();
}

//======================================================================================================================
void UPhysicsControlAsset::SetPhysicsAsset(UPhysicsAsset* InPhysicsAsset)
{
	PhysicsAsset = InPhysicsAsset;
}

//======================================================================================================================
const FName UPhysicsControlAsset::GetPreviewMeshPropertyName()
{
	return GET_MEMBER_NAME_STRING_CHECKED(UPhysicsAsset, PreviewSkeletalMesh);
};

#endif

//======================================================================================================================
void UPhysicsControlAsset::SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty)
{
#if WITH_EDITOR
	if (UPhysicsAsset* PA = GetPhysicsAsset())
	{
		PA->SetPreviewMesh(PreviewMesh, bMarkAsDirty);
	}
#endif
}

//======================================================================================================================
USkeletalMesh* UPhysicsControlAsset::GetPreviewMesh() const
{
#if WITH_EDITOR
	if (UPhysicsAsset* PA = GetPhysicsAsset())
	{
		return PA->GetPreviewMesh();
	}
#endif
	return nullptr;
}
