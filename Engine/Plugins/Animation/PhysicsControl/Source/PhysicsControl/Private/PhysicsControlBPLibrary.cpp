// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlBPLibrary.h"

#include "RigidBodyControlData.h"
#include "AnimNode_RigidBodyWithControl.h"
#include "PhysicsControlHelpers.h"
#include "PhysicsControlLog.h"

#include "Components/PrimitiveComponent.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "PhysicsEngine/BodyInstance.h"

#include "Chaos/ChaosEngineInterface.h"
#include "Chaos/Collision/CollisionConstraintFlags.h"
#include "Chaos/Collision/SpatialAccelerationBroadPhase.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

//======================================================================================================================

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsControlBPLibrary)
template<typename TParameterType> const TArray<FName>* FindNamesInSet(
	const FPhysicsControlNameRecords& NameRecords, const FName SetName)
{
	return nullptr;
}

//======================================================================================================================
template<> const TArray<FName>* FindNamesInSet<FPhysicsControl>(
	const FPhysicsControlNameRecords& NameRecords, const FName SetName)
{
	return NameRecords.ControlSets.Find(SetName);
}

//======================================================================================================================
template<> const TArray<FName>* FindNamesInSet<FPhysicsBodyModifier>(
	const FPhysicsControlNameRecords& NameRecords, const FName SetName)
{
	return NameRecords.BodyModifierSets.Find(SetName);
}

//======================================================================================================================
template<typename TParameterType> TArray<FName> GetNamesInSet(
	const FRigidBodyWithControlReference& RigidBodyWithControl, const FName SetName)
{
	TArray<FName> OutputNames;

	RigidBodyWithControl.CallAnimNodeFunction<FAnimNode_RigidBodyWithControl>(
		TEXT("GetControlNamesInSet"),
		[&OutputNames, SetName](FAnimNode_RigidBodyWithControl& InRigidBodyWithControl)
		{
			if (const TArray<FName>* FoundNames = FindNamesInSet<TParameterType>(
				InRigidBodyWithControl.GetNameRecords(), SetName))
			{
				OutputNames = *FoundNames;
			}
		});

	return OutputNames;
}

//======================================================================================================================
template<typename TNamedParameters> void InterpolateParametersContainers(
	TArray<TNamedParameters>& TargetContainer, const TArray<TNamedParameters>& SourceContainer, const float Weight)
{
	for (const TNamedParameters& Source : SourceContainer)
	{
		const FName SourceName = Source.Name;
		if (TNamedParameters* const ExistingUpdate = TargetContainer.FindByPredicate(
			[SourceName](const TNamedParameters& Element) { return Element.Name == SourceName; }))
		{
			// Interpolate values if the target container includes an element with the source values name.
			ExistingUpdate->Data = Interpolate(ExistingUpdate->Data, Source.Data, Weight);
		}
		else
		{
			// Add this element to the target container.
			TargetContainer.Add(Source);
		}
	}
}

//======================================================================================================================
template<typename TNamedParameters> void BlendParametersThroughSet(
	const FPhysicsControlControlAndModifierParameters& InParametersContainer, 
	const TNamedParameters&                            InStartParameters,
	const TNamedParameters&                            InEndParameters,
	const TArray<FName>&                               Names,
	FPhysicsControlControlAndModifierParameters&       OutParametersContainer)
{
	OutParametersContainer = InParametersContainer;

	const float WeightDelta = 1.0f / static_cast<float>(Names.Num() - 1);
	float Weight = 0.0f;

	for (const FName& Name : Names)
	{
		OutParametersContainer.Add(
			TNamedParameters(Name, Interpolate(InStartParameters.Data, InEndParameters.Data, Weight)));
		Weight += WeightDelta;
	}
}

//======================================================================================================================
void UPhysicsControlBPLibrary::AddControlParameters(
	UPARAM(ref) FPhysicsControlControlAndModifierParameters& InParametersContainer,
	FPhysicsControlControlAndModifierParameters&             OutParametersContainer,
	const FName                                              Name, 
	const FPhysicsControlSparseData&                         ControlData)
{
	OutParametersContainer = InParametersContainer;
	OutParametersContainer.Add(FPhysicsControlNamedControlParameters(Name, ControlData));
}

//======================================================================================================================
void UPhysicsControlBPLibrary::AddMultipleControlParameters(
	UPARAM(ref) FPhysicsControlControlAndModifierParameters& InParametersContainer,
	FPhysicsControlControlAndModifierParameters&             OutParametersContainer,
	const TArray<FName>&                                     Names,
	const FPhysicsControlSparseData&                         ControlData)
{
	OutParametersContainer = InParametersContainer;
	OutParametersContainer.ControlParameters.Reserve(OutParametersContainer.ControlParameters.Num() + Names.Num());

	for (const FName Name : Names)
	{
		OutParametersContainer.Add(FPhysicsControlNamedControlParameters(Name, ControlData));
	}	
}

//======================================================================================================================
void UPhysicsControlBPLibrary::AddModifierParameters(
	UPARAM(ref) FPhysicsControlControlAndModifierParameters& InParametersContainer,
	FPhysicsControlControlAndModifierParameters&             OutParametersContainer,
	const FName                                              Name,
	const FPhysicsControlModifierSparseData&                 ModifierData)
{
	OutParametersContainer = InParametersContainer;
	OutParametersContainer.Add(FPhysicsControlNamedModifierParameters(Name, ModifierData));
}

//======================================================================================================================
void UPhysicsControlBPLibrary::AddMultipleModifierParameters(
	UPARAM(ref) FPhysicsControlControlAndModifierParameters& InParametersContainer,
	FPhysicsControlControlAndModifierParameters&             OutParametersContainer,
	const TArray<FName>&                                     Names,
	const FPhysicsControlModifierSparseData&                 ModifierData)
{
	OutParametersContainer = InParametersContainer;
	OutParametersContainer.ModifierParameters.Reserve(OutParametersContainer.ModifierParameters.Num() + Names.Num());

	for (const FName Name : Names)
	{
		OutParametersContainer.Add(FPhysicsControlNamedModifierParameters(Name, ModifierData));
	}
}

//======================================================================================================================
void UPhysicsControlBPLibrary::BlendParameters(
	UPARAM(ref) FPhysicsControlControlAndModifierParameters& InParametersContainerA, 
	UPARAM(ref) FPhysicsControlControlAndModifierParameters& InParametersContainerB, 
	const float                                              InInterpolationWeight, 
	FPhysicsControlControlAndModifierParameters&             OutParametersContainers)
{
	OutParametersContainers = InParametersContainerA;

	InterpolateParametersContainers(
		OutParametersContainers.ControlParameters, InParametersContainerB.ControlParameters, InInterpolationWeight);
	InterpolateParametersContainers(
		OutParametersContainers.ModifierParameters, InParametersContainerB.ModifierParameters, InInterpolationWeight);
}

//======================================================================================================================
void UPhysicsControlBPLibrary::BlendControlParametersThroughSet(
	UPARAM(ref) FPhysicsControlControlAndModifierParameters& InParametersContainer,
	UPARAM(ref) const FPhysicsControlNamedControlParameters& InStartParameters,
	UPARAM(ref) const FPhysicsControlNamedControlParameters& InEndParameters,
	const TArray<FName>&                                     InNames,
	FPhysicsControlControlAndModifierParameters&             OutParametersContainer)
{
	BlendParametersThroughSet(InParametersContainer, InStartParameters, InEndParameters, InNames, OutParametersContainer);
}

//======================================================================================================================
void UPhysicsControlBPLibrary::BlendModifierParametersThroughSet(
	UPARAM(ref) FPhysicsControlControlAndModifierParameters&  InParametersContainer, 
	UPARAM(ref) const FPhysicsControlNamedModifierParameters& InStartParameters,
	UPARAM(ref) const FPhysicsControlNamedModifierParameters& InEndParameters,
	const TArray<FName>&                                      InNames,
	FPhysicsControlControlAndModifierParameters&              OutParametersContainer)
{
	BlendParametersThroughSet(InParametersContainer, InStartParameters, InEndParameters, InNames, OutParametersContainer);
}

//======================================================================================================================
FRigidBodyWithControlReference UPhysicsControlBPLibrary::ConvertToRigidBodyWithControl(
	const FAnimNodeReference&           Node,
	EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FRigidBodyWithControlReference>(Node, Result);
}

//======================================================================================================================
FRigidBodyWithControlReference UPhysicsControlBPLibrary::SetOverridePhysicsAsset(
	const FRigidBodyWithControlReference& Node, UPhysicsAsset* PhysicsAsset)
{
	Node.CallAnimNodeFunction<FAnimNode_RigidBodyWithControl>(
		TEXT("SetOverridePhysicsAsset"),
		[PhysicsAsset](FAnimNode_RigidBodyWithControl& Node)
		{
			Node.SetOverridePhysicsAsset(PhysicsAsset);
		});

	return Node;
}

//======================================================================================================================
UFUNCTION(BlueprintPure, Category = "Animation|PhysicsControl", meta = (BlueprintThreadSafe))
TArray<FName> UPhysicsControlBPLibrary::GetControlNamesInSet(
	const FRigidBodyWithControlReference& RigidBodyWithControl, 
	const FName                           SetName)
{
	return GetNamesInSet<FPhysicsControl>(RigidBodyWithControl, SetName);
}

//======================================================================================================================
UFUNCTION(BlueprintPure, Category = "Animation|PhysicsControl", meta = (BlueprintThreadSafe))
TArray<FName> UPhysicsControlBPLibrary::GetBodyModifierNamesInSet(
	const FRigidBodyWithControlReference& RigidBodyWithControl,
	const FName                           SetName)
{
	return GetNamesInSet<FPhysicsBodyModifier>(RigidBodyWithControl, SetName);
}

//======================================================================================================================
// True when the actor handle wraps a rigid (dynamic or kinematic) Chaos particle. Static
// particles are not supported by FIgnoreCollisionManager and pair-ignore is a silent no-op for
// them.
static bool IsRigidParticle(const FPhysicsActorHandle Handle)
{
	return Handle->GetParticle_LowLevel() != nullptr
		&& Handle->GetParticle_LowLevel()->CastToRigidParticle() != nullptr;
}
//======================================================================================================================
// Resolves both (component, bone) pairs to FPhysicsActorHandles, validating that each is
// non-null, dynamic-or-kinematic, and that both live in the same solver. Logs a warning and
// returns false on any failure.
static bool ResolveCollisionPair(
	UPrimitiveComponent* FirstComponent,  const FName FirstBoneName,
	UPrimitiveComponent* SecondComponent, const FName SecondBoneName,
	FPhysicsActorHandle& OutFirstHandle,  FPhysicsActorHandle& OutSecondHandle)
{
	FBodyInstance* const FirstBody  = UE::PhysicsControl::GetBodyInstance(FirstComponent,  FirstBoneName);
	FBodyInstance* const SecondBody = UE::PhysicsControl::GetBodyInstance(SecondComponent, SecondBoneName);
	if (!FirstBody || !SecondBody)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"CollisionBetweenBodies: could not resolve a body instance for one or both of (%ls / %ls)",
			*FirstBoneName.ToString(), *SecondBoneName.ToString());
		return false;
	}

	OutFirstHandle  = FirstBody->GetPhysicsActor();
	OutSecondHandle = SecondBody->GetPhysicsActor();
	if (!OutFirstHandle || !OutSecondHandle)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"CollisionBetweenBodies: body instance(s) not initialised in the physics scene (%ls / %ls)",
			*FirstBoneName.ToString(), *SecondBoneName.ToString());
		return false;
	}

	Chaos::FPhysicsSolver* const SolverA = OutFirstHandle->GetSolver<Chaos::FPhysicsSolver>();
	Chaos::FPhysicsSolver* const SolverB = OutSecondHandle->GetSolver<Chaos::FPhysicsSolver>();
	if (!SolverA || SolverA != SolverB)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"CollisionBetweenBodies: bodies live in invalid/different physics solvers (%ls / %ls)",
			*FirstBoneName.ToString(), *SecondBoneName.ToString());
		return false;
	}

	// FIgnoreCollisionManager silently drops non-rigid (static) particles. Surface as a warning
	// rather than a silent no-op so callers can tell why it didn't take effect.
	if (!IsRigidParticle(OutFirstHandle) || !IsRigidParticle(OutSecondHandle))
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"CollisionBetweenBodies: one or both bodies are static; pair-ignore not supported (%ls / %ls)",
			*FirstBoneName.ToString(), *SecondBoneName.ToString());
		return false;
	}

	return true;
}

//======================================================================================================================
bool UPhysicsControlBPLibrary::DisableCollisionBetweenBodies(
	UPrimitiveComponent* FirstComponent,  const FName FirstBoneName,
	UPrimitiveComponent* SecondComponent, const FName SecondBoneName)
{
	FPhysicsActorHandle HandleA, HandleB;
	if (!ResolveCollisionPair(FirstComponent, FirstBoneName, SecondComponent, SecondBoneName, HandleA, HandleB))
	{
		return false;
	}

	// Enqueue a physics-thread command that calls FIgnoreCollisionManager::AddIgnoreCollisions
	// directly, mirroring the Enable path. We don't use FChaosEngineInterface::
	// AddDisabledCollisionsFor_AssumesLocked because its pending-queue path destructively replaces
	// any prior pending entry for the same key actor in the same frame, which silently wipes the
	// asset-driven disables queued by USkeletalMeshComponent::InitCollisionRelationships.
	// AddIgnoreCollisions is symmetric, so a single A->B call covers both directions.
	Chaos::FPhysicsSolver* const Solver = HandleA->GetSolver<Chaos::FPhysicsSolver>();
	// ResolveCollisionPair will have confirmed we have a valid solver here
	Solver->EnqueueCommandImmediate([HandleA, HandleB]()
	{
		Chaos::FGeometryParticleHandle* const HA = HandleA->GetHandle_LowLevel();
		Chaos::FGeometryParticleHandle* const HB = HandleB->GetHandle_LowLevel();
		if (!HA || !HB)
		{
			return;
		}
		if (Chaos::FPhysicsSolver* const PhysicsThreadSolver = HandleA->GetSolver<Chaos::FPhysicsSolver>())
		{
			PhysicsThreadSolver->GetEvolution()->GetBroadPhase().GetIgnoreCollisionManager().
				AddIgnoreCollisions(HA, HB);
		}
	});
	return true;
}

//======================================================================================================================
bool UPhysicsControlBPLibrary::EnableCollisionBetweenBodies(
	UPrimitiveComponent* FirstComponent,  const FName FirstBoneName,
	UPrimitiveComponent* SecondComponent, const FName SecondBoneName)
{
	FPhysicsActorHandle HandleA, HandleB;
	if (!ResolveCollisionPair(FirstComponent, FirstBoneName, SecondComponent, SecondBoneName, HandleA, HandleB))
	{
		return false;
	}

	// There is no precise game-thread remove API for FIgnoreCollisionManager - the engine wrapper
	// RemoveDisabledCollisionsFor_AssumesLocked is coarse (clears ALL ignores for an actor). For
	// per-pair precision we enqueue a physics-thread command that calls the manager directly.
	Chaos::FPhysicsSolver* const Solver = HandleA->GetSolver<Chaos::FPhysicsSolver>();
	// ResolveCollisionPair will have confirmed we have a valid solver here
	Solver->EnqueueCommandImmediate([HandleA, HandleB]()
	{
		Chaos::FGeometryParticleHandle* const HA = HandleA->GetHandle_LowLevel();
		Chaos::FGeometryParticleHandle* const HB = HandleB->GetHandle_LowLevel();
		if (!HA || !HB)
		{
			return;
		}
		if (Chaos::FPhysicsSolver* const PhysicsThreadSolver = HandleA->GetSolver<Chaos::FPhysicsSolver>())
		{
			PhysicsThreadSolver->GetEvolution()->GetBroadPhase().GetIgnoreCollisionManager().
				RemoveIgnoreCollisions(HA, HB);
		}
	});
	return true;
}

//======================================================================================================================
bool UPhysicsControlBPLibrary::DisableCollisionBetweenBodyArrays(
	UPrimitiveComponent* FirstComponent,  const TArray<FName>& FirstBoneNames,
	UPrimitiveComponent* SecondComponent, const TArray<FName>& SecondBoneNames)
{
	// An empty bone array (or a single entry with name None) means "use the component's single
	// body" - e.g. for a static-mesh component, which has no per-bone bodies. Substitute a single
	// NAME_None so the loop can still work.
	const FName UnnamedFallback = NAME_None;
	const TArrayView<const FName> FirstNames  =
		FirstBoneNames.IsEmpty()  ? MakeArrayView(&UnnamedFallback, 1) : MakeArrayView(FirstBoneNames);
	const TArrayView<const FName> SecondNames =
		SecondBoneNames.IsEmpty() ? MakeArrayView(&UnnamedFallback, 1) : MakeArrayView(SecondBoneNames);

	bool bAllOk = true;
	TArray<TPair<FPhysicsActorHandle, FPhysicsActorHandle>> Pairs;
	Pairs.Reserve(FirstNames.Num() * SecondNames.Num());
	for (const FName& FirstBoneName : FirstNames)
	{
		for (const FName& SecondBoneName : SecondNames)
		{
			FPhysicsActorHandle HandleA, HandleB;
			if (!ResolveCollisionPair(FirstComponent, FirstBoneName, SecondComponent, SecondBoneName, HandleA, HandleB))
			{
				bAllOk = false;
				continue;
			}
			// Same body resolves on both sides - ignoring a body against itself is meaningless.
			if (HandleA == HandleB)
			{
				continue;
			}
			Pairs.Emplace(HandleA, HandleB);
		}
	}
	if (Pairs.IsEmpty())
	{
		return bAllOk;
	}

	// One EnqueueCommandImmediate for the whole batch instead of multiple physics-thread hops.
	// AddIgnoreCollisions is symmetric, so each pair only needs a single A->B call. See
	// DisableCollisionBetweenBodies for why we bypass FChaosEngineInterface:: AddDisabledCollisionsFor_AssumesLocked.
	Chaos::FPhysicsSolver* const Solver = Pairs[0].Key->GetSolver<Chaos::FPhysicsSolver>();
	// ResolveCollisionPair confirmed a valid, matching solver for every pair we kept.
	Solver->EnqueueCommandImmediate([Pairs = MoveTemp(Pairs)]()
	{
		for (const TPair<FPhysicsActorHandle, FPhysicsActorHandle>& Pair : Pairs)
		{
			Chaos::FGeometryParticleHandle* const HA = Pair.Key->GetHandle_LowLevel();
			Chaos::FGeometryParticleHandle* const HB = Pair.Value->GetHandle_LowLevel();
			if (!HA || !HB)
			{
				continue;
			}
			if (Chaos::FPhysicsSolver* const PhysicsThreadSolver = Pair.Key->GetSolver<Chaos::FPhysicsSolver>())
			{
				PhysicsThreadSolver->GetEvolution()->GetBroadPhase().GetIgnoreCollisionManager().
					AddIgnoreCollisions(HA, HB);
			}
		}
	});
	return bAllOk;
}

//======================================================================================================================
bool UPhysicsControlBPLibrary::EnableCollisionBetweenBodyArrays(
	UPrimitiveComponent* FirstComponent,  const TArray<FName>& FirstBoneNames,
	UPrimitiveComponent* SecondComponent, const TArray<FName>& SecondBoneNames)
{
	// An empty bone array (or a single entry with name None) means "use the component's single
	// body" - e.g. for a static-mesh component, which has no per-bone bodies. Substitute a single
	// NAME_None if empty so the loop can still work.
	const FName UnnamedFallback = NAME_None;
	const TArrayView<const FName> FirstNames  = 
		FirstBoneNames.IsEmpty()  ? MakeArrayView(&UnnamedFallback, 1) : MakeArrayView(FirstBoneNames);
	const TArrayView<const FName> SecondNames = 
		SecondBoneNames.IsEmpty() ? MakeArrayView(&UnnamedFallback, 1) : MakeArrayView(SecondBoneNames);

	bool bAllOk = true;
	TArray<TPair<FPhysicsActorHandle, FPhysicsActorHandle>> Pairs;
	Pairs.Reserve(FirstNames.Num() * SecondNames.Num());
	for (const FName& FirstBoneName : FirstNames)
	{
		for (const FName& SecondBoneName : SecondNames)
		{
			FPhysicsActorHandle HandleA, HandleB;
			if (!ResolveCollisionPair(FirstComponent, FirstBoneName, SecondComponent, SecondBoneName, HandleA, HandleB))
			{
				bAllOk = false;
				continue;
			}
			if (HandleA == HandleB)
			{
				continue;
			}
			Pairs.Emplace(HandleA, HandleB);
		}
	}
	if (Pairs.IsEmpty())
	{
		return bAllOk;
	}

	// One EnqueueCommandImmediate for the whole batch instead of multiple physics-thread hops.
	Chaos::FPhysicsSolver* const Solver = Pairs[0].Key->GetSolver<Chaos::FPhysicsSolver>();
	// ResolveCollisionPair confirmed a valid, matching solver for every pair we kept.
	Solver->EnqueueCommandImmediate([Pairs = MoveTemp(Pairs)]()
	{
		for (const TPair<FPhysicsActorHandle, FPhysicsActorHandle>& Pair : Pairs)
		{
			Chaos::FGeometryParticleHandle* const HA = Pair.Key->GetHandle_LowLevel();
			Chaos::FGeometryParticleHandle* const HB = Pair.Value->GetHandle_LowLevel();
			if (!HA || !HB)
			{
				continue;
			}
			if (Chaos::FPhysicsSolver* const PhysicsThreadSolver = Pair.Key->GetSolver<Chaos::FPhysicsSolver>())
			{
				PhysicsThreadSolver->GetEvolution()->GetBroadPhase().GetIgnoreCollisionManager().
					RemoveIgnoreCollisions(HA, HB);
			}
		}
	});
	return bAllOk;
}

