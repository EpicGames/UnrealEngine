// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlComponent.h"
#include "PhysicsControlLog.h"
#include "PhysicsControlRecord.h"
#include "PhysicsControlHelpers.h"
#include "PhysicsControlOperatorNameGeneration.h"

#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"

#include "Physics/PhysicsInterfaceCore.h"

#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/BillboardComponent.h"

#include "Engine/Engine.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/Texture2D.h"

#include "PrimitiveDrawingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsControlComponent)

DECLARE_CYCLE_STAT(TEXT("PhysicsControl UpdateTargetCaches"), STAT_PhysicsControl_UpdateTargetCaches, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("PhysicsControl UpdateControls"), STAT_PhysicsControl_UpdateControls, STATGROUP_Anim);

//======================================================================================================================
// This file contains the public member functions of UPhysicsControlComponent
//======================================================================================================================

//======================================================================================================================
// This is used, rather than UEnum::GetValueAsString, so that we have more control over the string returned, which 
// gets used as a prefix for the automatically named sets etc
static FName GetControlTypeName(EPhysicsControlType ControlType)
{
	switch (ControlType)
	{
	case EPhysicsControlType::ParentSpace:
		return "ParentSpace";
	case EPhysicsControlType::WorldSpace:
		return "WorldSpace";
	default:
		return "None";
	}
}

//======================================================================================================================
// Helper for disabling warnings about control etc names not existing. Used when looking for
// controls or modifiers that could be referenced by name or set.
struct FDisableNameWarnings
{
	FDisableNameWarnings(bool& bOriginal)
		: bSaved(bOriginal), bWarnAboutInvalidNames(bOriginal)
	{
		bOriginal = false;
	}
	~FDisableNameWarnings()
	{
		bWarnAboutInvalidNames = bSaved;
	}
	bool bSaved;
	bool& bWarnAboutInvalidNames;
};

//======================================================================================================================
UPhysicsControlComponent::UPhysicsControlComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// ActorComponent setup
	bWantsInitializeComponent = true;
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bTickEvenWhenPaused = true;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;

#if WITH_EDITORONLY_DATA
	bVisualizeComponent = true;
#endif
}

//======================================================================================================================
void UPhysicsControlComponent::InitializeComponent()
{
	Super::InitializeComponent();
	ResetControls(false);
}

//======================================================================================================================
void UPhysicsControlComponent::BeginDestroy()
{
	DestroyPhysicsState();
	Super::BeginDestroy();
}

//======================================================================================================================
void UPhysicsControlComponent::UpdateTargetCaches(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_PhysicsControl_UpdateTargetCaches);
	TRACE_CPUPROFILER_EVENT_SCOPE(UPhysicsControlComponent::UpdateTargetCaches);

	// Update the skeletal mesh caches
	UpdateCachedSkeletalBoneData(DeltaTime);
}

//======================================================================================================================
void UPhysicsControlComponent::UpdateControls(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_PhysicsControl_UpdateControls);
	TRACE_CPUPROFILER_EVENT_SCOPE(UPhysicsControlComponent::UpdateControls);

	ControlRecords.Compact();
	BodyModifierRecords.Compact();

	for (TPair<FName, FPhysicsControlRecord>& RecordPair : ControlRecords)
	{
		// New constraint requested when one doesn't exist
		FName ControlName = RecordPair.Key;
		FPhysicsControlRecord& Record = RecordPair.Value;
		if (!Record.ConstraintInstance)
		{
			Record.InitConstraint(this, ControlName, bWarnAboutInvalidNames);
		}
		else if (bAttemptToRecreateDisabledControls && !Record.ConstraintInstance->GetPhysicsScene())
		{
			// If bodies have been removed from the simulation, then the constraint is removed too,
			// so attempt to quietly reinitialize the constraint.
			Record.InitConstraint(this, ControlName, false);
		}

		// Refresh and use the cached joint constraint index for fast live-instance and
		// default-instance lookups. RefreshJointConstraintIndex re-resolves on asset change.
		FConstraintInstance*       PhysicsAssetConstraintInstance = nullptr;
		const FConstraintInstance* DefaultConstraintInstance      = nullptr;
		if (Record.PhysicsControl.ControlData.bUseSkeletalAnimation)
		{
			if (USkeletalMeshComponent* ChildSKM =
				Cast<USkeletalMeshComponent>(Record.ChildComponent.Get()))
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_PhysicsControl_RefreshJointConstraint);
				UPhysicsAsset* PhysAsset = ChildSKM->GetPhysicsAsset();
				if (PhysAsset && Record.RefreshJointConstraintIndex(ChildSKM, PhysAsset))
				{
					PhysicsAssetConstraintInstance = ChildSKM->GetConstraintInstanceByIndex(
						Record.JointConstraintIndex);
					if (PhysicsAssetConstraintInstance)
					{
						DefaultConstraintInstance = &PhysAsset->ConstraintSetup[
							Record.JointConstraintIndex]->DefaultInstance;
					}
				}
			}
		}

		ApplyControl(Record, PhysicsAssetConstraintInstance, DefaultConstraintInstance);
	}

	// Handle body modifiers
	for (TPair<FName, FPhysicsBodyModifierRecord>& BodyModifierPair : BodyModifierRecords)
	{
		FPhysicsBodyModifierRecord& BodyModifier = BodyModifierPair.Value;
		ApplyBodyModifier(BodyModifier);
	}
}

//======================================================================================================================
void UPhysicsControlComponent::TickComponent(
	float                        DeltaTime,
	enum ELevelTick              TickType,
	FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	// We only want to continue the update if this is a "real" tick that corresponds to updating the
	// world. We certainly don't want to tick during a pause, because part of the processing involves 
	// (optionally) calculating target velocities based on target positions in previous ticks etc.
	if (TickType != LEVELTICK_All)
	{
		return;
	}

	UpdateTargetCaches(DeltaTime);

	UpdateControls(DeltaTime);
}

//======================================================================================================================
TMap<FName, FPhysicsControlLimbBones> UPhysicsControlComponent::GetLimbBonesFromSkeletalMesh(
	USkeletalMeshComponent*                     SkeletalMeshComponent,
	const TArray<FPhysicsControlLimbSetupData>& LimbSetupDatas) const
{
	TMap<FName, FPhysicsControlLimbBones> Result;

	UPhysicsAsset* PhysicsAsset = SkeletalMeshComponent ? SkeletalMeshComponent->GetPhysicsAsset() : nullptr;
	const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent ? SkeletalMeshComponent->GetSkeletalMeshAsset() : nullptr;
	if (!PhysicsAsset || !SkeletalMesh)
	{
		UE_LOGF(LogPhysicsControl, Warning, "No physics asset or skeletal mesh in %ls",
			GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return Result;
	}

	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	Result = UE::PhysicsControl::GetLimbBones(LimbSetupDatas, RefSkeleton, PhysicsAsset);
	for (TPair<FName, FPhysicsControlLimbBones>& Pair : Result)
	{
		Pair.Value.SkeletalMeshComponent = SkeletalMeshComponent;
	}

	return Result;
}

//======================================================================================================================
// Note - some params passed by value to allow them to be unconnected and enable inline BP editing
FName UPhysicsControlComponent::CreateControl(
	UPrimitiveComponent*          ParentComponent,
	const FName                   ParentBoneName,
	UPrimitiveComponent*          ChildComponent,
	const FName                   ChildBoneName,
	const FPhysicsControlData     ControlData, 
	const FPhysicsControlTarget   ControlTarget, 
	const FName                   Set,
	const FString                 NamePrefix)
{
	const FName Name = UE::PhysicsControl::GetUniqueControlName(
		ParentComponent, ParentBoneName, ChildComponent, ChildBoneName, ControlRecords, NamePrefix);
	if (CreateNamedControl(
		Name, ParentComponent, ParentBoneName, ChildComponent, ChildBoneName, ControlData, ControlTarget, Set))
	{
		return Name;
	}
	return FName();
}

//======================================================================================================================
// Note - some params passed by value to allow them to be unconnected and enable inline BP editing
bool UPhysicsControlComponent::CreateNamedControl(
	const FName                   Name, 
	UPrimitiveComponent*          ParentComponent,
	const FName                   ParentBoneName,
	UPrimitiveComponent*          ChildComponent,
	const FName                   ChildBoneName,
	const FPhysicsControlData     ControlData, 
	const FPhysicsControlTarget   ControlTarget, 
	const FName                   Set)
{
	if (FindControlRecord(Name))
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"Unable to make a Control as one with the desired name %ls already exists in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return false;
	}

	if (!ChildComponent)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"Unable to make a Control as the child mesh component has not been set in %ls", 
			GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return false;
	}

	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ParentComponent))
	{
		AddSkeletalMeshReferenceForCaching(SkeletalMeshComponent);
	}
	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ChildComponent))
	{
		AddSkeletalMeshReferenceForCaching(SkeletalMeshComponent);
	}

	FPhysicsControlRecord& NewRecord = ControlRecords.Add(
		Name, FPhysicsControlRecord(
			FPhysicsControl(ParentBoneName, ChildBoneName, ControlData), 
			ControlTarget, ParentComponent, ChildComponent));
	NewRecord.ResetControlPoint();

	NameRecords.AddControl(Name, Set);

	return true;
}

//======================================================================================================================
// Note - some params passed by value to allow them to be unconnected and enable inline BP editing
TArray<FName> UPhysicsControlComponent::CreateControlsFromSkeletalMeshBelow(
	USkeletalMeshComponent* SkeletalMeshComponent,
	const FName                   BoneName,
	const bool                    bIncludeSelf,
	const EPhysicsControlType     ControlType,
	const FPhysicsControlData     ControlData,
	const FName                   Set)
{
	TArray<FName> Result;
	UPhysicsAsset* PhysicsAsset = SkeletalMeshComponent ? SkeletalMeshComponent->GetPhysicsAsset() : nullptr;
	if (!PhysicsAsset)
	{
		UE_LOGF(LogPhysicsControl, Warning, "No physics asset in skeletal mesh in %ls",
			GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return Result;
	}

	UPrimitiveComponent* ParentComponent = 
		(ControlType == EPhysicsControlType::ParentSpace) ? SkeletalMeshComponent : nullptr;

	SkeletalMeshComponent->ForEachBodyBelow(
		BoneName, bIncludeSelf, /*bSkipCustomType=*/false, 
		[
			this, PhysicsAsset, ParentComponent, SkeletalMeshComponent, 
			ControlType, &ControlData, Set, &Result
		](const FBodyInstance* BI)
		{
			if (USkeletalBodySetup* BodySetup = Cast<USkeletalBodySetup>(BI->BodySetup.Get()))
			{
				const FName ChildBoneName = PhysicsAsset->SkeletalBodySetups[BI->InstanceBodyIndex]->BoneName;

				FName ParentBoneName;
				if (ParentComponent)
				{
					ParentBoneName = UE::PhysicsControl::GetPhysicalParentBone(
						SkeletalMeshComponent, ChildBoneName);
					if (ParentBoneName.IsNone())
					{
						return;
					}
				}
				const FName ControlName = CreateControl(
					ParentComponent, ParentBoneName, SkeletalMeshComponent, ChildBoneName,
					ControlData, FPhysicsControlTarget(), 
					FName(GetControlTypeName(ControlType).ToString().Append("_").Append(Set.ToString())));
				if (!ControlName.IsNone())
				{
					Result.Add(ControlName);
					NameRecords.AddControl(ControlName, GetControlTypeName(ControlType));
				}
				else
				{
					UE_LOGF(LogPhysicsControl, Warning, 
						"Failed to make control for %ls in %ls",
						*ChildBoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
				}
			}
		});

	return Result;
}

//======================================================================================================================
// Note - some params passed by value to allow them to be unconnected and enable inline BP editing
TArray<FName> UPhysicsControlComponent::CreateControlsFromSkeletalMeshAndConstraintProfileBelow(
	USkeletalMeshComponent* SkeletalMeshComponent,
	const FName             BoneName,
	const bool              bIncludeSelf,
	const FName             ConstraintProfile,
	const FName             Set,
	const bool              bEnabled)
{
	TArray<FName> Result;
	UPhysicsAsset* PhysicsAsset = SkeletalMeshComponent ? SkeletalMeshComponent->GetPhysicsAsset() : nullptr;
	if (!PhysicsAsset)
	{
		UE_LOGF(LogPhysicsControl, Warning, "No physics asset in skeletal mesh in %ls", 
			GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return Result;
	}

	SkeletalMeshComponent->ForEachBodyBelow(
		BoneName, bIncludeSelf, /*bSkipCustomType=*/false,
		[
			this, PhysicsAsset, SkeletalMeshComponent,
			ConstraintProfile, Set, &Result, bEnabled
		](const FBodyInstance* BI)
		{
			if (USkeletalBodySetup* BodySetup = Cast<USkeletalBodySetup>(BI->BodySetup.Get()))
			{
				const FName ChildBoneName = PhysicsAsset->SkeletalBodySetups[BI->InstanceBodyIndex]->BoneName;

				FName ParentBoneName;
				ParentBoneName = UE::PhysicsControl::GetPhysicalParentBone(
					SkeletalMeshComponent, ChildBoneName);
				if (ParentBoneName.IsNone())
				{
					return;
				}

				FPhysicsControlData ControlData;
				// This is to match the skeletal mesh component velocity drive, which does not use the
				// target animation velocity.
				ControlData.AngularTargetVelocityMultiplier = 0;
				ControlData.LinearTargetVelocityMultiplier = 0;
				FConstraintProfileProperties ProfileProperties;
				if (!SkeletalMeshComponent->GetConstraintProfilePropertiesOrDefault(
					ProfileProperties, ChildBoneName, ConstraintProfile))
				{
					UE_LOGF(LogPhysicsControl, Warning, 
						"Failed get constraint profile for %ls in %ls",
						*ChildBoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
					return;
				}

				UE::PhysicsControl::ConvertConstraintProfileToControlData(ControlData, ProfileProperties);
				ControlData.bEnabled = bEnabled;

				const FName ControlName = CreateControl(
					SkeletalMeshComponent, ParentBoneName, SkeletalMeshComponent, ChildBoneName,
					ControlData, FPhysicsControlTarget(), 
					FName(GetControlTypeName(EPhysicsControlType::ParentSpace).ToString().Append("_").Append(Set.ToString())));
				if (!ControlName.IsNone())
				{
					Result.Add(ControlName);
					NameRecords.AddControl(
						ControlName, GetControlTypeName(EPhysicsControlType::ParentSpace));
				}
				else
				{
					UE_LOGF(LogPhysicsControl, Warning,
						"Failed to make control for %ls in %ls", 
						*ChildBoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
				}
			}
		});

	return Result;
}

//======================================================================================================================
// Note - some params passed by value to allow them to be unconnected and enable inline BP editing
TArray<FName> UPhysicsControlComponent::CreateControlsFromSkeletalMesh(
	USkeletalMeshComponent*       SkeletalMeshComponent,
	const TArray<FName>&          BoneNames,
	const EPhysicsControlType     ControlType,
	const FPhysicsControlData     ControlData,
	const FName                   Set)
{
	TArray<FName> Result;
	UPhysicsAsset* PhysicsAsset = SkeletalMeshComponent ? SkeletalMeshComponent->GetPhysicsAsset() : nullptr;
	if (!PhysicsAsset)
	{
		UE_LOGF(LogPhysicsControl, Warning, "No physics asset in skeletal mesh in %ls",
			GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return Result;
	}

	UPrimitiveComponent* ParentComponent =
		(ControlType == EPhysicsControlType::ParentSpace) ? SkeletalMeshComponent : nullptr;

	for (FName ChildBoneName : BoneNames)
	{
		FName ParentBoneName;
		if (ParentComponent)
		{
			ParentBoneName = UE::PhysicsControl::GetPhysicalParentBone(
				SkeletalMeshComponent, ChildBoneName);
			if (ParentBoneName.IsNone())
			{
				continue;
			}
		}
		const FName ControlName = CreateControl(
			ParentComponent, ParentBoneName, SkeletalMeshComponent, ChildBoneName,
			ControlData, FPhysicsControlTarget(), 
			FName(GetControlTypeName(ControlType).ToString().Append("_").Append(Set.ToString())));
		if (!ControlName.IsNone())
		{
			Result.Add(ControlName);
			NameRecords.AddControl(ControlName, GetControlTypeName(ControlType));
		}
		else
		{
			UE_LOGF(LogPhysicsControl, Warning, "Failed to make control for %ls in %ls",
				*ChildBoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		}
	}

	return Result;
}

//======================================================================================================================
// Note - some params passed by value to allow them to be unconnected and enable inline BP editing
TArray<FName> UPhysicsControlComponent::CreateControlsFromSkeletalMeshAndConstraintProfile(
	USkeletalMeshComponent* SkeletalMeshComponent,
	const TArray<FName>&    BoneNames,
	const FName             ConstraintProfile,
	const FName             Set,
	const bool              bEnabled)
{
	TArray<FName> Result;
	UPhysicsAsset* PhysicsAsset = SkeletalMeshComponent ? SkeletalMeshComponent->GetPhysicsAsset() : nullptr;
	if (!PhysicsAsset)
	{
		UE_LOGF(LogPhysicsControl, Warning, "No physics asset in skeletal mesh in %ls", 
			GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return Result;
	}

	for (FName ChildBoneName : BoneNames)
	{
		const FName ParentBoneName = 
			UE::PhysicsControl::GetPhysicalParentBone(SkeletalMeshComponent, ChildBoneName);
		if (ParentBoneName.IsNone())
		{
			continue;
		}

		FPhysicsControlData ControlData;
		// This is to match the skeletal mesh component velocity drive, which does not use the
		// target animation velocity.
		ControlData.AngularTargetVelocityMultiplier = 0;
		ControlData.LinearTargetVelocityMultiplier = 0;
		FConstraintProfileProperties ProfileProperties;
		if (!SkeletalMeshComponent->GetConstraintProfilePropertiesOrDefault(
			ProfileProperties, ChildBoneName, ConstraintProfile))
		{
			UE_LOGF(LogPhysicsControl, Warning, 
				"Failed get constraint profile for %ls in %ls", 
				*ChildBoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
			continue;
		}

		UE::PhysicsControl::ConvertConstraintProfileToControlData(ControlData, ProfileProperties);
		ControlData.bEnabled = bEnabled;

		const FName ControlName = CreateControl(
			SkeletalMeshComponent, ParentBoneName, SkeletalMeshComponent, ChildBoneName,
			ControlData, FPhysicsControlTarget(), 
			FName(GetControlTypeName(EPhysicsControlType::ParentSpace).ToString().Append("_").Append(Set.ToString())));
		if (!ControlName.IsNone())
		{
			Result.Add(ControlName);
			NameRecords.AddControl(ControlName, GetControlTypeName(EPhysicsControlType::ParentSpace));
		}
		else
		{
			UE_LOGF(LogPhysicsControl, Warning, "Failed to make control for %ls in %ls",
				*ChildBoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		}
	}

	return Result;
}

//======================================================================================================================
// Note - some params passed by value to allow them to be unconnected and enable inline BP editing
TMap<FName, FPhysicsControlNames> UPhysicsControlComponent::CreateControlsFromLimbBones(
	FPhysicsControlNames&                        AllControls,
	const TMap<FName, FPhysicsControlLimbBones>& LimbBones,
	const EPhysicsControlType                    ControlType,
	const FPhysicsControlData                    ControlData,
	UPrimitiveComponent*                         WorldComponent,
	FName                                        WorldBoneName,
	FString                                      NamePrefix)
{
	TMap<FName, FPhysicsControlNames> Result;
	Result.Reserve(LimbBones.Num());

	for (const TPair<FName, FPhysicsControlLimbBones>& LimbBoneEntry : LimbBones)
	{
		const FName LimbName = LimbBoneEntry.Key;
		const FPhysicsControlLimbBones& BonesInLimb = LimbBoneEntry.Value;

		if (!BonesInLimb.SkeletalMeshComponent.IsValid())
		{
			UE_LOGF(LogPhysicsControl, Warning, "No Skeletal mesh in limb %ls", *LimbName.ToString());
			continue;
		}

		if ((ControlType == EPhysicsControlType::WorldSpace && !BonesInLimb.bCreateWorldSpaceControls) ||
			(ControlType == EPhysicsControlType::ParentSpace && !BonesInLimb.bCreateParentSpaceControls))
		{
			continue;
		}

		USkeletalMeshComponent* ParentSkeletalMeshComponent =
			(ControlType == EPhysicsControlType::ParentSpace) ? BonesInLimb.SkeletalMeshComponent.Get() : nullptr;

		const int32 NumBonesInLimb = BonesInLimb.BoneNames.Num();

		FPhysicsControlNames& LimbResult = Result.Add(LimbName);
		LimbResult.Names.Reserve(NumBonesInLimb);
		AllControls.Names.Reserve(AllControls.Names.Num() + NumBonesInLimb);

		FString SetName = 
			NamePrefix + GetControlTypeName(ControlType).ToString().Append("_").Append(LimbName.ToString());

		for (int32 BoneIndex = 0 ; BoneIndex != NumBonesInLimb ; ++BoneIndex)
		{
			// Don't create the parent space control if it's the first bone in a limb that had bIncludeParentBone
			if (BoneIndex == 0 && BonesInLimb.bFirstBoneIsAdditional && ControlType == EPhysicsControlType::ParentSpace)
			{
				continue;
			}

			FName ChildBoneName = BonesInLimb.BoneNames[BoneIndex];

			FName ParentBoneName;
			if (ParentSkeletalMeshComponent)
			{
				ParentBoneName = UE::PhysicsControl::GetPhysicalParentBone(
					ParentSkeletalMeshComponent, ChildBoneName);
				if (ParentBoneName.IsNone())
				{
					continue;
				}
			}

			UPrimitiveComponent* ParentComponent = ParentSkeletalMeshComponent;
			if (!ParentComponent && WorldComponent)
			{
				ParentComponent = WorldComponent;
				ParentBoneName = WorldBoneName;
			}

			// When the control is created it will be added to SetName, which will be something like
			// WorldSpace_Head, as well as "All".
			const FName ControlName = CreateControl(
				ParentComponent, ParentBoneName, BonesInLimb.SkeletalMeshComponent.Get(), ChildBoneName,
				ControlData, FPhysicsControlTarget(), FName(SetName));

			if (!ControlName.IsNone())
			{
				LimbResult.Names.Add(ControlName);
				AllControls.Names.Add(ControlName);
				NameRecords.AddControl(ControlName, { LimbName, GetControlTypeName(ControlType) }, false);
			}
			else
			{
				UE_LOGF(LogPhysicsControl, Warning, 
					"Failed to make control for %ls in %ls", 
					*ChildBoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
			}
		}
	}
	return Result;
}

//======================================================================================================================
TMap<FName, FPhysicsControlNames> UPhysicsControlComponent::CreateControlsFromLimbBonesAndConstraintProfile(
	FPhysicsControlNames&                        AllControls,
	const TMap<FName, FPhysicsControlLimbBones>& LimbBones,
	const FName                                  ConstraintProfile,
	const bool                                   bEnabled)
{
	TMap<FName, FPhysicsControlNames> Result;
	Result.Reserve(LimbBones.Num());
	for (const TPair< FName, FPhysicsControlLimbBones>& LimbBoneEntry : LimbBones)
	{
		const FName LimbName = LimbBoneEntry.Key;
		const FPhysicsControlLimbBones& BonesInLimb = LimbBoneEntry.Value;

		USkeletalMeshComponent* SkeletalMeshComponent = BonesInLimb.SkeletalMeshComponent.Get();
		if (!SkeletalMeshComponent)
		{
			UE_LOGF(LogPhysicsControl, Warning, "No Skeletal mesh in limb %ls in %ls", 
				*LimbName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
			continue;
		}
		UPhysicsAsset* PhysicsAsset = SkeletalMeshComponent ? SkeletalMeshComponent->GetPhysicsAsset() : nullptr;
		if (!PhysicsAsset)
		{
			UE_LOGF(LogPhysicsControl, Warning, "No physics asset in skeletal mesh in %ls", 
				GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
			return Result;
		}

		const int32 NumBonesInLimb = BonesInLimb.BoneNames.Num();

		FPhysicsControlNames& LimbResult = Result.Add(LimbName);
		LimbResult.Names.Reserve(NumBonesInLimb);
		AllControls.Names.Reserve(AllControls.Names.Num() + NumBonesInLimb);

		for (int32 BoneIndex = 0; BoneIndex != NumBonesInLimb; ++BoneIndex)
		{
			// Don't create the parent space control if it's the first bone in a limb that had bIncludeParentBone
			if (BoneIndex == 0 && BonesInLimb.bFirstBoneIsAdditional)
			{
				continue; 
			}

			const FName ChildBoneName = BonesInLimb.BoneNames[BoneIndex];
			const FName ParentBoneName = 
				UE::PhysicsControl::GetPhysicalParentBone(SkeletalMeshComponent, ChildBoneName);
			if (ParentBoneName.IsNone())
			{
				continue;
			}

			FPhysicsControlData ControlData;
			// This is to match the skeletal mesh component velocity drive, which does not use the
			// target animation velocity.
			ControlData.AngularTargetVelocityMultiplier = 0;
			ControlData.LinearTargetVelocityMultiplier = 0;

			FConstraintProfileProperties ProfileProperties;
			if (!SkeletalMeshComponent->GetConstraintProfilePropertiesOrDefault(
				ProfileProperties, ChildBoneName, ConstraintProfile))
			
			{
				UE_LOGF(LogPhysicsControl, Warning, 
					"Failed get constraint profile for %ls in %ls",
					*ChildBoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
				continue;
			}

			UE::PhysicsControl::ConvertConstraintProfileToControlData(ControlData, ProfileProperties);
			ControlData.bEnabled = bEnabled;

			const FName ControlName = CreateControl(
				SkeletalMeshComponent, ParentBoneName, SkeletalMeshComponent, ChildBoneName,
				ControlData, FPhysicsControlTarget(), 
				FName(GetControlTypeName(EPhysicsControlType::ParentSpace).ToString().Append("_").Append(LimbName.ToString())));
			if (!ControlName.IsNone())
			{
				LimbResult.Names.Add(ControlName);
				AllControls.Names.Add(ControlName);
				NameRecords.AddControl(
					ControlName, { LimbName, GetControlTypeName(EPhysicsControlType::ParentSpace) }, false);
			}
			else
			{
				UE_LOGF(LogPhysicsControl, Warning, 
					"Failed to make control for %ls in %ls",
					*ChildBoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
			}
		}
	}
	return Result;
}

//======================================================================================================================
void UPhysicsControlComponent::DestroyAllControlsAndBodyModifiers()
{
	DestroyControl(TEXT("All"), true, true);
	DestroyBodyModifier(TEXT("All"), true, true);
}

//======================================================================================================================
bool UPhysicsControlComponent::DestroyControl(const FName Name, const bool bApplyToControlsWithName, const bool bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		bSuccessControl = DestroyControl(Name, EDestroyBehavior::RemoveRecord);
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		bSuccessSet = DestroyControlsInSet(Name);
	}

	bool bSuccess =
		(bApplyToControlsWithName && bSuccessControl) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"DestroyControl - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::DestroyControls(const TArray<FName>& Names, const bool bApplyToControlsWithName, const bool bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= DestroyControl(Name, bApplyToControlsWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::DestroyControlsInSet(const FName SetName)
{
	TArray<FName> Names = GetControlNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"DestroyControlsInSet - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return DestroyControls(Names, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlEnabled(
	const FName Name, const bool bEnable, const bool bApplyToControlsWithName, const bool bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		FPhysicsControlRecord* Record = FindControlRecord(Name);
		if (Record)
		{
			Record->PhysicsControl.ControlData.bEnabled = bEnable;
		}
		else
		{
			bSuccessControl = false;
		}
	}
	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		bSuccessSet = SetControlsInSetEnabled(Name, bEnable);
	}

	bool bSuccess =
		(bApplyToControlsWithName && bSuccessControl) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlEnabled - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}

	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlsEnabled(
	const TArray<FName>& Names, const bool bEnable, const bool bApplyToControlsWithName, const bool bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetControlEnabled(Name, bEnable, bApplyToControlsWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlsInSetEnabled(FName SetName, bool bEnable)
{
	const TArray<FName>& Names = GetControlNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetControlsInSetEnabled - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetControlsEnabled(GetControlNamesInSet(SetName), bEnable, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlParent(
	const FName          Name,
	UPrimitiveComponent* ParentComponent,
	const FName          ParentBoneName,
	const bool           bApplyToControlsWithName, 
	const bool           bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		FPhysicsControlRecord* Record = FindControlRecord(Name);
		if (Record)
		{
			if (USkeletalMeshComponent* SkeletalMeshComponent = 
				Cast<USkeletalMeshComponent>(Record->ParentComponent.Get()))
			{
				RemoveSkeletalMeshReferenceForCaching(SkeletalMeshComponent);
			}

			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ParentComponent))
			{
				AddSkeletalMeshReferenceForCaching(SkeletalMeshComponent);
			}

			Record->ParentComponent = ParentComponent;
			Record->PhysicsControl.ParentBoneName = ParentBoneName;
			Record->InitConstraint(this, Name, bWarnAboutInvalidNames);
		}
		else
		{
			bSuccessControl = false;
		}
	}
	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		bSuccessSet = SetControlParentsInSet(Name, ParentComponent, ParentBoneName);
	}

	bool bSuccess =
		(bApplyToControlsWithName && bSuccessControl) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlParent - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}

	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlParents(
	const TArray<FName>& Names,
	UPrimitiveComponent* ParentComponent,
	const FName          ParentBoneName,
	const bool           bApplyToControlsWithName,
	const bool           bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetControlParent(Name, ParentComponent, ParentBoneName, bApplyToControlsWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlParentsInSet(
	const FName          SetName,
	UPrimitiveComponent* ParentComponent,
	const FName          ParentBoneName)
{
	const TArray<FName>& Names = GetControlNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetControlParentsInSet - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetControlParents(Names, ParentComponent, ParentBoneName, true, false);
	}
}

//======================================================================================================================
// Note - some params passed by value to allow them to be unconnected and enable inline BP editing
bool UPhysicsControlComponent::SetControlData(
	const FName               Name, 
	const FPhysicsControlData ControlData,
	const bool                bApplyToControlsWithName,
	const bool                bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		FPhysicsControlRecord* Record = FindControlRecord(Name);
		if (Record)
		{
			Record->PhysicsControl.ControlData = ControlData;
		}
		else
		{
			bSuccessControl = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		bSuccessSet = SetControlDatasInSet(Name, ControlData);
	}

	bool bSuccess =
		(bApplyToControlsWithName && bSuccessControl) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlData - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}

	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlDatas(
	const TArray<FName>&      Names, 
	const FPhysicsControlData ControlData,
	const bool                bApplyToControlsWithName,
	const bool                bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetControlData(Name, ControlData, bApplyToControlsWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlDatasInSet(
	const FName               SetName,
	const FPhysicsControlData ControlData)
{
	const TArray<FName>& Names = GetControlNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetControlDatasInSet - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetControlDatas(Names, ControlData, true, false);
	}
}

//======================================================================================================================
// Note - some params passed by value to allow them to be unconnected and enable inline BP editing
bool UPhysicsControlComponent::SetControlSparseData(
	const FName                     Name, 
	const FPhysicsControlSparseData ControlData,
	const bool                      bApplyToControlsWithName,
	const bool                      bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		FPhysicsControlRecord* Record = FindControlRecord(Name);
		if (Record)
		{
			Record->PhysicsControl.ControlData.UpdateFromSparseData(ControlData);
		}
		else
		{
			bSuccessControl = false;
		}
	}
	
	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		bSuccessSet = SetControlSparseDatasInSet(Name, ControlData);
	}

	bool bSuccess =
		(bApplyToControlsWithName && bSuccessControl) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlSparseData - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlSparseDatas(
	const TArray<FName>&            Names, 
	const FPhysicsControlSparseData ControlData,
	const bool                      bApplyToControlsWithName,
	const bool                      bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetControlSparseData(Name, ControlData, bApplyToControlsWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlSparseDatasInSet(
	const FName                     SetName,
	const FPhysicsControlSparseData ControlData)
{
	const TArray<FName>& Names = GetControlNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetControlSparseDatasInSet - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetControlSparseDatas(Names, ControlData, true, false);
	}
}

//======================================================================================================================
// Note - some params passed by value to allow them to be unconnected and enable inline BP editing
bool UPhysicsControlComponent::SetControlMultiplier(
	const FName                      Name, 
	const FPhysicsControlMultiplier  ControlMultiplier, 
	const bool                       bEnableControl,
	const bool                       bApplyToControlsWithName,
	const bool                       bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		FPhysicsControlRecord* Record = FindControlRecord(Name);
		if (Record)
		{
			Record->PhysicsControl.ControlMultiplier = ControlMultiplier;
			if (bEnableControl)
			{
				Record->PhysicsControl.ControlData.bEnabled = true;
			}
		}
		else
		{
			bSuccessControl = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		bSuccessSet = SetControlMultipliersInSet(Name, ControlMultiplier, bEnableControl);
	}

	bool bSuccess =
		(bApplyToControlsWithName && bSuccessControl) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlMultiplier - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlMultipliers(
	const TArray<FName>&             Names,
	const FPhysicsControlMultiplier  ControlMultiplier, 
	const bool                       bEnableControl,
	const bool                       bApplyToControlsWithName,
	const bool                       bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetControlMultiplier(Name, ControlMultiplier, bEnableControl, bApplyToControlsWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlMultipliersInSet(
	const FName                      SetName,
	const FPhysicsControlMultiplier  ControlMultiplier, 
	const bool                       bEnableControl)
{
	TArray<FName> Names = GetControlNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetControlMultipliersInSet - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetControlMultipliers(Names, ControlMultiplier, bEnableControl, true, false);
	}
}

//======================================================================================================================
// Note - some params passed by value to allow them to be unconnected and enable inline BP editing
bool UPhysicsControlComponent::SetControlSparseMultiplier(
	const FName                            Name,
	const FPhysicsControlSparseMultiplier  ControlMultiplier,
	const bool                             bEnableControl,
	const bool                             bApplyToControlsWithName,
	const bool                             bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		FPhysicsControlRecord* Record = FindControlRecord(Name);
		if (Record)
		{
			Record->PhysicsControl.ControlMultiplier.UpdateFromSparseData(ControlMultiplier);
			if (bEnableControl)
			{
				Record->PhysicsControl.ControlData.bEnabled = true;
			}
		}
		else
		{
			bSuccessControl = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		bSuccessSet = SetControlSparseMultipliersInSet(Name, ControlMultiplier, bEnableControl);
	}

	bool bSuccess =
		(bApplyToControlsWithName && bSuccessControl) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlSparseMultiplier - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlSparseMultipliers(
	const TArray<FName>&                  Names,
	const FPhysicsControlSparseMultiplier ControlMultiplier,
	const bool                            bEnableControl,
	const bool                            bApplyToControlsWithName,
	const bool                            bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetControlSparseMultiplier(Name, ControlMultiplier, bEnableControl, bApplyToControlsWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlSparseMultipliersInSet(
	const FName                           SetName,
	const FPhysicsControlSparseMultiplier ControlMultiplier,
	const bool                            bEnableControl)
{
	TArray<FName> Names = GetControlNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetControlSparseMultipliersInSet - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetControlSparseMultipliers(Names, ControlMultiplier, bEnableControl, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlLinearData(
	const FName Name, 
	const float Strength, 
	const float DampingRatio, 
	const float ExtraDamping, 
	const float MaxForce, 
	const bool  bEnableControl,
	const bool  bApplyToControlsWithName,
	const bool  bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		FPhysicsControlRecord* Record = FindControlRecord(Name);
		if (Record)
		{
			Record->PhysicsControl.ControlData.LinearStrength = Strength;
			Record->PhysicsControl.ControlData.LinearDampingRatio = DampingRatio;
			Record->PhysicsControl.ControlData.LinearExtraDamping = ExtraDamping;
			Record->PhysicsControl.ControlData.MaxForce = MaxForce;
			if (bEnableControl)
			{
				Record->PhysicsControl.ControlData.bEnabled = true;
			}
		}
		else
		{
			bSuccessControl = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		TArray<FName> Names = GetControlNamesInSet(Name);
		for (FName ControlName : Names)
		{
			bSuccessSet &= SetControlLinearData(ControlName, Strength, DampingRatio, ExtraDamping, MaxForce, bEnableControl, true, false);
		}
	}

	bool bSuccess =
		(bApplyToControlsWithName && bSuccessControl) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlLinearData - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;

}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlAngularData(
	const FName Name, 
	const float Strength, 
	const float DampingRatio, 
	const float ExtraDamping, 
	const float MaxTorque, 
	const bool  bEnableControl,
	const bool  bApplyToControlsWithName,
	const bool  bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		FPhysicsControlRecord* Record = FindControlRecord(Name);
		if (Record)
		{
			Record->PhysicsControl.ControlData.AngularStrength = Strength;
			Record->PhysicsControl.ControlData.AngularDampingRatio = DampingRatio;
			Record->PhysicsControl.ControlData.AngularExtraDamping = ExtraDamping;
			Record->PhysicsControl.ControlData.MaxTorque = MaxTorque;
			if (bEnableControl)
			{
				Record->PhysicsControl.ControlData.bEnabled = true;
			}
		}
		else
		{
			bSuccessControl = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		TArray<FName> Names = GetControlNamesInSet(Name);
		for (FName ControlName : Names)
		{
			bSuccessSet &= SetControlAngularData(ControlName, Strength, DampingRatio, ExtraDamping, MaxTorque, bEnableControl, true, false);
		}
	}

	bool bSuccess =
		(bApplyToControlsWithName && bSuccessControl) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlAngularData - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlPoint(const FName Name, const FVector Position)
{
	FPhysicsControlRecord* Record = FindControlRecord(Name);
	if (Record)
	{
		Record->PhysicsControl.ControlData.bUseCustomControlPoint = true;
		Record->PhysicsControl.ControlData.CustomControlPoint = Position;
		Record->UpdateConstraintControlPoint();
		return true;
	}
	if (bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning, "SetControlPoint - invalid control name %ls for %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return false;
}

//======================================================================================================================
bool UPhysicsControlComponent::ResetControlPoint(const FName Name)
{
	FPhysicsControlRecord* Record = FindControlRecord(Name);
	if (Record)
	{
		Record->ResetControlPoint();
		return true;
	}
	if (bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning, "ResetControlPoint - invalid control name %ls for %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return false;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTarget(
	const FName                 Name, 
	const FPhysicsControlTarget ControlTarget, 
	const bool                  bEnableControl,
	const bool                  bApplyToControlsWithName,
	const bool                  bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		FPhysicsControlRecord* Record = FindControlRecord(Name);
		if (Record)
		{
			Record->ControlTarget = ControlTarget;
			if (bEnableControl)
			{
				Record->PhysicsControl.ControlData.bEnabled = true;
			}
		}
		else
		{
			bSuccessControl = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		bSuccessSet = SetControlTargetsInSet(Name, ControlTarget, bEnableControl);
	}

	bool bSuccess =
		(bApplyToControlsWithName && bSuccessControl) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlTarget - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargets(
	const TArray<FName>&        Names, 
	const FPhysicsControlTarget ControlTarget, 
	const bool                  bEnableControl,
	const bool                  bApplyToControlsWithName,
	const bool                  bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetControlTarget(Name, ControlTarget, bEnableControl, bApplyToControlsWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargetsInSet(
	const FName                 SetName, 
	const FPhysicsControlTarget ControlTarget, 
	const bool                  bEnableControl)
{
	TArray<FName> Names = GetControlNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetControlTargetsInSet - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetControlTargets(Names, ControlTarget, bEnableControl, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargetPositionAndOrientation(
	const FName    Name, 
	const FVector  Position, 
	const FRotator Orientation, 
	const float    VelocityDeltaTime, 
	const bool     bEnableControl, 
	const bool     bApplyControlPointToTarget,
	const bool     bApplyToControlsWithName,
	const bool     bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		bSuccessControl &= SetControlTargetPosition(
			Name, Position, VelocityDeltaTime, bEnableControl, bApplyControlPointToTarget, true, false);
		bSuccessControl &= SetControlTargetOrientation(
			Name, Orientation, VelocityDeltaTime, bEnableControl, bApplyControlPointToTarget, true, false);
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		bSuccessSet = SetControlTargetPositionsAndOrientationsInSet(
			Name, Position, Orientation, VelocityDeltaTime, bEnableControl, bApplyControlPointToTarget);
	}

	bool bSuccess =
		(bApplyToControlsWithName && bSuccessControl) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlTargetPositionAndOrientation - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargetPositionsAndOrientations(
	const TArray<FName>& Names,
	const FVector        Position,
	const FRotator       Orientation,
	const float          VelocityDeltaTime,
	const bool           bEnableControl,
	const bool           bApplyControlPointToTarget,
	const bool           bApplyToControlsWithName,
	const bool           bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetControlTargetPositionAndOrientation(
			Name, Position, Orientation, VelocityDeltaTime, bEnableControl, bApplyControlPointToTarget,
			bApplyToControlsWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargetPositionsAndOrientationsInSet(
	const FName          SetName,
	const FVector        Position,
	const FRotator       Orientation,
	const float          VelocityDeltaTime,
	const bool           bEnableControl,
	const bool           bApplyControlPointToTarget)
{
	const TArray<FName>& Names = GetControlNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetControlTargetPositionsAndOrientationsInSet - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetControlTargetPositionsAndOrientations(
			Names, Position, Orientation, VelocityDeltaTime, bEnableControl, bApplyControlPointToTarget, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargetPosition(
	const FName   Name, 
	const FVector Position, 
	const float   VelocityDeltaTime, 
	const bool    bEnableControl, 
	const bool    bApplyControlPointToTarget,
	const bool    bApplyToControlsWithName, 
	const bool    bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		FPhysicsControlRecord* Record = FindControlRecord(Name);
		if (Record)
		{
			if (VelocityDeltaTime != 0)
			{
				Record->ControlTarget.TargetVelocity =
					(Position - Record->ControlTarget.TargetPosition) / VelocityDeltaTime;
			}
			else
			{
				Record->ControlTarget.TargetVelocity = FVector::ZeroVector;
			}
			Record->ControlTarget.TargetPosition = Position;
			Record->ControlTarget.bApplyControlPointToTarget = bApplyControlPointToTarget;
			if (bEnableControl)
			{
				Record->PhysicsControl.ControlData.bEnabled = true;
			}
		}
		else
		{
			bSuccessControl = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		bSuccessSet = SetControlTargetPositionsInSet(
			Name, Position, VelocityDeltaTime, bEnableControl, bApplyControlPointToTarget);
	}

	bool bSuccess = 
		(bApplyToControlsWithName && bSuccessControl) || 
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlTargetPosition - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}

	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargetPositions(
	const TArray<FName>& Names,
	const FVector        Position,
	const float          VelocityDeltaTime,
	const bool           bEnableControl,
	const bool           bApplyControlPointToTarget,
	const bool           bApplyToControlsWithName,
	const bool           bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetControlTargetPosition(
			Name, Position, VelocityDeltaTime, bEnableControl, bApplyControlPointToTarget,
			bApplyToControlsWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargetPositionsInSet(
	const FName   SetName,
	const FVector Position,
	const float   VelocityDeltaTime,
	const bool    bEnableControl,
	const bool    bApplyControlPointToTarget)
{
	const TArray<FName>& Names = GetControlNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetControlTargetPositionsInSet - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetControlTargetPositions(
			Names, Position, VelocityDeltaTime, bEnableControl, bApplyControlPointToTarget, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargetOrientation(
	const FName    Name, 
	const FRotator Orientation, 
	const float    AngularVelocityDeltaTime, 
	const bool     bEnableControl, 
	const bool     bApplyControlPointToTarget,
	const bool     bApplyToControlsWithName,
	const bool     bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		FPhysicsControlRecord* Record = FindControlRecord(Name);
		if (Record)
		{
			if (AngularVelocityDeltaTime != 0)
			{
				const FQuat OldQ = Record->ControlTarget.TargetOrientation.Quaternion();
				const FQuat OrientationQ = Orientation.Quaternion();
				// Note that quats multiply in the opposite order to TMs
				const FQuat DeltaQ = (OrientationQ * OldQ.Inverse()).GetShortestArcWith(FQuat::Identity);
				Record->ControlTarget.TargetAngularVelocity =
					DeltaQ.ToRotationVector() / (UE_TWO_PI * AngularVelocityDeltaTime);
			}
			else
			{
				Record->ControlTarget.TargetAngularVelocity = FVector::ZeroVector;
			}
			Record->ControlTarget.TargetOrientation = Orientation;
			Record->ControlTarget.bApplyControlPointToTarget = bApplyControlPointToTarget;
			if (bEnableControl)
			{
				Record->PhysicsControl.ControlData.bEnabled = true;
			}
		}
		else
		{
			bSuccessControl = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		bSuccessSet &= SetControlTargetOrientationsInSet(
			Name, Orientation, AngularVelocityDeltaTime, bEnableControl, bApplyControlPointToTarget);
	}

	bool bSuccess =
		(bApplyToControlsWithName && bSuccessControl) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlTargetOrientation - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}

	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargetOrientations(
	const TArray<FName>& Names,
	const FRotator Orientation,
	const float    AngularVelocityDeltaTime,
	const bool     bEnableControl,
	const bool     bApplyControlPointToTarget,
	const bool     bApplyToControlsWithName,
	const bool     bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetControlTargetOrientation(
			Name, Orientation, AngularVelocityDeltaTime, bEnableControl, bApplyControlPointToTarget,
			bApplyToControlsWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargetOrientationsInSet(
	const FName    SetName,
	const FRotator Orientation,
	const float    AngularVelocityDeltaTime,
	const bool     bEnableControl,
	const bool     bApplyControlPointToTarget)
{
	const TArray<FName>& Names = GetControlNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetControlTargetOrientationsInSet - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetControlTargetOrientations(
			Names, Orientation, AngularVelocityDeltaTime, bEnableControl, bApplyControlPointToTarget, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargetPositionsFromArray(
	const TArray<FName>&   Names,
	const TArray<FVector>& Positions,
	const float            VelocityDeltaTime,
	const bool             bEnableControl,
	const bool             bApplyControlPointToTarget)
{
	int32 NumControlNames = Names.Num();
	int32 NumPositions = Positions.Num();
	if (NumControlNames != NumPositions)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlTargetPositionsFromArray - names and positions arrays sizes do not match in %ls", 
			GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return false;
	}
	for (int32 Index = 0 ; Index != NumControlNames ; ++Index)
	{
		SetControlTargetPosition(
			Names[Index], Positions[Index], VelocityDeltaTime, bEnableControl, bApplyControlPointToTarget);
	}
	return true;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargetOrientationsFromArray(
	const TArray<FName>&    Names,
	const TArray<FRotator>& Orientations,
	const float             VelocityDeltaTime,
	const bool              bEnableControl,
	const bool              bApplyControlPointToTarget)
{
	int32 NumControlNames = Names.Num();
	int32 NumOrientations = Orientations.Num();
	if (NumControlNames != NumOrientations)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlTargetOrientationsFromArray - names and orientations arrays sizes do not match in %ls",
			GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return false;
	}
	for (int32 Index = 0 ; Index != NumControlNames ; ++Index)
	{
		SetControlTargetOrientation(
			Names[Index], Orientations[Index], VelocityDeltaTime, bEnableControl, bApplyControlPointToTarget);
	}
	return true;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargetPositionsAndOrientationsFromArray(
	const TArray<FName>&    Names,
	const TArray<FVector>&  Positions,
	const TArray<FRotator>& Orientations,
	const float             VelocityDeltaTime,
	const bool              bEnableControl,
	const bool              bApplyControlPointToTarget)
{
	int32 NumControlNames = Names.Num();
	int32 NumPositions = Positions.Num();
	int32 NumOrientations = Orientations.Num();
	if (NumControlNames != NumPositions || NumControlNames != NumOrientations)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlTargetPositionsAndOrientationsFromArray - names and positions/orientation arrays sizes do not match in %ls", 
			GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return false;
	}
	for (int32 Index = 0; Index != NumControlNames; ++Index)
	{
		SetControlTargetPositionAndOrientation(
			Names[Index], Positions[Index], Orientations[Index], VelocityDeltaTime, bEnableControl, bApplyControlPointToTarget);
	}
	return true;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlTargetPoses(
	const FName    Name,
	const FVector  ParentPosition, 
	const FRotator ParentOrientation,
	const FVector  ChildPosition, 
	const FRotator ChildOrientation,
	const float    VelocityDeltaTime, 
	const bool     bEnableControl)
{
	FPhysicsControlRecord* Record = FindControlRecord(Name);
	if (Record)
	{
		const FTransform ParentTM(ParentOrientation, ParentPosition, FVector::One());
		const FTransform ChildTM(ChildOrientation, ChildPosition, FVector::One());

		const FTransform OffsetTM = ChildTM * ParentTM.Inverse();
		const FVector Position = OffsetTM.GetTranslation();
		const FQuat OrientationQ = OffsetTM.GetRotation();

		if (VelocityDeltaTime != 0)
		{
			const FQuat OldQ = Record->ControlTarget.TargetOrientation.Quaternion();
			// Note that quats multiply in the opposite order to TMs
			FQuat DeltaQ = (OrientationQ * OldQ.Inverse()).GetShortestArcWith(FQuat::Identity);
			Record->ControlTarget.TargetAngularVelocity = DeltaQ.ToRotationVector() / (UE_TWO_PI * VelocityDeltaTime);

			Record->ControlTarget.TargetVelocity =
				(Position - Record->ControlTarget.TargetPosition) / VelocityDeltaTime;
		}
		else
		{
			Record->ControlTarget.TargetAngularVelocity = FVector::ZeroVector;
			Record->ControlTarget.TargetVelocity = FVector::ZeroVector;
		}
		Record->ControlTarget.TargetOrientation = OrientationQ.Rotator();
		Record->ControlTarget.TargetPosition = Position;
		Record->ControlTarget.bApplyControlPointToTarget = true;
		if (bEnableControl)
		{
			Record->PhysicsControl.ControlData.bEnabled = true;
		}
		return true;
	}
	if (bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning, "SetControlTargetPoses - invalid control name %ls for %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return false;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlUseSkeletalAnimation(
	const FName Name,
	const bool  bUseSkeletalAnimation,
	const float AngularTargetVelocityMultiplier,
	const float LinearTargetVelocityMultiplier,
	const bool  bApplyToControlsWithName,
	const bool  bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		FPhysicsControl* PhysicsControl = FindControl(Name);
		if (PhysicsControl)
		{
			PhysicsControl->ControlData.bUseSkeletalAnimation = bUseSkeletalAnimation;
			PhysicsControl->ControlData.AngularTargetVelocityMultiplier = AngularTargetVelocityMultiplier;
			PhysicsControl->ControlData.LinearTargetVelocityMultiplier = LinearTargetVelocityMultiplier;
		}
		else
		{
			bSuccessControl = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		SetControlsInSetUseSkeletalAnimation(
			Name, bUseSkeletalAnimation, AngularTargetVelocityMultiplier, LinearTargetVelocityMultiplier);
	}

	bool bSuccess =
		(bApplyToControlsWithName && bSuccessControl) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlUseSkeletalAnimation - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlsUseSkeletalAnimation(
	const TArray<FName>& Names,
	const bool           bUseSkeletalAnimation,
	const float          AngularTargetVelocityMultiplier,
	const float          LinearTargetVelocityMultiplier,
	const bool           bApplyToControlsWithName,
	const bool           bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetControlUseSkeletalAnimation(
			Name, bUseSkeletalAnimation, AngularTargetVelocityMultiplier, LinearTargetVelocityMultiplier,
			bApplyToControlsWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlsInSetUseSkeletalAnimation(
	const FName SetName,
	const bool  bUseSkeletalAnimation,
	const float AngularTargetVelocityMultiplier,
	const float LinearTargetVelocityMultiplier)
{
	const TArray<FName>& Names = GetControlNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetControlsInSetUseSkeletalAnimation - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetControlsUseSkeletalAnimation(
			Names, bUseSkeletalAnimation, AngularTargetVelocityMultiplier,
			LinearTargetVelocityMultiplier, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlDisableCollision(
	const FName   Name, 
	const bool    bDisableCollision,
	const bool    bApplyToControlsWithName,
	const bool    bApplyToSetsWithName)
{
	bool bSuccessControl = true;
	if (bApplyToControlsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		FPhysicsControl* PhysicsControl = FindControl(Name);
		if (PhysicsControl)
		{
			PhysicsControl->ControlData.bDisableCollision = bDisableCollision;
		}
		else
		{
			bSuccessControl = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		bSuccessSet = SetControlsInSetDisableCollision(Name, bDisableCollision);
	}
	bool bSuccess =
		(bApplyToControlsWithName && bSuccessControl) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToControlsWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetControlDisableCollision - failed to find controls/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlsDisableCollision(
	const TArray<FName>& Names, 
	const bool           bDisableCollision,
	const bool           bApplyToControlsWithName,
	const bool           bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetControlDisableCollision(Name, bDisableCollision, bApplyToControlsWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlsInSetDisableCollision(const FName SetName, const bool bDisableCollision)
{
	const TArray<FName>& Names = GetControlNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetControlsInSetDisableCollision - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetControlsDisableCollision(GetControlNamesInSet(SetName), bDisableCollision, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetControlsInSetDriveLimitViolationResponse(
	const FName Set, const EAngularDriveLimitViolationResponse Response)
{
	const TArray<FName>& Names = GetControlNamesInSet(Set);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetControlsInSetDriveLimitViolationResponse - Set %ls not found in %ls",
				*Set.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		}
		return false;
	}

	bool bSuccess = true;
	for (const FName ControlName : Names)
	{
		FPhysicsControlRecord* Record = FindControlRecord(ControlName);
		if (!Record)
		{
			bSuccess = false;
			continue;
		}
		USkeletalMeshComponent* ChildSKM = Cast<USkeletalMeshComponent>(Record->ChildComponent.Get());
		if (!ChildSKM)
		{
			bSuccess = false;
			continue;
		}
		UPhysicsAsset* PhysAsset = ChildSKM->GetPhysicsAsset();
		if (!PhysAsset)
		{
			bSuccess = false;
			continue;
		}
		if (!Record->RefreshJointConstraintIndex(ChildSKM, PhysAsset))
		{
			bSuccess = false;
			continue;
		}
		FConstraintInstance* ConstraintInstance = ChildSKM->GetConstraintInstanceByIndex(Record->JointConstraintIndex);
		if (!ConstraintInstance)
		{
			bSuccess = false;
			continue;
		}
		ConstraintInstance->ProfileInstance.AngularDrive.LimitViolationResponse = Response;
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::GetControlData(const FName Name, FPhysicsControlData& ControlData) const
{
	const FPhysicsControl* PhysicsControl = FindControl(Name);
	if (PhysicsControl)
	{
		ControlData = PhysicsControl->ControlData;
		return true;
	}
	if (bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning, "GetControlData - invalid control name %ls for %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return false;
}

//======================================================================================================================
bool UPhysicsControlComponent::GetControlMultiplier(const FName Name, FPhysicsControlMultiplier& ControlMultiplier) const
{
	const FPhysicsControl* PhysicsControl = FindControl(Name);
	if (PhysicsControl)
	{
		ControlMultiplier = PhysicsControl->ControlMultiplier;
		return true;
	}
	if (bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning, "GetControlMultiplier - invalid control name %ls for %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return false;
}

//======================================================================================================================
bool UPhysicsControlComponent::GetControlTarget(const FName Name, FPhysicsControlTarget& ControlTarget) const
{
	const FPhysicsControlRecord* Record = FindControlRecord(Name);
	if (Record)
	{
		ControlTarget = Record->ControlTarget;
		return true;
	}
	if (bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning, "GetControlTarget - invalid control name %ls for %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return false;
}

//======================================================================================================================
bool UPhysicsControlComponent::GetControlEnabled(const FName Name) const
{
	const FPhysicsControlRecord* Record = FindControlRecord(Name);
	if (Record)
	{
		return Record->PhysicsControl.IsEnabled();
	}
	if (bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning, "GetControlEnabled - invalid control name %ls for %ls", 
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return false;
}

//======================================================================================================================
FName UPhysicsControlComponent::CreateBodyModifier(
	UPrimitiveComponent*              Component,
	const FName                       BoneName,
	const FName                       Set,
	const FPhysicsControlModifierData BodyModifierData)
{
	const FName Name = UE::PhysicsControl::GetUniqueBodyModifierName(
		Component, BoneName, BodyModifierRecords, TEXT(""));
	if (CreateNamedBodyModifier(Name, Component, BoneName, Set, BodyModifierData))
	{
		return Name;
	}
	return FName();
}

//======================================================================================================================
bool UPhysicsControlComponent::CreateNamedBodyModifier(
	const FName                       Name,
	UPrimitiveComponent*              Component,
	const FName                       BoneName,
	const FName                       Set,
	const FPhysicsControlModifierData BodyModifierData)
{
	if (FindBodyModifierRecord(Name))
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"CreateNamedBodyModifier - modifier with name %ls already exists in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return false;
	}

	if (!Component)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"Unable to make a PhysicsBodyModifier as the mesh component has not been set in %ls", 
			GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return false;
	}

	FPhysicsBodyModifierRecord& Modifier = BodyModifierRecords.Add(
		Name, FPhysicsBodyModifierRecord(Component, BoneName, BodyModifierData));

	USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Component);
	if (SkeletalMeshComponent)
	{
		AddSkeletalMeshReferenceForCaching(SkeletalMeshComponent);
		AddSkeletalMeshReferenceForModifier(SkeletalMeshComponent);
	}

	NameRecords.AddBodyModifier(Name, Set);

	return true;
}

//======================================================================================================================
TArray<FName> UPhysicsControlComponent::CreateBodyModifiersFromSkeletalMeshBelow(
	USkeletalMeshComponent*           SkeletalMeshComponent,
	const FName                       BoneName,
	const bool                        bIncludeSelf,
	const FName                       Set,
	const FPhysicsControlModifierData BodyModifierData)
{
	TArray<FName> Result;
	UPhysicsAsset* PhysicsAsset = SkeletalMeshComponent ? SkeletalMeshComponent->GetPhysicsAsset() : nullptr;
	if (!PhysicsAsset)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"CreateBodyModifiersFromSkeletalMeshBelow - No physics asset available in %ls", 
			GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return Result;
	}

	SkeletalMeshComponent->ForEachBodyBelow(
		BoneName, bIncludeSelf, /*bSkipCustomType=*/false,
		[this, PhysicsAsset, SkeletalMeshComponent, Set, BodyModifierData, &Result](const FBodyInstance* BI)
		{
			if (USkeletalBodySetup* BodySetup = Cast<USkeletalBodySetup>(BI->BodySetup.Get()))
			{
				const FName BoneName = PhysicsAsset->SkeletalBodySetups[BI->InstanceBodyIndex]->BoneName;
				const FName BodyModifierName = CreateBodyModifier(
					SkeletalMeshComponent, BoneName, Set, BodyModifierData);
				Result.Add(BodyModifierName);
			}
		});

	return Result;
}

//======================================================================================================================
TMap<FName, FPhysicsControlNames> UPhysicsControlComponent::CreateBodyModifiersFromLimbBones(
	FPhysicsControlNames&                        AllBodyModifiers,
	const TMap<FName, FPhysicsControlLimbBones>& LimbBones,
	const FPhysicsControlModifierData            BodyModifierData)
{
	TMap<FName, FPhysicsControlNames> Result;
	Result.Reserve(LimbBones.Num());

	for (const TPair<FName, FPhysicsControlLimbBones>& LimbBoneEntry : LimbBones)
	{
		const FName LimbName = LimbBoneEntry.Key;
		const FPhysicsControlLimbBones& BonesInLimb = LimbBoneEntry.Value;

		if (!BonesInLimb.SkeletalMeshComponent.Get())
		{
			UE_LOGF(LogPhysicsControl, Warning, "No Skeletal mesh in limb %ls", *LimbName.ToString());
			continue;
		}

		const int32 NumBonesInLimb = BonesInLimb.BoneNames.Num();

		FPhysicsControlNames& LimbResult = Result.Add(LimbName);
		LimbResult.Names.Reserve(NumBonesInLimb);
		AllBodyModifiers.Names.Reserve(AllBodyModifiers.Names.Num() + NumBonesInLimb);

		for (const FName BoneName : BonesInLimb.BoneNames)
		{
			const FName BodyModifierName = CreateBodyModifier(
				BonesInLimb.SkeletalMeshComponent.Get(), BoneName, LimbName, BodyModifierData);
			if (!BodyModifierName.IsNone())
			{
				LimbResult.Names.Add(BodyModifierName);
				AllBodyModifiers.Names.Add(BodyModifierName);
			}
			else
			{
				UE_LOGF(LogPhysicsControl, Warning, 
					"Failed to make body modifier for %ls in %ls",
					*BoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
			}
		}
	}
	return Result;
}

//======================================================================================================================
bool UPhysicsControlComponent::DestroyBodyModifier(
	const FName Name, const bool bApplyToModifiersWithName, const bool bApplyToSetsWithName)
{
	bool bSuccessModifier = true;
	if (bApplyToModifiersWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		bSuccessModifier &= DestroyBodyModifier(Name, EDestroyBehavior::RemoveRecord);
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		bSuccessSet = DestroyBodyModifiersInSet(Name);
	}

	bool bSuccess =
		(bApplyToModifiersWithName && bSuccessModifier) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToModifiersWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"DestroyBodyModifier - failed to find modifier/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::DestroyBodyModifiers(
	const TArray<FName>& Names, const bool bApplyToModifiersWithName, const bool bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= DestroyBodyModifier(Name, bApplyToModifiersWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::DestroyBodyModifiersInSet(const FName SetName)
{
	TArray<FName> Names = GetBodyModifierNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"DestroyBodyModifiersInSet - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return DestroyBodyModifiers(Names, true, false);
	}
}

//======================================================================================================================
// Note - some params passed by value to allow them to be unconnected and enable inline BP editing
bool UPhysicsControlComponent::SetBodyModifierData(
	const FName                       Name,
	const FPhysicsControlModifierData ModifierData, 
	const bool                        bApplyToModifiersWithName, 
	const bool                        bApplyToSetsWithName)
{
	bool bSuccessModifier = true;
	if (bApplyToModifiersWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		FPhysicsBodyModifierRecord* Record = FindBodyModifierRecord(Name);
		if (Record)
		{
			Record->BodyModifier.ModifierData = ModifierData;
		}
		else
		{
			bSuccessModifier = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		bSuccessSet = SetBodyModifierDatasInSet(Name, ModifierData);
	}

	bool bSuccess =
		(bApplyToModifiersWithName && bSuccessModifier) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToModifiersWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetBodyModifierData - failed to find modifier/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifierDatas(
	const TArray<FName>&              Names,
	const FPhysicsControlModifierData ModifierData,
	const bool                        bApplyToModifiersWithName,
	const bool                        bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess = SetBodyModifierData(Name, ModifierData, bApplyToModifiersWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifierDatasInSet(
	const FName                       SetName,
	const FPhysicsControlModifierData ModifierData)
{
	TArray<FName> Names = GetBodyModifierNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetBodyModifierDatasInSet - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetBodyModifierDatas(Names, ModifierData, true, false);
	}
}

//======================================================================================================================
// Note - some params passed by value to allow them to be unconnected and enable inline BP editing
bool UPhysicsControlComponent::SetBodyModifierSparseData(
	const FName                             Name,
	const FPhysicsControlModifierSparseData ModifierData,
	const bool                              bApplyToModifiersWithName,
	const bool                              bApplyToSetsWithName)
{
	bool bSuccessModifier = true;
	if (bApplyToModifiersWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		FPhysicsBodyModifierRecord* Record = FindBodyModifierRecord(Name);
		if (Record)
		{
			Record->BodyModifier.ModifierData.UpdateFromSparseData(ModifierData);
		}
		else
		{
			bSuccessModifier = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		bSuccessSet = SetBodyModifierSparseDatasInSet(Name, ModifierData);
	}

	bool bSuccess =
		(bApplyToModifiersWithName && bSuccessModifier) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToModifiersWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetBodyModifierSparseData - failed to find modifier/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;

}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifierSparseDatas(
	const TArray<FName>&                    Names,
	const FPhysicsControlModifierSparseData ModifierData,
	const bool                              bApplyToModifiersWithName,
	const bool                              bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetBodyModifierSparseData(Name, ModifierData, bApplyToModifiersWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifierSparseDatasInSet(
	const FName                             SetName,
	const FPhysicsControlModifierSparseData ModifierData)
{
	TArray<FName> Names = GetBodyModifierNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetBodyModifierSparseDatasInSet - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetBodyModifierSparseDatas(Names, ModifierData, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifierKinematicTarget(
	const FName    Name,
	const FVector  KinematicTargetPosition,
	const FRotator KinematicTargetOrienation,
	const bool     bMakeKinematic)
{
	FPhysicsBodyModifierRecord* Record = FindBodyModifierRecord(Name);
	if (Record)
	{
		Record->KinematicTarget.Translation = KinematicTargetPosition;
		Record->KinematicTarget.Rotation = KinematicTargetOrienation.Quaternion();
		if (bMakeKinematic)
		{
			Record->BodyModifier.ModifierData.MovementType = EPhysicsMovementType::Kinematic;
		}
		return true;
	}
	if (bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning, "SetBodyModifierKinematicTarget - invalid modifier name %ls for %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return false;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifierMovementType(
	const FName                Name,
	const EPhysicsMovementType MovementType,
	const bool                 bApplyToModifiersWithName,
	const bool                 bApplyToSetsWithName)
{
	bool bSuccessModifier = true;
	if (bApplyToModifiersWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		FPhysicsBodyModifierRecord* Record = FindBodyModifierRecord(Name);
		if (Record)
		{
			Record->BodyModifier.ModifierData.MovementType = MovementType;
		}
		else
		{
			bSuccessModifier = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);

		bSuccessSet = SetBodyModifiersInSetMovementType(Name, MovementType);
	}

	bool bSuccess =
		(bApplyToModifiersWithName && bSuccessModifier) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToModifiersWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetBodyModifierMovementType - failed to find modifier/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifiersMovementType(
	const TArray<FName>&       Names,
	const EPhysicsMovementType MovementType,
	const bool                 bApplyToModifiersWithName,
	const bool                 bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetBodyModifierMovementType(Name, MovementType, bApplyToModifiersWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifiersInSetMovementType(
	const FName                SetName,
	const EPhysicsMovementType MovementType)
{
	TArray<FName> Names = GetBodyModifierNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetBodyModifiersInSetMovementType - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetBodyModifiersMovementType(Names, MovementType, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifierCollisionType(
	const FName                   Name,
	const ECollisionEnabled::Type CollisionType,
	const bool                    bApplyToModifiersWithName,
	const bool                    bApplyToSetsWithName)
{
	bool bSuccessModifier = true;
	if (bApplyToModifiersWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		FPhysicsBodyModifierRecord* Record = FindBodyModifierRecord(Name);
		if (Record)
		{
			Record->BodyModifier.ModifierData.CollisionType = CollisionType;
		}
		else
		{
			bSuccessModifier = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		bSuccessSet = SetBodyModifiersInSetCollisionType(Name, CollisionType);
	}

	bool bSuccess =
		(bApplyToModifiersWithName && bSuccessModifier) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToModifiersWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetBodyModifierCollisionType - failed to find modifier/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifiersCollisionType(
	const TArray<FName>&          Names,
	const ECollisionEnabled::Type CollisionType,
	const bool                    bApplyToModifiersWithName,
	const bool                    bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetBodyModifierCollisionType(Name, CollisionType, bApplyToModifiersWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifiersInSetCollisionType(
	const FName                   SetName,
	const ECollisionEnabled::Type CollisionType)
{
	TArray<FName> Names = GetBodyModifierNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetBodyModifiersInSetCollisionType - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetBodyModifiersCollisionType(Names, CollisionType, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifierGravityMultiplier(
	const FName Name,
	const float GravityMultiplier,
	const bool  bApplyToModifiersWithName,
	const bool  bApplyToSetsWithName)
{
	bool bSuccessModifier = true;
	if (bApplyToModifiersWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		FPhysicsBodyModifierRecord* Record = FindBodyModifierRecord(Name);
		if (Record)
		{
			Record->BodyModifier.ModifierData.GravityMultiplier = GravityMultiplier;
		}
		else
		{
			bSuccessModifier = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		bSuccessSet = SetBodyModifiersInSetGravityMultiplier(Name, GravityMultiplier);
	}

	bool bSuccess =
		(bApplyToModifiersWithName && bSuccessModifier) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToModifiersWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetBodyModifierGravityMultiplier - failed to find modifier/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;

}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifiersGravityMultiplier(
	const TArray<FName>& Names,
	const float          GravityMultiplier,
	const bool           bApplyToModifiersWithName,
	const bool           bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess = SetBodyModifierGravityMultiplier(Name, GravityMultiplier, bApplyToModifiersWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifiersInSetGravityMultiplier(
	const FName SetName,
	const float GravityMultiplier)
{
	TArray<FName> Names = GetBodyModifierNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetBodyModifiersInSetGravityMultiplier - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetBodyModifiersGravityMultiplier(Names, GravityMultiplier, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifierPhysicsBlendWeight(
	const FName Name,
	const float PhysicsBlendWeight,
	const bool  bApplyToModifiersWithName,
	const bool  bApplyToSetsWithName)
{
	bool bSuccessModifier = true;
	if (bApplyToModifiersWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		FPhysicsBodyModifierRecord* Record = FindBodyModifierRecord(Name);
		if (Record)
		{
			Record->BodyModifier.ModifierData.PhysicsBlendWeight = PhysicsBlendWeight;
		}
		else
		{
			bSuccessModifier = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		bSuccessSet = SetBodyModifiersInSetPhysicsBlendWeight(Name, PhysicsBlendWeight);
	}

	bool bSuccess =
		(bApplyToModifiersWithName && bSuccessModifier) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToModifiersWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetBodyModifierPhysicsBlendWeight - failed to find modifier/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifiersPhysicsBlendWeight(
	const TArray<FName>& Names,
	const float          PhysicsBlendWeight,
	const bool           bApplyToModifiersWithName,
	const bool           bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetBodyModifierPhysicsBlendWeight(Name, PhysicsBlendWeight, bApplyToModifiersWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifiersInSetPhysicsBlendWeight(
	const FName SetName,
	const float PhysicsBlendWeight)
{
	TArray<FName> Names = GetBodyModifierNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetBodyModifiersInSetPhysicsBlendWeight - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetBodyModifiersPhysicsBlendWeight(Names, PhysicsBlendWeight, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifierKinematicTargetSpace(
	const FName                               Name,
	const EPhysicsControlKinematicTargetSpace KinematicTargetSpace,
	const bool                                bApplyToModifiersWithName,
	const bool                                bApplyToSetsWithName)
{
	bool bSuccessModifier = true;
	if (bApplyToModifiersWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		FPhysicsBodyModifierRecord* Record = FindBodyModifierRecord(Name);
		if (Record)
		{
			Record->BodyModifier.ModifierData.KinematicTargetSpace = KinematicTargetSpace;
		}
		else
		{
			bSuccessModifier = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		bSuccessSet = SetBodyModifiersInSetKinematicTargetSpace(Name, KinematicTargetSpace);
	}

	bool bSuccess =
		(bApplyToModifiersWithName && bSuccessModifier) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToModifiersWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetBodyModifierKinematicTargetSpace - failed to find modifier/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifiersKinematicTargetSpace(
	const TArray<FName>&                      Names,
	const EPhysicsControlKinematicTargetSpace KinematicTargetSpace,
	const bool                                bApplyToModifiersWithName,
	const bool                                bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetBodyModifierKinematicTargetSpace(Name, KinematicTargetSpace, bApplyToModifiersWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifiersInSetKinematicTargetSpace(
	const FName                               SetName,
	const EPhysicsControlKinematicTargetSpace KinematicTargetSpace)
{
	TArray<FName> Names = GetBodyModifierNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetBodyModifiersInSetKinematicTargetSpace - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetBodyModifiersKinematicTargetSpace(Names, KinematicTargetSpace, true, false);
	}
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifierUpdateKinematicFromSimulation(
	const FName Name,
	const bool  bUpdateKinematicFromSimulation,
	const bool  bApplyToModifiersWithName,
	const bool  bApplyToSetsWithName)
{
	bool bSuccessModifier = true;
	if (bApplyToModifiersWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		FPhysicsBodyModifierRecord* Record = FindBodyModifierRecord(Name);
		if (Record)
		{
			Record->BodyModifier.ModifierData.bUpdateKinematicFromSimulation = bUpdateKinematicFromSimulation;
		}
		else
		{
			bSuccessModifier = false;
		}
	}

	bool bSuccessSet = true;
	if (bApplyToSetsWithName)
	{
		FDisableNameWarnings DisableNameWarnings(bWarnAboutInvalidNames);
		bSuccessSet = SetBodyModifiersInSetUpdateKinematicFromSimulation(Name, bUpdateKinematicFromSimulation);
	}

	bool bSuccess =
		(bApplyToModifiersWithName && bSuccessModifier) ||
		(bApplyToSetsWithName && bSuccessSet) ||
		(!bApplyToModifiersWithName && !bApplyToSetsWithName);
	if (!bSuccess && bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetBodyModifierUpdateKinematicFromSimulation - failed to find modifier/set %ls in %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return bSuccess;

}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifiersUpdateKinematicFromSimulation(
	const TArray<FName>& Names,
	const bool           bUpdateKinematicFromSimulation,
	const bool           bApplyToModifiersWithName,
	const bool           bApplyToSetsWithName)
{
	bool bSuccess = true;
	for (FName Name : Names)
	{
		bSuccess &= SetBodyModifierUpdateKinematicFromSimulation(
			Name, bUpdateKinematicFromSimulation, bApplyToModifiersWithName, bApplyToSetsWithName);
	}
	return bSuccess;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetBodyModifiersInSetUpdateKinematicFromSimulation(
	const FName SetName,
	const bool  bUpdateKinematicFromSimulation)
{
	TArray<FName> Names = GetBodyModifierNamesInSet(SetName);
	if (Names.IsEmpty())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"SetBodyModifiersInSetUpdateKinematicFromSimulation - Set %ls not found in %ls",
				*SetName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

		}
		return false;
	}
	else
	{
		return SetBodyModifiersUpdateKinematicFromSimulation(Names, bUpdateKinematicFromSimulation, true, false);
	}
}

//======================================================================================================================
const TArray<FName>& UPhysicsControlComponent::GetAllControlNames() const
{
	return GetControlNamesInSet(TEXT("All"));
}

//======================================================================================================================
bool UPhysicsControlComponent::CreateControlsAndBodyModifiersFromLimbBones(
	FPhysicsControlNames&                       AllWorldSpaceControls,
	TMap<FName, FPhysicsControlNames>&          LimbWorldSpaceControls,
	FPhysicsControlNames&                       AllParentSpaceControls,
	TMap<FName, FPhysicsControlNames>&          LimbParentSpaceControls,
	FPhysicsControlNames&                       AllBodyModifiers,
	TMap<FName, FPhysicsControlNames>&          LimbBodyModifiers,
	USkeletalMeshComponent*                     SkeletalMeshComponent,
	const TArray<FPhysicsControlLimbSetupData>& LimbSetupData,
	const FPhysicsControlData                   WorldSpaceControlData,
	const FPhysicsControlData                   ParentSpaceControlData,
	const FPhysicsControlModifierData           BodyModifierData,
	UPrimitiveComponent*                        WorldComponent,
	FName                                       WorldBoneName)
{
	UPhysicsAsset* PhysicsAsset = SkeletalMeshComponent ? SkeletalMeshComponent->GetPhysicsAsset() : nullptr;
	if (!PhysicsAsset)
	{
		UE_LOGF(LogPhysicsControl, Warning, "No physics asset in skeletal mesh");
		return false;
	}

	TMap<FName, FPhysicsControlLimbBones> LimbBones = 
		GetLimbBonesFromSkeletalMesh(SkeletalMeshComponent, LimbSetupData);

	LimbWorldSpaceControls = CreateControlsFromLimbBones(
		AllWorldSpaceControls, LimbBones, EPhysicsControlType::WorldSpace, 
		WorldSpaceControlData, WorldComponent, WorldBoneName);

	LimbParentSpaceControls = CreateControlsFromLimbBones(
		AllParentSpaceControls, LimbBones, EPhysicsControlType::ParentSpace,
		ParentSpaceControlData);

	LimbBodyModifiers = CreateBodyModifiersFromLimbBones(AllBodyModifiers, LimbBones, BodyModifierData);

	return true;
}

//======================================================================================================================
bool UPhysicsControlComponent::CreateControlsAndBodyModifiersFromPhysicsControlAsset(
	USkeletalMeshComponent* SkeletalMeshComponent,
	UPrimitiveComponent*    WorldComponent,
	FName                   WorldBoneName)
{
	if (!PhysicsControlAsset.LoadSynchronous())
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"CreateControlsAndBodyModifiersFromPhysicsControlAsset - unable to get/load the control profile asset in %ls",
			GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return false;
	}

	FPhysicsControlNames AllWorldSpaceControls;
	TMap<FName, FPhysicsControlNames> LimbWorldSpaceControls;
	FPhysicsControlNames AllParentSpaceControls;
	TMap<FName, FPhysicsControlNames> LimbParentSpaceControls;
	FPhysicsControlNames AllBodyModifiers;
	TMap<FName, FPhysicsControlNames> LimbBodyModifiers;

	if (!CreateControlsAndBodyModifiersFromLimbBones(
		AllWorldSpaceControls, LimbWorldSpaceControls, AllParentSpaceControls, LimbParentSpaceControls, 
		AllBodyModifiers, LimbBodyModifiers,
		SkeletalMeshComponent,
		PhysicsControlAsset->CharacterSetupData.LimbSetupData,
		PhysicsControlAsset->CharacterSetupData.DefaultWorldSpaceControlData,
		PhysicsControlAsset->CharacterSetupData.DefaultParentSpaceControlData,
		PhysicsControlAsset->CharacterSetupData.DefaultBodyModifierData,
		WorldComponent,
		WorldBoneName))
	{
		// We assume that if this one fails, then everything fails. Also that if we can create the
		// basic setup, then the rest is OK too.
		return false;
	}

	// Create additional controls
	for (const TPair<FName, FPhysicsControlCreationData>& ControlPair : 
		PhysicsControlAsset->AdditionalControlsAndModifiers.Controls)
	{
		FName ControlName = ControlPair.Key;
		const FPhysicsControlCreationData& ControlCreationData = ControlPair.Value;
		if (CreateNamedControl(ControlName,
			!ControlCreationData.Control.ParentBoneName.IsNone() 
			? SkeletalMeshComponent : nullptr, ControlCreationData.Control.ParentBoneName,
			SkeletalMeshComponent, ControlCreationData.Control.ChildBoneName,
			ControlCreationData.Control.ControlData, FPhysicsControlTarget(), FName()))
		{
			for (FName SetName : ControlCreationData.Sets)
			{
				NameRecords.AddControl(ControlName, SetName);
			}
		}
	}

	// Create additional modifiers
	for (const TPair<FName, FPhysicsBodyModifierCreationData>& ModifierPair : 
		PhysicsControlAsset->AdditionalControlsAndModifiers.Modifiers)
	{
		FName ModifierName = ModifierPair.Key;
		const FPhysicsBodyModifierCreationData& ModifierCreationData = ModifierPair.Value;
		if (CreateNamedBodyModifier(ModifierName,
			SkeletalMeshComponent, ModifierCreationData.Modifier.BoneName,
			FName(), ModifierCreationData.Modifier.ModifierData))
		{
			for (FName SetName : ModifierCreationData.Sets)
			{
				NameRecords.AddBodyModifier(ModifierName, SetName);
			}
		}
	}

	// Create any additional sets that have been requested
	UE::PhysicsControl::CreateAdditionalSets(
		PhysicsControlAsset->AdditionalSets, BodyModifierRecords, ControlRecords, NameRecords);

	for (FPhysicsControlControlAndModifierUpdates& Updates : PhysicsControlAsset->InitialControlAndModifierUpdates)
	{
		ApplyControlAndModifierUpdates(Updates);
	}
	return true;
}

//======================================================================================================================
bool UPhysicsControlComponent::InvokeControlProfile(
	const FName ProfileName, const FName ControlSetMask, const FName BodyModifierSetMask)
{
	if (!PhysicsControlAsset.IsValid())
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning, 
				"InvokeControlProfile - control profile asset is invalid or missing for %ls",
				GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		}
		return false;
	}

	const FPhysicsControlControlAndModifierUpdates* ControlAndModifierUpdates =
		PhysicsControlAsset->Profiles.Find(ProfileName);

	if (!ControlAndModifierUpdates)
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"InvokeControlProfile - control profile %ls not found for %ls", 
				*ProfileName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		}
		return false;
	}

	ApplyControlAndModifierUpdates(*ControlAndModifierUpdates, ControlSetMask, BodyModifierSetMask);

	return true;
}

//======================================================================================================================
void UPhysicsControlComponent::ApplyControlAndModifierUpdates(
	const FPhysicsControlControlAndModifierUpdates& ControlAndModifierUpdates,
	const FName                                     ControlSetMask, 
	const FName                                     BodyModifierSetMask)
{
	// Danny TODO Note that when we check the masks, we have to do an O(N) check, because the names
	// are stored in an array. Would it be better to have them in a map, or at least sorted? Also
	// note that the default empty name will not incur the search cost (i.e. leaving the name blank
	// is better than saying "All").
	const TArray<FName>& ControlsInMask = GetControlNamesInSet(ControlSetMask);
	const TArray<FName>& BodyModifiersInMask = GetBodyModifierNamesInSet(BodyModifierSetMask);

	for (const FPhysicsControlNamedControlParameters& ControlParameters : ControlAndModifierUpdates.ControlUpdates)
	{
		TArray<FName> Names = ExpandName(ControlParameters.Name, NameRecords.ControlSets);
		for (FName Name : Names)
		{
			if (ControlsInMask.IsEmpty() || ControlsInMask.FindByKey(Name))
			{
				const FPhysicsControlSparseData& ControlData = ControlParameters.Data;
				if (FPhysicsControlRecord* ControlRecord = ControlRecords.Find(Name))
				{
					ControlRecord->PhysicsControl.ControlData.UpdateFromSparseData(ControlData);
				}
				else
				{
					if (bWarnAboutInvalidNames)
					{
						UE_LOGF(LogPhysicsControl, Warning,
							"ApplyControlAndModifierUpdates: Failed to find control with name %ls for %ls",
							*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
					}
				}
			}
		}
	}

	for (const FPhysicsControlNamedControlMultiplierParameters& ControlMultiplierParameters :
		ControlAndModifierUpdates.ControlMultiplierUpdates)
	{
		TArray<FName> Names = ExpandName(ControlMultiplierParameters.Name, NameRecords.ControlSets);
		for (FName Name : Names)
		{
			if (ControlsInMask.IsEmpty() || ControlsInMask.FindByKey(Name))
			{
				const FPhysicsControlSparseMultiplier& Multiplier = ControlMultiplierParameters.Data;
				if (FPhysicsControlRecord* ControlRecord = ControlRecords.Find(Name))
				{
					ControlRecord->PhysicsControl.ControlMultiplier.UpdateFromSparseData(Multiplier);
				}
				else
				{
					if (bWarnAboutInvalidNames)
					{
						UE_LOGF(LogPhysicsControl, Warning,
							"ApplyControlAndModifierUpdates: Failed to find control with name %ls for %ls",
							*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
					}
				}
			}
		}
	}

	for (const FPhysicsControlNamedModifierParameters& ModifierParameters : ControlAndModifierUpdates.ModifierUpdates)
	{
		TArray<FName> Names = ExpandName(ModifierParameters.Name, NameRecords.BodyModifierSets);
		for (FName Name : Names)
		{
			if (BodyModifiersInMask.IsEmpty() || BodyModifiersInMask.FindByKey(Name))
			{
				const FPhysicsControlModifierSparseData& ModifierData = ModifierParameters.Data;
				if (FPhysicsBodyModifierRecord* Record = BodyModifierRecords.Find(Name))
				{
					Record->BodyModifier.ModifierData.UpdateFromSparseData(ModifierData);
				}
				else
				{
					UE_LOGF(LogPhysicsControl, Warning,
						"ApplyControlAndModifierUpdates: Failed to find modifier with name %ls in %ls",
						*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
				}
			}
		}
	}
}

//======================================================================================================================
void UPhysicsControlComponent::AddControlToSet(
	FPhysicsControlNames& NewSet, 
	const FName           Control, 
	const FName           SetName)
{
	NameRecords.AddControl(Control, SetName);
	NewSet.Names = GetControlNamesInSet(SetName);
}

//======================================================================================================================
void UPhysicsControlComponent::AddControlsToSet(
	FPhysicsControlNames& NewSet, 
	const TArray<FName>&  Controls, 
	const FName           SetName)
{
	for (FName Control : Controls)
	{
		NameRecords.AddControl(Control, SetName);
	}
	NewSet.Names = GetControlNamesInSet(SetName);
}

//======================================================================================================================
const TArray<FName>& UPhysicsControlComponent::GetControlNamesInSet(const FName SetName) const
{
	return NameRecords.GetControlNamesInSet(SetName);
}

//======================================================================================================================
const TArray<FName>& UPhysicsControlComponent::GetAllBodyModifierNames() const
{
	return GetBodyModifierNamesInSet(TEXT("All"));
}

//======================================================================================================================
const TArray<FName>& UPhysicsControlComponent::GetBodyModifierNamesInSet(const FName SetName) const
{
	return NameRecords.GetBodyModifierNamesInSet(SetName);
}

//======================================================================================================================
void UPhysicsControlComponent::AddBodyModifierToSet(
	FPhysicsControlNames& NewSet, 
	const FName           BodyModifier, 
	const FName           SetName)
{
	NameRecords.AddBodyModifier(BodyModifier, SetName);
	NewSet.Names = GetBodyModifierNamesInSet(SetName);
}


//======================================================================================================================
void UPhysicsControlComponent::AddBodyModifiersToSet(
	FPhysicsControlNames& NewSet, 
	const TArray<FName>&  InBodyModifiers, 
	const FName           SetName)
{
	for (FName BodyModifier : InBodyModifiers)
	{
		NameRecords.AddBodyModifier(BodyModifier, SetName);
	}
	NewSet.Names = GetBodyModifierNamesInSet(SetName);
}

//======================================================================================================================
TArray<FName> UPhysicsControlComponent::GetSetsContainingControl(const FName Control) const
{
	TArray<FName> Result;
	for (const TPair<FName, TArray<FName>>& ControlSetPair : NameRecords.ControlSets)
	{
		for (const FName& ControlName : ControlSetPair.Value)
		{
			if (ControlName == Control)
			{
				Result.Add(ControlSetPair.Key);
			}
		}
	}
	return Result;
}

//======================================================================================================================
TArray<FName> UPhysicsControlComponent::GetSetsContainingBodyModifier(const FName BodyModifier) const
{
	TArray<FName> Result;
	for (const TPair<FName, TArray<FName>>& BodyModifierSetPair : NameRecords.BodyModifierSets)
	{
		for (const FName& BodyModifierName : BodyModifierSetPair.Value)
		{
			if (BodyModifierName == BodyModifier)
			{
				Result.Add(BodyModifierSetPair.Key);
			}
		}
	}
	return Result;
}

//======================================================================================================================
TArray<FTransform> UPhysicsControlComponent::GetCachedBoneTransforms(
	const USkeletalMeshComponent* SkeletalMeshComponent, 
	const TArray<FName>&          BoneNames)
{
	TArray<FTransform> Result;
	Result.Reserve(BoneNames.Num());
	UE::PhysicsControl::FBoneData BoneData;
	const UE::PhysicsControl::FPhysicsControlPoseData* PoseData;

	for (const FName& BoneName : BoneNames)
	{
		if (GetBoneData(BoneData, PoseData, SkeletalMeshComponent, BoneName))
		{
			FTransform BoneTransform(BoneData.CurrentTM.GetRotation(), BoneData.CurrentTM.GetTranslation());
			Result.Add(BoneTransform);
		}
		else
		{
			if (bWarnAboutInvalidNames)
			{
				UE_LOGF(LogPhysicsControl, Warning,
					"GetCachedBoneTransforms - unable to get bone %ls data for %ls",
					*BoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
			}
			Result.Add(FTransform::Identity);
		}
	}
	return Result;
}

//======================================================================================================================
TArray<FVector> UPhysicsControlComponent::GetCachedBonePositions(
	const USkeletalMeshComponent* SkeletalMeshComponent,
	const TArray<FName>&          BoneNames)
{
	TArray<FVector> Result;
	Result.Reserve(BoneNames.Num());
	UE::PhysicsControl::FBoneData BoneData;
	const UE::PhysicsControl::FPhysicsControlPoseData* PoseData;

	for (const FName& BoneName : BoneNames)
	{
		if (GetBoneData(BoneData, PoseData, SkeletalMeshComponent, BoneName))
		{
			Result.Add(BoneData.CurrentTM.GetTranslation());
		}
		else
		{
			if (bWarnAboutInvalidNames)
			{
				UE_LOGF(LogPhysicsControl, Warning,
					"GetCachedBonePositions - unable to get bone %ls data for %ls",
					*BoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
			}
			Result.Add(FVector::ZeroVector);
		}
	}
	return Result;
}

//======================================================================================================================
TArray<FRotator> UPhysicsControlComponent::GetCachedBoneOrientations(
	const USkeletalMeshComponent* SkeletalMeshComponent,
	const TArray<FName>&          BoneNames)
{
	TArray<FRotator> Result;
	Result.Reserve(BoneNames.Num());
	UE::PhysicsControl::FBoneData BoneData;
	const UE::PhysicsControl::FPhysicsControlPoseData* PoseData;

	for (const FName& BoneName : BoneNames)
	{
		if (GetBoneData(BoneData, PoseData, SkeletalMeshComponent, BoneName))
		{
			Result.Add(BoneData.CurrentTM.GetRotation().Rotator());
		}
		else
		{
			if (bWarnAboutInvalidNames)
			{
				UE_LOGF(LogPhysicsControl, Warning, 
					"GetCachedBoneOrientations - unable to get bone %ls data for %ls", 
					*BoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
			}
			Result.Add(FRotator::ZeroRotator);
		}
	}
	return Result;
}

//======================================================================================================================
FTransform UPhysicsControlComponent::GetCachedBoneTransform(
	const USkeletalMeshComponent* SkeletalMeshComponent,
	const FName                   BoneName)
{
	UE::PhysicsControl::FBoneData BoneData;
	const UE::PhysicsControl::FPhysicsControlPoseData* PoseData;
	if (GetBoneData(BoneData, PoseData, SkeletalMeshComponent, BoneName))
	{
		return FTransform(BoneData.CurrentTM.GetRotation(), BoneData.CurrentTM.GetTranslation());
	}
	if (bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"GetCachedBoneTransform - invalid bone name %ls for %ls",
			*BoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return FTransform();
}

//======================================================================================================================
FVector UPhysicsControlComponent::GetCachedBonePosition(
	const USkeletalMeshComponent* SkeletalMeshComponent,
	const FName                   BoneName)
{
	UE::PhysicsControl::FBoneData BoneData;
	const UE::PhysicsControl::FPhysicsControlPoseData* PoseData;
	if (GetBoneData(BoneData, PoseData, SkeletalMeshComponent, BoneName))
	{
		return BoneData.CurrentTM.GetTranslation();
	}
	if (bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"GetCachedBonePosition - invalid bone name %ls for %ls",
			*BoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return FVector::ZeroVector;
}

//======================================================================================================================
FRotator UPhysicsControlComponent::GetCachedBoneOrientation(
	const USkeletalMeshComponent* SkeletalMeshComponent,
	const FName                   BoneName)
{
	UE::PhysicsControl::FBoneData BoneData;
	const UE::PhysicsControl::FPhysicsControlPoseData* PoseData;
	if (GetBoneData(BoneData, PoseData, SkeletalMeshComponent, BoneName))
	{
		return BoneData.CurrentTM.GetRotation().Rotator();
	}
	if (bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"GetCachedBoneOrientation - invalid bone name %ls for %ls",
			*BoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return FRotator::ZeroRotator;
}

//======================================================================================================================
bool UPhysicsControlComponent::SetCachedBoneData(
	const USkeletalMeshComponent* SkeletalMeshComponent,
	const FName                   BoneName,
	const FTransform&             TM)
{
	UE::PhysicsControl::FBoneData* BoneData;
	if (GetModifiableBoneData(BoneData, SkeletalMeshComponent, BoneName))
	{
		BoneData->CurrentTM = TM;
		return true;
	}
	if (bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetCachedBoneData - invalid bone name %ls for %ls",
			*BoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return false;
}

//======================================================================================================================
void UPhysicsControlComponent::SetCachedBoneVelocitiesToZero()
{
	for (TPair<TWeakObjectPtr<USkeletalMeshComponent>, UE::PhysicsControl::FPhysicsControlPoseData>&
		CachedSkeletalMeshDataPair : CachedPoseDatas)
	{
		UE::PhysicsControl::FPhysicsControlPoseData& CachedSkeletalMeshData = CachedSkeletalMeshDataPair.Value;
		CachedSkeletalMeshData.BoneDatas.Reset(); // Doesn't change memory allocations
	}
}

//======================================================================================================================
void UPhysicsControlComponent::ResetBodyModifierToCachedBoneTransform(
	const FName                        Name,
	const EResetToCachedTargetBehavior Behavior,
	const bool                         bApplyToModifiersWithName,
	const bool                         bApplyToSetsWithName)
{
	if (bApplyToModifiersWithName)
	{
		FPhysicsBodyModifierRecord* Record = FindBodyModifierRecord(Name);
		if (Record)
		{
			if (Behavior == EResetToCachedTargetBehavior::ResetImmediately)
			{
				ResetToCachedTarget(*Record);
			}
			else
			{
				Record->bResetToCachedTarget = true;
			}
		}
	}
	if (bApplyToSetsWithName)
	{
		ResetBodyModifiersInSetToCachedBoneTransforms(Name, Behavior);
	}
}

//======================================================================================================================
void UPhysicsControlComponent::ResetBodyModifiersToCachedBoneTransforms(
	const TArray<FName>&               Names,
	const EResetToCachedTargetBehavior Behavior,
	const bool                         bApplyToModifiersWithName,
	const bool                         bApplyToSetsWithName)
{
	for (FName Name : Names)
	{
		ResetBodyModifierToCachedBoneTransform(Name, Behavior, bApplyToModifiersWithName, bApplyToSetsWithName);
	}
}

//======================================================================================================================
void UPhysicsControlComponent::ResetBodyModifiersInSetToCachedBoneTransforms(
	const FName                        SetName,
	const EResetToCachedTargetBehavior Behavior)
{
	ResetBodyModifiersToCachedBoneTransforms(GetBodyModifierNamesInSet(SetName), Behavior, true, false);
}

//======================================================================================================================
bool UPhysicsControlComponent::GetControlExists(const FName Name) const
{
	return FindControlRecord(Name) != nullptr;
}

//======================================================================================================================
bool UPhysicsControlComponent::GetBodyModifierExists(const FName Name) const
{
	return FindBodyModifierRecord(Name) != nullptr;
}

#define PCC_LOG_LEVEL Display

static FString GetComponentName(TWeakObjectPtr<UPrimitiveComponent> Component)
{
	if (Component.Get())
	{
		return Component.Get()->GetName();
	}
	return TEXT("None");
}

//======================================================================================================================
void UPhysicsControlComponent::LogControls() const
{
#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING

	UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "Controls for %ls:",
		GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	for (const TPair<FName, FPhysicsControlRecord>& RecordPair : ControlRecords)
	{
		UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "  %ls:", *RecordPair.Key.ToString());
		const FPhysicsControlRecord& Record = RecordPair.Value;
		UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "    Parent: %ls (%ls) -> Child: %ls (%ls)",
			*GetComponentName(Record.ParentComponent),
			*Record.PhysicsControl.ParentBoneName.ToString(), 
			*GetComponentName(Record.ChildComponent),
			*Record.PhysicsControl.ChildBoneName.ToString());
		UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "    Enabled %d",
			Record.PhysicsControl.IsEnabled());
		UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "    Linear: Strength %f DampingRatio %f ExtraDamping %f",
			Record.PhysicsControl.ControlData.LinearStrength,
			Record.PhysicsControl.ControlData.LinearDampingRatio,
			Record.PhysicsControl.ControlData.LinearExtraDamping);
		UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "    Angular: Strength %f DampingRatio %f ExtraDamping %f",
			Record.PhysicsControl.ControlData.AngularStrength,
			Record.PhysicsControl.ControlData.AngularDampingRatio,
			Record.PhysicsControl.ControlData.AngularExtraDamping);
	}

	UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "Control sets:");
	for (const TMap<FName, TArray<FName>>::ElementType& Pair : NameRecords.ControlSets)
	{
		UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "  %ls:", *Pair.Key.ToString());
		const TArray<FName>& Names = Pair.Value;
		for (FName Name : Names)
		{
			UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "    %ls", *Name.ToString());
		}
	}
#endif
}

//======================================================================================================================
void UPhysicsControlComponent::LogBodyModifiers() const
{
#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING

	UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "Body Modifiers for %ls:",
		GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	for (const TPair<FName, FPhysicsBodyModifierRecord>& RecordPair : BodyModifierRecords)
	{
		UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "  %ls:", *RecordPair.Key.ToString());
		const FPhysicsBodyModifierRecord& Record = RecordPair.Value;
		UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "    Body: %ls (%ls)",
			*GetComponentName(Record.Component),
			*Record.BodyModifier.BoneName.ToString());
		UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "    Movement: %ls GravityMultiplier %f",
			*GetPhysicsMovementTypeName(Record.BodyModifier.ModifierData.MovementType).ToString(),
			Record.BodyModifier.ModifierData.GravityMultiplier);
	}

	UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "Body Modifier sets:");
	for (const TMap<FName, TArray<FName>>::ElementType& Pair : NameRecords.BodyModifierSets)
	{
		UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "  %ls:", *Pair.Key.ToString());
		const TArray<FName>& Names = Pair.Value;
		for (FName Name : Names)
		{
			UE_LOGF(LogPhysicsControl, PCC_LOG_LEVEL, "    %ls", *Name.ToString());
		}
	}
#endif
}

#undef PCC_LOG_LEVEL

//======================================================================================================================
void UPhysicsControlComponent::LogControlsAndBodyModifiers() const
{
	LogControls();
	LogBodyModifiers();
}

//======================================================================================================================
bool UPhysicsControlComponent::ShouldCreatePhysicsState() const
{
	// This is needed to ensure we get the destroy call
	return true;
}

//======================================================================================================================
void UPhysicsControlComponent::OnCreatePhysicsState()
{
	Super::OnCreatePhysicsState();
}

//======================================================================================================================
void UPhysicsControlComponent::DestroyPhysicsState()
{
	for (TPair<FName, FPhysicsControlRecord>& ControlRecordPair : ControlRecords)
	{
		DestroyControl(ControlRecordPair.Key, EDestroyBehavior::KeepRecord);
	}
	ControlRecords.Empty();

	for (TPair<FName, FPhysicsBodyModifierRecord>& BodyModifierPair : BodyModifierRecords)
	{
		DestroyBodyModifier(BodyModifierPair.Key, EDestroyBehavior::KeepRecord);
	}
	BodyModifierRecords.Empty();
}

//======================================================================================================================
void UPhysicsControlComponent::OnDestroyPhysicsState()
{
	DestroyPhysicsState();
	Super::OnDestroyPhysicsState();
}

//======================================================================================================================
void UPhysicsControlComponent::OnRegister()
{
	Super::OnRegister();
#if WITH_EDITOR

	if (SpriteComponent)
	{
		SpriteComponent->SetSprite(LoadObject<UTexture2D>(NULL, TEXT("/Engine/EditorResources/S_KBSJoint.S_KBSJoint")));
		SpriteComponent->SpriteInfo.Category = TEXT("Physics");
		SpriteComponent->SpriteInfo.DisplayName = NSLOCTEXT("SpriteCategory", "Physics", "Physics");
	}
#endif
}

#if WITH_EDITOR
//======================================================================================================================
void UPhysicsControlComponent::DebugDraw(FPrimitiveDrawInterface* PDI) const
{
	// Draw gizmos
	if (bShowDebugVisualization && VisualizationSizeScale > 0)
	{
		for (const TPair<FName, FPhysicsControlRecord>& ControlRecordPair : ControlRecords)
		{
			const FName Name = ControlRecordPair.Key;
			const FPhysicsControlRecord& Record = ControlRecordPair.Value;
			DebugDrawControl(PDI, Record, Name);
		} 
	}

	// Detailed controls - if there's a filter
	if (!DebugControlDetailFilter.IsEmpty())
	{
		for (const TPair<FName, FPhysicsControlRecord>& ControlRecordPair : ControlRecords)
		{
			const FName Name = ControlRecordPair.Key;

			if (Name.ToString().Contains(DebugControlDetailFilter))
			{
				const FPhysicsControlRecord& Record = ControlRecordPair.Value;

				const FString ParentComponentName = Record.ParentComponent.IsValid() ?
					Record.ParentComponent->GetName() : TEXT("NoParent");
				const FString ChildComponentName = Record.ChildComponent.IsValid() ?
					Record.ChildComponent->GetName() : TEXT("NoChild");

				const FString Text = FString::Printf(
					TEXT("%s: Parent %s (%s) Child %s (%s): Linear strength %f Angular strength %f"),
					*Name.ToString(),
					*ParentComponentName,
					*Record.PhysicsControl.ParentBoneName.ToString(),
					*ChildComponentName,
					*Record.PhysicsControl.ChildBoneName.ToString(),
					Record.PhysicsControl.ControlData.LinearStrength,
					Record.PhysicsControl.ControlData.AngularStrength);

				GEngine->AddOnScreenDebugMessage(
					-1, 0.0f,
					Record.PhysicsControl.IsEnabled() ? FColor::Green : FColor::Red, Text);
			}
		}
	}

	// Summary of control list
	if (bShowDebugControlList)
	{
		FString AllNames;
		for (const TPair<FName, FPhysicsControlRecord>& ControlRecordPair : ControlRecords)
		{
			const FName Name = ControlRecordPair.Key;
			AllNames += Name.ToString() + TEXT(" ");
			if (AllNames.Len() > 256)
			{
				GEngine->AddOnScreenDebugMessage(-1, 0.0f, FColor::White, *AllNames);
				AllNames.Reset();
			}
		}
		GEngine->AddOnScreenDebugMessage(
			-1, 0.0f, FColor::White, FString::Printf(TEXT("%d Controls: %s"), ControlRecords.Num(), *AllNames));
	}

	// Detailed body modifiers - if there's a filter
	if (!DebugBodyModifierDetailFilter.IsEmpty())
	{
		for (const TPair<FName, FPhysicsBodyModifierRecord>& BodyModifierPair : BodyModifierRecords)
		{
			const FName Name = BodyModifierPair.Key;

			if (Name.ToString().Contains(DebugBodyModifierDetailFilter))
			{
				const FPhysicsBodyModifierRecord& Record = BodyModifierPair.Value;

				FString ComponentName = Record.Component.IsValid() ? Record.Component->GetName() : TEXT("None");

				FString Text = FString::Printf(
					TEXT("%s: %s: %s %s GravityMultiplier %f BlendWeight %f"),
					*Name.ToString(),
					*ComponentName,
					*UEnum::GetValueAsString(Record.BodyModifier.ModifierData.MovementType),
					*UEnum::GetValueAsString(Record.BodyModifier.ModifierData.CollisionType),
					Record.BodyModifier.ModifierData.GravityMultiplier,
					Record.BodyModifier.ModifierData.PhysicsBlendWeight);

				GEngine->AddOnScreenDebugMessage(-1, 0.0f, 
					Record.BodyModifier.ModifierData.MovementType == EPhysicsMovementType::Simulated ?
					FColor::Green : FColor::Red, Text);
			}
		}
	}

	// Summary of control list
	if (bShowDebugBodyModifierList)
	{
		FString AllNames;

		for (const TPair<FName, FPhysicsBodyModifierRecord>& BodyModifierPair : BodyModifierRecords)
		{
			const FName Name = BodyModifierPair.Key;
			AllNames += Name.ToString() + TEXT(" ");
			if (AllNames.Len() > 256)
			{
				GEngine->AddOnScreenDebugMessage(-1, 0.0f, FColor::White, *AllNames);
				AllNames.Reset();
			}
		}
		GEngine->AddOnScreenDebugMessage(
			-1, 0.0f, FColor::White,
			FString::Printf(TEXT("%d Body modifiers: %s"), BodyModifierRecords.Num(), *AllNames));

	}
}

//======================================================================================================================
void UPhysicsControlComponent::DebugDrawControl(
	FPrimitiveDrawInterface* PDI, const FPhysicsControlRecord& Record, FName ControlName) const
{
	const float GizmoWidthScale = 0.02f * VisualizationSizeScale;
	const FColor CurrentToTargetColor(255, 0, 0);
	const FColor TargetColor(0, 255, 0);
	const FColor CurrentColor(0, 0, 255);

	const FConstraintInstance* ConstraintInstance = Record.ConstraintInstance.Get();

	const bool bHaveLinear = Record.PhysicsControl.ControlData.LinearStrength > 0;
	const bool bHaveAngular = Record.PhysicsControl.ControlData.AngularStrength > 0;

	if (Record.PhysicsControl.IsEnabled() && ConstraintInstance)
	{
		FBodyInstance* ChildBodyInstance = UE::PhysicsControl::GetBodyInstance(
			Record.ChildComponent.Get(), Record.PhysicsControl.ChildBoneName);
		if (!ChildBodyInstance)
		{
			return;
		}
		FTransform ChildBodyTM = ChildBodyInstance->GetUnrealWorldTransform();

		FBodyInstance* ParentBodyInstance = UE::PhysicsControl::GetBodyInstance(
			Record.ParentComponent.Get(), Record.PhysicsControl.ParentBoneName);
		const FTransform ParentBodyTM = ParentBodyInstance ? ParentBodyInstance->GetUnrealWorldTransform() : FTransform();

		FTransform TargetTM, SkeletalTargetTM;
		FVector TargetVelocity;
		FVector TargetAngularVelocity;
		// Note that we want velocities, but there is a risk that they will be invalid, depending on the update times
		CalculateControlTargetData(TargetTM, SkeletalTargetTM, TargetVelocity, TargetAngularVelocity, Record, true);

		// WorldChildFrameTM is the world-space transform of the child (driven) constraint frame
		const FTransform WorldChildFrameTM = ConstraintInstance->GetRefFrame(EConstraintFrame::Frame1) * ChildBodyTM;

		// WorldParentFrameTM is the world-space transform of the parent constraint frame
		const FTransform WorldParentFrameTM = ConstraintInstance->GetRefFrame(EConstraintFrame::Frame2) * ParentBodyTM;

		const FTransform WorldCurrentTM = WorldChildFrameTM;

		FTransform WorldTargetTM = TargetTM * WorldParentFrameTM;
		if (!bHaveLinear)
		{
			WorldTargetTM.SetTranslation(WorldCurrentTM.GetTranslation());
		}
		if (!bHaveAngular)
		{
			WorldTargetTM.SetRotation(WorldCurrentTM.GetRotation());
		}

		FVector WorldTargetVelocity = WorldParentFrameTM.GetRotation() * TargetVelocity;
		FVector WorldTargetAngularVelocity = WorldParentFrameTM.GetRotation() * TargetAngularVelocity;

		// Indicate the velocities by predicting the TargetTM
		FTransform PredictedTargetTM = WorldTargetTM;
		PredictedTargetTM.AddToTranslation(WorldTargetVelocity * VelocityPredictionTime);

		// Draw the target and current positions/orientations
		if (bHaveAngular)
		{
			FQuat AngularVelocityQ = FQuat::MakeFromRotationVector(WorldTargetAngularVelocity * VelocityPredictionTime);
			PredictedTargetTM.SetRotation(AngularVelocityQ * WorldTargetTM.GetRotation());

			DrawCoordinateSystem(
				PDI, WorldCurrentTM.GetTranslation(), WorldCurrentTM.Rotator(), 
				VisualizationSizeScale, SDPG_Foreground, 1.0f * GizmoWidthScale);
			DrawCoordinateSystem(
				PDI, WorldTargetTM.GetTranslation(), WorldTargetTM.Rotator(),
				VisualizationSizeScale, SDPG_Foreground, 4.0f * GizmoWidthScale);
			if (VelocityPredictionTime != 0)
			{
				DrawCoordinateSystem(
					PDI, PredictedTargetTM.GetTranslation(), PredictedTargetTM.Rotator(),
					VisualizationSizeScale * 0.5f, SDPG_Foreground, 4.0f * GizmoWidthScale);
			}
		}
		else
		{
			DrawWireSphere(
				PDI, WorldCurrentTM, CurrentColor, 
				VisualizationSizeScale, 8, SDPG_Foreground, 1.0f * GizmoWidthScale);
			DrawWireSphere(
				PDI, WorldTargetTM, TargetColor, 
				VisualizationSizeScale, 8, SDPG_Foreground, 3.0f * GizmoWidthScale);
			if (VelocityPredictionTime != 0)
			{
				DrawWireSphere(
					PDI, PredictedTargetTM, TargetColor, 
					VisualizationSizeScale * 0.5f, 8, SDPG_Foreground, 3.0f * GizmoWidthScale);
			}
		}


		if (VelocityPredictionTime != 0)
		{
			PDI->DrawLine(
				WorldTargetTM.GetTranslation(), 
				WorldTargetTM.GetTranslation() + WorldTargetVelocity * VelocityPredictionTime, 
				TargetColor, SDPG_Foreground);
		}

		// Connect current to target
		DrawDashedLine(
			PDI,
			WorldTargetTM.GetTranslation(), WorldCurrentTM.GetTranslation(), 
			CurrentToTargetColor, VisualizationSizeScale * 0.2f, SDPG_Foreground);
	}
}

#endif

