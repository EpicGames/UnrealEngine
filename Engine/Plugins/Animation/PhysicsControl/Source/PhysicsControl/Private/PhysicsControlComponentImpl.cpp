// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlComponent.h"
#include "PhysicsControlLog.h"
#include "PhysicsControlHelpers.h"

#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"

#include "Physics/Experimental/PhysScene_Chaos.h"

#include "Components/SkeletalMeshComponent.h"

#include "Engine/SkeletalMesh.h"
#include "Engine/World.h"

//======================================================================================================================
// This file contains the non-public member functions of UPhysicsControlComponent
//======================================================================================================================

// Overrides the per-constraint LimitViolationResponse for all controls. Useful for comparing
// responses without having to change each constraint individually.
// -1 = no override (use per-constraint setting)
//  0 = None
//  1 = ClampLinear
//  2 = ClampExact
//  3 = WidenLimits
static TAutoConsoleVariable<int32> CVarPhysicsControlLimitViolationResponseOverride(
	TEXT("p.PhysicsControl.LimitViolationResponseOverride"),
	-1,
	TEXT("Overrides the LimitViolationResponse on all physics controls: -1=off, 0=None, 1=ClampLinear, 2=ClampExact, 3=WidenLimits"),
	ECVF_Default);

//======================================================================================================================
// Helper to get a valid skeletal mesh component pointer from a record
static USkeletalMeshComponent* GetValidSkeletalMeshComponentFromControlParent(
	const FPhysicsControlRecord& Record)
{
	return Cast<USkeletalMeshComponent>(Record.ParentComponent.Get());
}

//======================================================================================================================
// Helper to get a valid skeletal mesh component pointer from a record
static USkeletalMeshComponent* GetValidSkeletalMeshComponentFromControlChild(
	const FPhysicsControlRecord& Record)
{
	return Cast<USkeletalMeshComponent>(Record.ChildComponent.Get());
}

//======================================================================================================================
// Helper to get a valid skeletal mesh component pointer from a record
static USkeletalMeshComponent* GetValidSkeletalMeshComponentFromBodyModifier(
	const FPhysicsBodyModifierRecord& PhysicsBodyModifier)
{
	return Cast<USkeletalMeshComponent>(PhysicsBodyModifier.Component.Get());
}

//======================================================================================================================
bool UPhysicsControlComponent::GetBoneData(
	UE::PhysicsControl::FBoneData&                      OutBoneData,
	const UE::PhysicsControl::FPhysicsControlPoseData*& OutPoseData,
	const USkeletalMeshComponent*                       InSkeletalMeshComponent,
	const FName                                         InBoneName) const
{
	if (!ensure(InSkeletalMeshComponent))
	{
		return false;
	}
	OutPoseData = nullptr;
	const USkeletalMesh* SkeletalMesh = InSkeletalMeshComponent->GetSkeletalMeshAsset();
	if (!SkeletalMesh)
	{
		return false;
	}
	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	const int32 BoneIndex = RefSkeleton.FindBoneIndex(InBoneName);

	if (BoneIndex == INDEX_NONE)
	{
		UE_LOGF(LogPhysicsControl, Warning, "Failed to find BoneIndex for %ls in %ls", 
			*InBoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return false;
	}

	const UE::PhysicsControl::FPhysicsControlPoseData* CachedSkeletalMeshData =
		CachedPoseDatas.Find(InSkeletalMeshComponent);
	if (CachedSkeletalMeshData && CachedSkeletalMeshData->ReferenceCount > 0 &&
		!CachedSkeletalMeshData->BoneDatas.IsEmpty())
	{
		if (BoneIndex < CachedSkeletalMeshData->BoneDatas.Num())
		{
			OutBoneData = CachedSkeletalMeshData->BoneDatas[BoneIndex];
			OutPoseData = CachedSkeletalMeshData;
			return true;
		}
		UE_LOGF(LogPhysicsControl, Warning, 
			"BoneIndex is out of range in %ls", GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

	}
	UE_LOGF(LogPhysicsControl, Warning, "Failed to find bone data for %ls in %ls",
		*InBoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	return false;
}

//======================================================================================================================
bool UPhysicsControlComponent::GetModifiableBoneData(
	UE::PhysicsControl::FBoneData*& OutBoneData,
	const USkeletalMeshComponent*   InSkeletalMeshComponent,
	const FName                     InBoneName)
{
	if (!ensure(InSkeletalMeshComponent))
	{
		return false;
	}
	const USkeletalMesh* SkeletalMesh = InSkeletalMeshComponent->GetSkeletalMeshAsset();
	if (!SkeletalMesh)
	{
		return false;
	}
	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	const int32 BoneIndex = RefSkeleton.FindBoneIndex(InBoneName);

	if (BoneIndex == INDEX_NONE)
	{
		UE_LOGF(LogPhysicsControl, Warning, "Failed to find BoneIndex for %ls in %ls",
			*InBoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return false;
	}

	UE::PhysicsControl::FPhysicsControlPoseData* CachedSkeletalMeshData =
		CachedPoseDatas.Find(InSkeletalMeshComponent);
	if (CachedSkeletalMeshData &&
		CachedSkeletalMeshData->ReferenceCount > 0 &&
		!CachedSkeletalMeshData->BoneDatas.IsEmpty())
	{
		if (BoneIndex < CachedSkeletalMeshData->BoneDatas.Num())
		{
			OutBoneData = &CachedSkeletalMeshData->BoneDatas[BoneIndex];
			return true;
		}
		UE_LOGF(LogPhysicsControl, Warning, "BoneIndex is out of range in %ls", 
			GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));

	}
	UE_LOGF(LogPhysicsControl, Warning, "Failed to find bone data for %ls in %ls",
		*InBoneName.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	return false;
}

//======================================================================================================================
FPhysicsControlRecord* UPhysicsControlComponent::FindControlRecord(const FName Name)
{
	if (FPhysicsControlRecord* Record = ControlRecords.Find(Name))
	{
		return Record;
	}
	return nullptr;
}

//======================================================================================================================
const FPhysicsControlRecord* UPhysicsControlComponent::FindControlRecord(const FName Name) const
{
	if (const FPhysicsControlRecord* Record = ControlRecords.Find(Name))
	{
		return Record;
	}
	return nullptr;
}

//======================================================================================================================
FPhysicsControl* UPhysicsControlComponent::FindControl(const FName Name)
{
	if (FPhysicsControlRecord* Record = FindControlRecord(Name))
	{
		return &Record->PhysicsControl;
	}
	return nullptr;
}

//======================================================================================================================
const FPhysicsControl* UPhysicsControlComponent::FindControl(const FName Name) const
{
	if (const FPhysicsControlRecord* Record = FindControlRecord(Name))
	{
		return &Record->PhysicsControl;
	}
	return nullptr;
}

//======================================================================================================================
void UPhysicsControlComponent::UpdateCachedSkeletalBoneData(float DeltaTime)
{
	CurrentUpdateCounter.Increment();

	for (TPair<TWeakObjectPtr<USkeletalMeshComponent>, UE::PhysicsControl::FPhysicsControlPoseData>&
		CachedSkeletalMeshDataPair : CachedPoseDatas)
	{
		UE::PhysicsControl::FPhysicsControlPoseData& CachedSkeletalMeshData = CachedSkeletalMeshDataPair.Value;
		if (!CachedSkeletalMeshData.ReferenceCount)
		{
			continue;
		}

		TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent = CachedSkeletalMeshDataPair.Key;
		if (USkeletalMeshComponent* SkeletalMesh = SkeletalMeshComponent.Get())
		{
			// Ensure any in-flight parallel animation evaluation or physics blend task has
			// completed before we read the editable component space transforms. Without this,
			// SwapEvaluationContextBuffers() (called during parallel work) can temporarily replace
			// the buffer that GetEditableComponentSpaceTransforms() returns, causing us to read
			// stale or invalid data. This can occur when tick prerequisites push this component
			// into a later tick group (e.g. PostPhysics), overlapping with the skeletal mesh's 
			// EndPhysicsTickFunction.
			SkeletalMesh->HandleExistingParallelEvaluationTask(true, true);
			CachedSkeletalMeshData.Update(
				SkeletalMesh, DeltaTime, TeleportDistanceThreshold, TeleportRotationThreshold);
		}
		else
		{
			CachedSkeletalMeshData.Reset();
		}
	}
}

//======================================================================================================================
void UPhysicsControlComponent::ResetControls(bool bKeepControlRecords)
{
	for (TPair<FName, FPhysicsControlRecord>& PhysicsControlRecordPair : ControlRecords)
	{
		FPhysicsControlRecord& Record = PhysicsControlRecordPair.Value;
		Record.ResetConstraint();
	}

	if (!bKeepControlRecords)
	{
		ControlRecords.Empty();
	}
}

//======================================================================================================================
void UPhysicsControlComponent::ApplyKinematicTarget(const FPhysicsBodyModifierRecord& Record) const
{
	// Seems like static and skeletal meshes need to be handled differently
	if (USkeletalMeshComponent* SkeletalMeshComponent = GetValidSkeletalMeshComponentFromBodyModifier(Record))
	{
		FBodyInstance* BodyInstance = UE::PhysicsControl::GetBodyInstance(
			Record.Component.Get(), Record.BodyModifier.BoneName);
		if (!BodyInstance)
		{
			return;
		}

		const FTransform CurrentBodyTM = BodyInstance->GetUnrealWorldTransform(); // Preserve scale
		const FTransform MeshTM = SkeletalMeshComponent->GetComponentTransform();
		FTransform KinematicTargetWS = CurrentBodyTM;
		FTransform KinematicTargetOffset = Record.KinematicTarget.ToTransform();

		switch (Record.BodyModifier.ModifierData.KinematicTargetSpace)
		{
			case EPhysicsControlKinematicTargetSpace::World:
				KinematicTargetWS = KinematicTargetOffset;
				break;
			case EPhysicsControlKinematicTargetSpace::Component:
				KinematicTargetWS = KinematicTargetOffset * MeshTM;
				break;
			case EPhysicsControlKinematicTargetSpace::IgnoreTarget:
				break;
			default:
			{
				// All the other options are relative to a bone, so the first task is to get that
				UE::PhysicsControl::FBoneData BoneData;
				const UE::PhysicsControl::FPhysicsControlPoseData* PoseData;
				if (GetBoneData(BoneData, PoseData, SkeletalMeshComponent, Record.BodyModifier.BoneName))
				{
					FTransform BoneTM = BoneData.CurrentTM.ToTransform(); // Bone in WS
					switch (Record.BodyModifier.ModifierData.KinematicTargetSpace)
					{
						case EPhysicsControlKinematicTargetSpace::OffsetInBoneSpace:
							KinematicTargetWS = KinematicTargetOffset * BoneTM;
							break;
						case EPhysicsControlKinematicTargetSpace::OffsetInWorldSpace:
						{
							KinematicTargetWS = BoneTM;
							// Note that we don't want to rotate the translation by the target - we apply it
							// individually to orientation and position.
							KinematicTargetWS.AddToTranslation(KinematicTargetOffset.GetTranslation());
							KinematicTargetWS.SetRotation(
								KinematicTargetOffset.GetRotation() * KinematicTargetWS.GetRotation());
							break;
						}
						case EPhysicsControlKinematicTargetSpace::OffsetInComponentSpace:
						{
							FTransform KinematicTargetCS = BoneTM.GetRelativeTransform(MeshTM);
							KinematicTargetCS.AddToTranslation(KinematicTargetOffset.GetTranslation());
							KinematicTargetCS.SetRotation(
								KinematicTargetOffset.GetRotation() * KinematicTargetCS.GetRotation());
							KinematicTargetWS = KinematicTargetCS * MeshTM;
							break;
						}
						default:
							break;
					}
				}
				break;
			}
		}

		ETeleportType TT = UE::PhysicsControl::DetectTeleport(
			CurrentBodyTM, KinematicTargetWS, TeleportDistanceThreshold, TeleportRotationThreshold) ? 
			ETeleportType::ResetPhysics : ETeleportType::None;
		BodyInstance->SetBodyTransform(KinematicTargetWS, TT);
	}
	else
	{
		UPrimitiveComponent* Component = Record.Component.Get();
		if (!Component)
		{
			return;
		}

		const FTransform CurrentBodyTM = Component->GetComponentToWorld();
		FTransform KinematicTargetWS = CurrentBodyTM;

		FTransform KinematicTargetOffset = Record.KinematicTarget.ToTransform();

		switch (Record.BodyModifier.ModifierData.KinematicTargetSpace)
		{
		case EPhysicsControlKinematicTargetSpace::World:
		case EPhysicsControlKinematicTargetSpace::OffsetInWorldSpace:
		case EPhysicsControlKinematicTargetSpace::OffsetInBoneSpace:
		default:
			KinematicTargetWS = KinematicTargetOffset;
			break;
		case EPhysicsControlKinematicTargetSpace::Component:
		case EPhysicsControlKinematicTargetSpace::OffsetInComponentSpace:
			KinematicTargetWS = KinematicTargetOffset * this->GetComponentTransform();
			break;
		case EPhysicsControlKinematicTargetSpace::IgnoreTarget:
			KinematicTargetWS = this->GetComponentTransform();
			break;
		}

		const ETeleportType TT = UE::PhysicsControl::DetectTeleport(
			CurrentBodyTM, KinematicTargetWS, TeleportDistanceThreshold, TeleportRotationThreshold)
			? ETeleportType::ResetPhysics : ETeleportType::None;

		// Note that calling BodyInstance->SetBodyTransform moves the physics, but not the mesh
		Component->SetWorldLocationAndRotation(
			KinematicTargetWS.GetTranslation(), KinematicTargetWS.GetRotation(), false, nullptr, TT);
	}
}

//======================================================================================================================
void UPhysicsControlComponent::ResetToCachedTarget(const FPhysicsBodyModifierRecord& Record) const
{
	FBodyInstance* BodyInstance = UE::PhysicsControl::GetBodyInstance(
		Record.Component.Get(), Record.BodyModifier.BoneName);
	if (!BodyInstance)
	{
		return;
	}

	if (USkeletalMeshComponent* SkeletalMeshComponent = GetValidSkeletalMeshComponentFromBodyModifier(Record))
	{
		UE::PhysicsControl::FBoneData BoneData;
		const UE::PhysicsControl::FPhysicsControlPoseData* PoseData;
		if (GetBoneData(BoneData, PoseData, SkeletalMeshComponent, Record.BodyModifier.BoneName))
		{
			FTransform BoneTM = BodyInstance->GetUnrealWorldTransform(); // Preserve scale
			BoneTM.SetLocation(BoneData.CurrentTM.GetTranslation());
			BoneTM.SetRotation(BoneData.CurrentTM.GetRotation());

			BodyInstance->SetBodyTransform(BoneTM, ETeleportType::TeleportPhysics);
			BodyInstance->SetLinearVelocity(BoneData.CalculateLinearVelocity(PoseData->DeltaTime), false);
			BodyInstance->SetAngularVelocityInRadians(BoneData.CalculateAngularVelocity(PoseData->DeltaTime), false);
		}
	}
}

//======================================================================================================================
void UPhysicsControlComponent::AddSkeletalMeshReferenceForCaching(
	USkeletalMeshComponent* InSkeletalMeshComponent)
{
	check(InSkeletalMeshComponent);
	for (TPair<TWeakObjectPtr<USkeletalMeshComponent>, UE::PhysicsControl::FPhysicsControlPoseData>& 
		CachedSkeletalMeshDataPair : CachedPoseDatas)
	{
		TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent = CachedSkeletalMeshDataPair.Key;
		if (SkeletalMeshComponent.Get() == InSkeletalMeshComponent)
		{
			UE::PhysicsControl::FPhysicsControlPoseData& CachedSkeletalMeshData = CachedSkeletalMeshDataPair.Value;
			++CachedSkeletalMeshData.ReferenceCount;
			return;
		}
	}
	UE::PhysicsControl::FPhysicsControlPoseData& Data = CachedPoseDatas.Add(InSkeletalMeshComponent);
	Data.ReferenceCount = 1;
	PrimaryComponentTick.AddPrerequisite(InSkeletalMeshComponent, InSkeletalMeshComponent->PrimaryComponentTick);
}

//======================================================================================================================
bool UPhysicsControlComponent::RemoveSkeletalMeshReferenceForCaching(
	USkeletalMeshComponent* InSkeletalMeshComponent)
{
	check(InSkeletalMeshComponent);
	if (!InSkeletalMeshComponent)
	{
		UE_LOGF(LogPhysicsControl, Warning, "Invalid skeletal mesh component in %ls", 
			GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
		return false;
	}

	for (auto It = CachedPoseDatas.CreateIterator(); It; ++It)
	{
		TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent = It.Key();
		UE::PhysicsControl::FPhysicsControlPoseData& Data = It.Value();
		if (SkeletalMeshComponent.Get() == InSkeletalMeshComponent)
		{
			if (--Data.ReferenceCount == 0)
			{
				PrimaryComponentTick.RemovePrerequisite(
					InSkeletalMeshComponent, InSkeletalMeshComponent->PrimaryComponentTick);
				It.RemoveCurrent();
				return true;
			}
			return false;
		}
	}
	UE_LOGF(LogPhysicsControl, Warning, "Failed to remove skeletal mesh component reference for caching");
	return false;
}

//======================================================================================================================
void UPhysicsControlComponent::AddSkeletalMeshReferenceForModifier(
	USkeletalMeshComponent* InSkeletalMeshComponent)
{
	check(InSkeletalMeshComponent);
	for (TPair<TWeakObjectPtr<USkeletalMeshComponent>, FModifiedSkeletalMeshData>& ModifiedSkeletalMeshDataPair :
		ModifiedSkeletalMeshDatas)
	{
		TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent = ModifiedSkeletalMeshDataPair.Key;
		if (SkeletalMeshComponent.Get() == InSkeletalMeshComponent)
		{
			FModifiedSkeletalMeshData& ModifiedSkeletalMeshData = ModifiedSkeletalMeshDataPair.Value;
			++ModifiedSkeletalMeshData.ReferenceCount;
			return;
		}
	}
	FModifiedSkeletalMeshData& Data = ModifiedSkeletalMeshDatas.Add(InSkeletalMeshComponent);
	Data.ReferenceCount = 1;
	Data.bOriginalUpdateMeshWhenKinematic = InSkeletalMeshComponent->bUpdateMeshWhenKinematic;
	Data.OriginalKinematicBonesUpdateType = InSkeletalMeshComponent->KinematicBonesUpdateType;
	InSkeletalMeshComponent->bUpdateMeshWhenKinematic = true;
	// By default, kinematic bodies will have their blend weight set to zero. This is a problem for us since:
	// 1. We expect there will be lots of cases where only part of the character is dynamic, and other 
	//    parts are kinematic
	// 2. If those parts are towards the root of the character, then if their physics blend weight is zero, 
	//    they are unable to "move away" from the component - e.g. if the component itself is moved by the 
	//    movement component
	// 3. We want to support users using the physics blend weight, so we can't simply force a physics blend 
	//    weight of 1 in the skeletal mesh component (PhysAnim.cpp).
	// So, we set all the bodies to have a blend weight of 1, noting that any under the control of a BodyModifier
	// will get updated each tick.
	InSkeletalMeshComponent->SetAllBodiesPhysicsBlendWeight(1.0f);
}

//======================================================================================================================
bool UPhysicsControlComponent::RemoveSkeletalMeshReferenceForModifier(
	USkeletalMeshComponent* InSkeletalMeshComponent)
{
	check(InSkeletalMeshComponent);
	if (!InSkeletalMeshComponent)
	{
		UE_LOGF(LogPhysicsControl, Warning, "Invalid skeletal mesh component");
		return false;
	}

	for (auto It = ModifiedSkeletalMeshDatas.CreateIterator(); It; ++It)
	{
		TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent = It.Key();
		FModifiedSkeletalMeshData& Data = It.Value();
		if (SkeletalMeshComponent.Get() == InSkeletalMeshComponent)
		{
			if (--Data.ReferenceCount == 0)
			{
				InSkeletalMeshComponent->bUpdateMeshWhenKinematic = Data.bOriginalUpdateMeshWhenKinematic;
				InSkeletalMeshComponent->KinematicBonesUpdateType = Data.OriginalKinematicBonesUpdateType;
				It.RemoveCurrent();
				return true;
			}
			return false;
		}
	}
	UE_LOGF(LogPhysicsControl, Warning, "Failed to remove skeletal mesh component reference for modifier");
	return false;
}

//======================================================================================================================
void UPhysicsControlComponent::CalculateControlTargetData(
	FTransform&                  OutTargetTM,
	FTransform&                  OutSkeletalTargetTM,
	FVector&                     OutTargetVelocity,
	FVector&                     OutTargetAngularVelocity,
	const FPhysicsControlRecord& Record,
	bool                         bUsePreviousSkeletalTargetTM) const
{
	OutTargetTM = FTransform();
	OutSkeletalTargetTM = FTransform();
	OutTargetVelocity.Set(0, 0, 0);
	OutTargetAngularVelocity.Set(0, 0, 0);
	float SkeletalDeltaTime = 0.0f;

	bool bUsedSkeletalAnimation = false;
	bool bHasJustTeleported = false;

	// Set the target TM and velocities based on any skeletal action. Note that the targets from animation 
	// should always account for the control point
	if (Record.PhysicsControl.ControlData.bUseSkeletalAnimation)
	{
		UE::PhysicsControl::FBoneData ChildBoneData, ParentBoneData;
		const UE::PhysicsControl::FPhysicsControlPoseData* ChildPoseData = nullptr;
		const UE::PhysicsControl::FPhysicsControlPoseData* ParentPoseData = nullptr;
		bool bHaveChildBoneData = false;
		bool bHaveParentBoneData = false;

		if (USkeletalMeshComponent* ChildSkeletalMeshComponent = GetValidSkeletalMeshComponentFromControlChild(Record))
		{
			bHaveChildBoneData = GetBoneData(
				ChildBoneData, ChildPoseData, ChildSkeletalMeshComponent, Record.PhysicsControl.ChildBoneName);
		}

		if (USkeletalMeshComponent* ParentSkeletalMeshComponent = GetValidSkeletalMeshComponentFromControlParent(Record))
		{
			bHaveParentBoneData = GetBoneData(
				ParentBoneData, ParentPoseData, ParentSkeletalMeshComponent, Record.PhysicsControl.ParentBoneName);
		}
		else if (Record.ParentComponent.IsValid())
		{
			const FTransform ParentTM = Record.ParentComponent->GetComponentTransform();
			ParentBoneData.CurrentTM = ParentTM;
			bHaveParentBoneData = true;
		}

		// Note that the TargetTM calculated so far are supposed to be interpreted as
		// expressed relative to the skeletal animation pose.
		if (bHaveChildBoneData)
		{
			bUsedSkeletalAnimation = true;
			bHasJustTeleported = ChildPoseData->bHasJustTeleported;
			SkeletalDeltaTime = ChildPoseData->DeltaTime;
			const UE::PhysicsControl::FPosQuat ChildBoneTM = ChildBoneData.CurrentTM;
			if (bHaveParentBoneData)
			{
				const UE::PhysicsControl::FPosQuat ParentBoneTM = ParentBoneData.CurrentTM;
				const UE::PhysicsControl::FPosQuat SkeletalDeltaTM = ParentBoneTM.Inverse() * ChildBoneTM;
				// This puts TargetTM in the space of the ParentBone
				OutSkeletalTargetTM = SkeletalDeltaTM.ToTransform();
			}
			else
			{
				OutSkeletalTargetTM = ChildBoneTM.ToTransform();
			}
			// Add on the control point offset
			OutSkeletalTargetTM.AddToTranslation(OutSkeletalTargetTM.GetRotation() * Record.GetControlPoint());
		}
	}

	// Calculate the velocity targets due to skeletal animation
	if (bUsePreviousSkeletalTargetTM && !bHasJustTeleported)
	{
		if (SkeletalDeltaTime * Record.PhysicsControl.ControlData.LinearTargetVelocityMultiplier != 0)
		{
			OutTargetVelocity = (OutSkeletalTargetTM.GetTranslation() - Record.PreviousSkeletalTargetTM.GetTranslation()) *
				(Record.PhysicsControl.ControlData.LinearTargetVelocityMultiplier / SkeletalDeltaTime);
		}
		if (SkeletalDeltaTime * Record.PhysicsControl.ControlData.AngularTargetVelocityMultiplier != 0)
		{
			const FQuat Q = OutSkeletalTargetTM.GetRotation();
			const FQuat PrevQ = Record.PreviousSkeletalTargetTM.GetRotation();
			const FQuat DeltaQ = (Q * PrevQ.Inverse()).GetShortestArcWith(FQuat::Identity);
			OutTargetAngularVelocity = DeltaQ.ToRotationVector() * (
				Record.PhysicsControl.ControlData.AngularTargetVelocityMultiplier / SkeletalDeltaTime);
		}
	}

	// Now apply the explicit target specified in the record. It operates in the space of the target
	// transform we (may have) just calculated.
	const FPhysicsControlTarget& Target = Record.ControlTarget;

	// Calculate the authored target position/orientation - i.e. not using the skeletal animation
	FQuat TargetOrientationQ = Target.TargetOrientation.Quaternion();
	const FVector TargetPosition = Target.TargetPosition;

	FVector ExtraTargetPosition(0);
	// Incorporate the offset from the control point. If we used animation, then we don't need
	// to do this.
	if (!bUsedSkeletalAnimation && 
		Record.ControlTarget.bApplyControlPointToTarget)
	{
		ExtraTargetPosition = TargetOrientationQ * Record.GetControlPoint();
	}

	// Note that Target.TargetAngularVelocity is in revs per second (as it's user-facing)
	// Also, these need to be converted (rotated) from the skeletal target space 
	const FVector TargetAngularVelocity = Target.TargetAngularVelocity * UE_TWO_PI;
	OutTargetAngularVelocity += OutSkeletalTargetTM.GetRotation() * TargetAngularVelocity;
	const FVector ExtraVelocity = TargetAngularVelocity.Cross(ExtraTargetPosition);
	OutTargetVelocity += ExtraVelocity + OutSkeletalTargetTM.GetRotation() * Target.TargetVelocity;

	// The record's target is specified in the space of the previously calculated/set OutSkeletalTargetTM
	OutTargetTM = FTransform(TargetOrientationQ, TargetPosition + ExtraTargetPosition) * OutSkeletalTargetTM;
}

//======================================================================================================================
bool UPhysicsControlComponent::ApplyControlStrengths(
	FPhysicsControlRecord& Record, FConstraintInstance* ConstraintInstance)
{
	const FPhysicsControlData& Data = Record.PhysicsControl.ControlData;
	const FPhysicsControlMultiplier& Multiplier = Record.PhysicsControl.ControlMultiplier;

	float AngularSpring;
	float AngularDamping;
	const float MaxTorque = Data.MaxTorque * Multiplier.MaxTorqueMultiplier;

	FVector LinearSpring;
	FVector LinearDamping;
	const FVector MaxForce = Data.MaxForce * Multiplier.MaxForceMultiplier;

	UE::PhysicsControl::ConvertStrengthToSpringParams(
		AngularSpring, AngularDamping,
		Data.AngularStrength * Multiplier.AngularStrengthMultiplier,
		Data.AngularDampingRatio * Multiplier.AngularDampingRatioMultiplier,
		Data.AngularExtraDamping * Multiplier.AngularExtraDampingMultiplier);
	UE::PhysicsControl::ConvertStrengthToSpringParams(
		LinearSpring, LinearDamping,
		Data.LinearStrength * Multiplier.LinearStrengthMultiplier,
		Data.LinearDampingRatio * Multiplier.LinearDampingRatioMultiplier,
		Data.LinearExtraDamping * Multiplier.LinearExtraDampingMultiplier);

	if (Multiplier.MaxTorqueMultiplier <= 0)
	{
		AngularSpring = 0;
		AngularDamping = 0;
	}
	if (Multiplier.MaxForceMultiplier.X <= 0)
	{
		LinearSpring.X = 0;
		LinearDamping.X = 0;
	}
	if (Multiplier.MaxForceMultiplier.Y <= 0)
	{
		LinearSpring.Y = 0;
		LinearDamping.Y = 0;
	}
	if (Multiplier.MaxForceMultiplier.Z <= 0)
	{
		LinearSpring.Z = 0;
		LinearDamping.Z = 0;
	}

	ConstraintInstance->SetLinearDriveAccelerationMode(Data.bUseAccelerationDriveMode);
	ConstraintInstance->SetAngularDriveAccelerationMode(Data.bUseAccelerationDriveMode);

	ConstraintInstance->SetDriveParams(
		LinearSpring, LinearDamping, MaxForce,
		FVector(0, 0, AngularSpring), FVector(0, 0, AngularDamping), FVector(0, 0, MaxTorque),
		EAngularDriveMode::SLERP);

	const bool bHaveAngular = (AngularSpring + AngularDamping) > 0;
	const bool bHaveLinear = (LinearSpring + LinearDamping).GetMax() > 0;
	return bHaveLinear || bHaveAngular;
}

//======================================================================================================================
void UPhysicsControlComponent::ApplyControl(
	FPhysicsControlRecord&     Record,
	FConstraintInstance*       PhysicsAssetConstraintInstance,
	const FConstraintInstance* DefaultConstraintInstance)
{
	FConstraintInstance* ConstraintInstance = Record.ConstraintInstance.Get();

	if (!ConstraintInstance)
	{
		return;
	}

	if (!Record.PhysicsControl.IsEnabled())
	{
		// Note that this will disable the constraint elements when strength/damping are zero
		ConstraintInstance->SetDriveParams(
			FVector::ZeroVector, FVector::ZeroVector, FVector::ZeroVector,
			FVector::ZeroVector, FVector::ZeroVector, FVector::ZeroVector,
			EAngularDriveMode::SLERP);
		return;
	}

	// Always control collision, because otherwise maintaining it is very difficult, since
	// constraint-controlled collision doesn't interact nicely when there are multiple constraints.
	ConstraintInstance->SetDisableCollision(Record.PhysicsControl.ControlData.bDisableCollision);

	FBodyInstance* ParentBodyInstance = UE::PhysicsControl::GetBodyInstance(
		Record.ParentComponent.Get(), Record.PhysicsControl.ParentBoneName);

	FBodyInstance* ChildBodyInstance = UE::PhysicsControl::GetBodyInstance(
		Record.ChildComponent.Get(), Record.PhysicsControl.ChildBoneName);

	if (!ParentBodyInstance && !ChildBodyInstance)
	{
		return;
	}

	// Set strengths etc and then targets (if there were strengths)
	if (ApplyControlStrengths(Record, ConstraintInstance))
	{
		// PhysicsAssetConstraintInstance and DefaultConstraintInstance were looked up once in UpdateControls before this call.

		// Resolve the response up front. When switching from WidenLimits to ClampLinear/ClampExact,
		// the clamp must see the authored limit values, not previously widened ones, so restore here
		// before computing the target.
		EAngularDriveLimitViolationResponse Response = EAngularDriveLimitViolationResponse::None;
		if (PhysicsAssetConstraintInstance)
		{
			Response = PhysicsAssetConstraintInstance->ProfileInstance.AngularDrive.LimitViolationResponse;
			const int32 Override = CVarPhysicsControlLimitViolationResponseOverride.GetValueOnGameThread();
			if (Override >= 0 && Override <= (int32)EAngularDriveLimitViolationResponse::WidenLimits)
			{
				Response = (EAngularDriveLimitViolationResponse)Override;
			}
			if (Record.bJointLimitsWidened && DefaultConstraintInstance &&
				(Response == EAngularDriveLimitViolationResponse::ClampLinear ||
				 Response == EAngularDriveLimitViolationResponse::ClampExact))
			{
				PhysicsAssetConstraintInstance->RestoreAngularLimitsToDefault(*DefaultConstraintInstance);
				Record.bJointLimitsWidened = false;
			}
		}

		FTransform TargetTM, SkeletalTargetTM;
		FVector TargetVelocity;
		FVector TargetAngularVelocity;
		bool bUsePreviousSkeletalTargetTM = CurrentUpdateCounter.HasEverBeenUpdated() && (CurrentUpdateCounter.Get() == Record.ExpectedUpdateCounter.Get());
		CalculateControlTargetData(TargetTM, SkeletalTargetTM, TargetVelocity, TargetAngularVelocity, Record, bUsePreviousSkeletalTargetTM);
		Record.PreviousSkeletalTargetTM = SkeletalTargetTM;
		Record.ExpectedUpdateCounter = CurrentUpdateCounter;
		Record.ExpectedUpdateCounter.Increment();

		// Apply the limit-violation response to the final composed drive target. Both clamp and
		// widen operate on TargetTM (not just the skeletal component) so they act on the rotation
		// that is actually sent to the physics constraint as the drive target.
		if (PhysicsAssetConstraintInstance && DefaultConstraintInstance)
		{
			if (Response == EAngularDriveLimitViolationResponse::ClampLinear ||
				Response == EAngularDriveLimitViolationResponse::ClampExact)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_PhysicsControl_ClampDriveTarget);
				const bool bUseExact = (Response == EAngularDriveLimitViolationResponse::ClampExact);
				TargetTM.SetRotation(PhysicsAssetConstraintInstance->ClampDriveTargetToLimits(TargetTM.GetRotation(), bUseExact));
			}
			else if (Response == EAngularDriveLimitViolationResponse::WidenLimits)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_PhysicsControl_WidenLimits);
				Record.bJointLimitsWidened = PhysicsAssetConstraintInstance->WidenLimitsForDriveTarget(
					TargetTM.GetRotation(), *DefaultConstraintInstance);
			}
			else if (Record.bJointLimitsWidened)
			{
				// Response is None; restore any previously widened limits.
				PhysicsAssetConstraintInstance->RestoreAngularLimitsToDefault(*DefaultConstraintInstance);
				Record.bJointLimitsWidened = false;
			}
		}

		ConstraintInstance->SetLinearPositionTarget(TargetTM.GetTranslation());
		ConstraintInstance->SetAngularOrientationTarget(TargetTM.GetRotation());
		ConstraintInstance->SetLinearVelocityTarget(TargetVelocity);
		ConstraintInstance->SetAngularVelocityTarget(TargetAngularVelocity / UE_TWO_PI); // In rev/sec
		ConstraintInstance->SetParentDominates(Record.PhysicsControl.ControlData.bOnlyControlChildObject);

		if (ParentBodyInstance)
		{
			ParentBodyInstance->WakeInstance();
		}
		if (ChildBodyInstance)
		{
			ChildBodyInstance->WakeInstance();
		}
	}
}

//======================================================================================================================
void UPhysicsControlComponent::ApplyBodyModifier(FPhysicsBodyModifierRecord& Record)
{
	USkeletalMeshComponent* SKM = GetValidSkeletalMeshComponentFromBodyModifier(Record);
	FBodyInstance* BodyInstance = UE::PhysicsControl::GetBodyInstance(
		Record.Component.Get(), Record.BodyModifier.BoneName);
	if (BodyInstance)
	{
		switch (Record.BodyModifier.ModifierData.MovementType)
		{
		case EPhysicsMovementType::Static:
			BodyInstance->SetInstanceSimulatePhysics(false, false, true);
			break;
		case EPhysicsMovementType::Kinematic:
			BodyInstance->SetInstanceSimulatePhysics(false, false, true);
			ApplyKinematicTarget(Record);
			break;
		case EPhysicsMovementType::Simulated:
			BodyInstance->SetInstanceSimulatePhysics(true, false, true);
			break;
		case EPhysicsMovementType::Default:
			// Default means do nothing, so let's do exactly that
			break;
		default:
			UE_LOGF(LogPhysicsControl, Warning, "Invalid movement type %d in %ls",
				int(Record.BodyModifier.ModifierData.MovementType),
				GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
			break;
		}

		// We always overwrite the physics blend weight, since the functions above can still modify
		// it (even though they all use the "maintain physics blending" option), since there is an
		// expectation that zero blend weight means to disable physics.
		BodyInstance->PhysicsBlendWeight = Record.BodyModifier.ModifierData.PhysicsBlendWeight;
		BodyInstance->SetUpdateKinematicFromSimulation(Record.BodyModifier.ModifierData.bUpdateKinematicFromSimulation);

		// On the shapes, this determines whether there is actually collision. Note that the bodies
		// need to also have "collision enabled" in order to even be allowed to simulate, which is
		// normally done via the skeletal mesh.
		UBodySetup* BodySetup = BodyInstance->GetBodySetup();
		if (BodySetup)
		{
			int32 NumShapes = BodySetup->AggGeom.GetElementCount();
			for (int32 ShapeIndex = 0; ShapeIndex != NumShapes; ++ShapeIndex)
			{
				BodyInstance->SetShapeCollisionEnabled(ShapeIndex, Record.BodyModifier.ModifierData.CollisionType);
			}
		}

		if (BodyInstance->IsInstanceSimulatingPhysics())
		{
			const float GravityZ = BodyInstance->GetPhysicsScene()->GetOwningWorld()->GetGravityZ();
			const float AppliedGravityZ = BodyInstance->bEnableGravity ? GravityZ : 0.0f;
			const float DesiredGravityZ = GravityZ * Record.BodyModifier.ModifierData.GravityMultiplier;
			const float GravityZToApply = DesiredGravityZ - AppliedGravityZ;
			BodyInstance->AddForce(FVector(0, 0, GravityZToApply), true, true);
		}
	}
	if (Record.bResetToCachedTarget)
	{
		Record.bResetToCachedTarget = false;
		ResetToCachedTarget(Record);
	}
}

//======================================================================================================================
FPhysicsBodyModifierRecord* UPhysicsControlComponent::FindBodyModifierRecord(const FName Name)
{
	return BodyModifierRecords.Find(Name);
}

//======================================================================================================================
const FPhysicsBodyModifierRecord* UPhysicsControlComponent::FindBodyModifierRecord(const FName Name) const
{
	return BodyModifierRecords.Find(Name);
}

//======================================================================================================================
bool UPhysicsControlComponent::DestroyControl(const FName Name, const EDestroyBehavior DestroyBehavior)
{
	FPhysicsControlRecord* Record = FindControlRecord(Name);
	if (Record)
	{
		if (USkeletalMeshComponent* SkeletalMeshComponent = GetValidSkeletalMeshComponentFromControlParent(*Record))
		{
			RemoveSkeletalMeshReferenceForCaching(SkeletalMeshComponent);
		}
		if (USkeletalMeshComponent* SkeletalMeshComponent = GetValidSkeletalMeshComponentFromControlChild(*Record))
		{
			RemoveSkeletalMeshReferenceForCaching(SkeletalMeshComponent);
		}

		Record->ResetConstraint(); // This terminates the constraint
		NameRecords.RemoveControl(Name);
		if (DestroyBehavior == EDestroyBehavior::RemoveRecord)
		{
			ensure(ControlRecords.Remove(Name) == 1);
		}
		return true;
	}
	if (bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning, "DestroyControl - invalid control name %ls for %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return false;
}

//======================================================================================================================
bool UPhysicsControlComponent::DestroyBodyModifier(const FName Name, const EDestroyBehavior DestroyBehavior)
{
	FPhysicsBodyModifierRecord* BodyModifier = FindBodyModifierRecord(Name);
	if (BodyModifier)
	{
		if (USkeletalMeshComponent* SkeletalMeshComponent = GetValidSkeletalMeshComponentFromBodyModifier(*BodyModifier))
		{
			RemoveSkeletalMeshReferenceForCaching(SkeletalMeshComponent);
			RemoveSkeletalMeshReferenceForModifier(SkeletalMeshComponent);
		}
		NameRecords.RemoveBodyModifier(Name);
		if (DestroyBehavior == EDestroyBehavior::RemoveRecord)
		{
			ensure(BodyModifierRecords.Remove(Name) == 1);
		}
		return true;
	}
	if (bWarnAboutInvalidNames)
	{
		UE_LOGF(LogPhysicsControl, Warning, "DestroyBodyModifier - invalid modifier name %ls for %ls",
			*Name.ToString(), GetOwner() ? *GetOwner()->GetName() : TEXT("Unknown"));
	}
	return false;
}

