// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlData.h"
#include "PhysicsControlObjectVersion.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsActorHandle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsControlData)

#define SET_SPARSE_DATA(NAME) NAME = SparseData.bEnable##NAME ? SparseData.NAME : NAME

#define INTERPOLATE_PARAM(NAME) Output.NAME = FMath::Lerp(A.NAME, B.NAME, Weight)
#define SELECT_PARAM(NAME) Output.NAME = (Weight < 0.5f) ? A.NAME : B.NAME
#define SET_ENABLED_PARAM(NAME) Output.bEnable##NAME = A.bEnable##NAME && B.bEnable##NAME

//======================================================================================================================
EPhysicsControlForceType ConvertForceType(ImmediatePhysics_Shared::EForceType InForceType)
{
	switch (InForceType)
	{
	case ImmediatePhysics_Shared::EForceType::AddForce: return EPhysicsControlForceType::Force;
	case ImmediatePhysics_Shared::EForceType::AddAcceleration: return EPhysicsControlForceType::Acceleration;
	case ImmediatePhysics_Shared::EForceType::AddImpulse: return EPhysicsControlForceType::Impulse;
	case ImmediatePhysics_Shared::EForceType::AddVelocity: return EPhysicsControlForceType::VelocityChange;
	default:
		return EPhysicsControlForceType::Force;
	}
}

//======================================================================================================================
ImmediatePhysics_Shared::EForceType ConvertForceType(EPhysicsControlForceType InForceType)
{
	switch (InForceType)
	{
	case EPhysicsControlForceType::Force: return ImmediatePhysics_Shared::EForceType::AddForce;
	case EPhysicsControlForceType::Acceleration: return ImmediatePhysics_Shared::EForceType::AddAcceleration;
	case EPhysicsControlForceType::Impulse: return ImmediatePhysics_Shared::EForceType::AddImpulse;
	case EPhysicsControlForceType::VelocityChange: return ImmediatePhysics_Shared::EForceType::AddVelocity;
	default:
		return ImmediatePhysics_Shared::EForceType::AddForce;
	}
}

//======================================================================================================================
FPhysicsControlSetUpdates& FPhysicsControlSetUpdates::operator+=(const FPhysicsControlSetUpdates& Other)
{
	ControlSetUpdates.Append(Other.ControlSetUpdates);
	ModifierSetUpdates.Append(Other.ModifierSetUpdates);
	return *this;
}

//======================================================================================================================
FPhysicsControlAndBodyModifierCreationDatas& FPhysicsControlAndBodyModifierCreationDatas::operator+=(
	const FPhysicsControlAndBodyModifierCreationDatas& Other)
{
	Controls.Append(Other.Controls);
	Modifiers.Append(Other.Modifiers);
	return *this;
}

//======================================================================================================================
bool FPhysicsControlAndBodyModifierCreationDatas::operator==(const FPhysicsControlAndBodyModifierCreationDatas& other) const
{
	if (Controls.OrderIndependentCompareEqual(other.Controls) &&
		Modifiers.OrderIndependentCompareEqual(other.Modifiers))
	{
		return true;
	}
	return false;
}

//======================================================================================================================
FVector FPhysicsControlData::GetControlPoint(const ImmediatePhysics::FActorHandle* ChildActorHandle) const
{
	if (bUseCustomControlPoint)
	{
		return CustomControlPoint;
	}
	return ChildActorHandle->GetLocalCoMLocation();
}

//======================================================================================================================
FPhysicsControlData Interpolate(
	const FPhysicsControlData& A, const FPhysicsControlData& B, const float Weight)
{
	FPhysicsControlData Output;
	INTERPOLATE_PARAM(LinearStrength);
	INTERPOLATE_PARAM(LinearDampingRatio);
	INTERPOLATE_PARAM(LinearExtraDamping);
	INTERPOLATE_PARAM(AngularStrength);
	INTERPOLATE_PARAM(AngularDampingRatio);
	INTERPOLATE_PARAM(AngularExtraDamping);
	INTERPOLATE_PARAM(LinearTargetVelocityMultiplier);
	INTERPOLATE_PARAM(AngularTargetVelocityMultiplier);
	INTERPOLATE_PARAM(CustomControlPoint);
	SELECT_PARAM(bEnabled);
	SELECT_PARAM(bUseCustomControlPoint);
	SELECT_PARAM(bUseSkeletalAnimation);
	SELECT_PARAM(bDisableCollision);
	SELECT_PARAM(bOnlyControlChildObject);
	SELECT_PARAM(bUseAccelerationDriveMode);
	return Output;
}

//======================================================================================================================
FPhysicsControlSparseData Interpolate(
	const FPhysicsControlSparseData& A, const FPhysicsControlSparseData& B, const float Weight)
{
	FPhysicsControlSparseData Output;
	INTERPOLATE_PARAM(LinearStrength);
	INTERPOLATE_PARAM(LinearDampingRatio);
	INTERPOLATE_PARAM(LinearExtraDamping);
	INTERPOLATE_PARAM(AngularStrength);
	INTERPOLATE_PARAM(AngularDampingRatio);
	INTERPOLATE_PARAM(AngularExtraDamping);
	INTERPOLATE_PARAM(LinearTargetVelocityMultiplier);
	INTERPOLATE_PARAM(AngularTargetVelocityMultiplier);
	INTERPOLATE_PARAM(CustomControlPoint);
	SELECT_PARAM(bEnabled);
	SELECT_PARAM(bUseCustomControlPoint);
	SELECT_PARAM(bUseSkeletalAnimation);
	SELECT_PARAM(bDisableCollision);
	SELECT_PARAM(bOnlyControlChildObject);
	SELECT_PARAM(bUseAccelerationDriveMode);

	SET_ENABLED_PARAM(LinearStrength);
	SET_ENABLED_PARAM(LinearDampingRatio);
	SET_ENABLED_PARAM(LinearExtraDamping);
	SET_ENABLED_PARAM(AngularStrength);
	SET_ENABLED_PARAM(AngularDampingRatio);
	SET_ENABLED_PARAM(AngularExtraDamping);
	SET_ENABLED_PARAM(LinearTargetVelocityMultiplier);
	SET_ENABLED_PARAM(AngularTargetVelocityMultiplier);
	SET_ENABLED_PARAM(CustomControlPoint);
	SET_ENABLED_PARAM(bEnabled);
	SET_ENABLED_PARAM(bUseCustomControlPoint);
	SET_ENABLED_PARAM(bUseSkeletalAnimation);
	SET_ENABLED_PARAM(bDisableCollision);
	SET_ENABLED_PARAM(bOnlyControlChildObject);
	SET_ENABLED_PARAM(bUseAccelerationDriveMode);

	return Output;
}

//======================================================================================================================
FPhysicsControlModifierData Interpolate(
	const FPhysicsControlModifierData& A, const FPhysicsControlModifierData& B, const float Weight)
{
	FPhysicsControlModifierData Output;
	SELECT_PARAM(MovementType);
	SELECT_PARAM(CollisionType);
	INTERPOLATE_PARAM(GravityMultiplier);
	INTERPOLATE_PARAM(PhysicsBlendWeight);
	SELECT_PARAM(KinematicTargetSpace);
	SELECT_PARAM(bUpdateKinematicFromSimulation);
	SELECT_PARAM(bEnableCCD);
	return Output;
}

//======================================================================================================================
FPhysicsControlModifierSparseData Interpolate(
	const FPhysicsControlModifierSparseData& A, const FPhysicsControlModifierSparseData& B, const float Weight)
{
	FPhysicsControlModifierSparseData Output;
	SELECT_PARAM(MovementType);
	SELECT_PARAM(CollisionType);
	INTERPOLATE_PARAM(GravityMultiplier);
	INTERPOLATE_PARAM(PhysicsBlendWeight);
	SELECT_PARAM(KinematicTargetSpace);
	SELECT_PARAM(bUpdateKinematicFromSimulation);
	SELECT_PARAM(bEnableCCD);

	SET_ENABLED_PARAM(MovementType);
	SET_ENABLED_PARAM(CollisionType);
	SET_ENABLED_PARAM(GravityMultiplier);
	SET_ENABLED_PARAM(PhysicsBlendWeight);
	SET_ENABLED_PARAM(KinematicTargetSpace);
	SET_ENABLED_PARAM(bUpdateKinematicFromSimulation);
	SET_ENABLED_PARAM(bEnableCCD);
	return Output;
}

//======================================================================================================================
void FPhysicsControlData::UpdateFromSparseData(const FPhysicsControlSparseData& SparseData)
{
	SET_SPARSE_DATA(LinearStrength);
	SET_SPARSE_DATA(LinearDampingRatio);
	SET_SPARSE_DATA(LinearExtraDamping);
	SET_SPARSE_DATA(MaxForce);
	SET_SPARSE_DATA(AngularStrength);
	SET_SPARSE_DATA(AngularDampingRatio);
	SET_SPARSE_DATA(AngularExtraDamping);
	SET_SPARSE_DATA(MaxTorque);
	SET_SPARSE_DATA(LinearTargetVelocityMultiplier);
	SET_SPARSE_DATA(AngularTargetVelocityMultiplier);
	SET_SPARSE_DATA(CustomControlPoint);
	SET_SPARSE_DATA(bEnabled);
	SET_SPARSE_DATA(bUseCustomControlPoint);
	SET_SPARSE_DATA(bUseSkeletalAnimation);
	SET_SPARSE_DATA(bDisableCollision);
	SET_SPARSE_DATA(bOnlyControlChildObject);
	SET_SPARSE_DATA(bUseAccelerationDriveMode);
}

//======================================================================================================================
void FPhysicsControlMultiplier::UpdateFromSparseData(const FPhysicsControlSparseMultiplier& SparseData)
{
	SET_SPARSE_DATA(LinearStrengthMultiplier);
	SET_SPARSE_DATA(LinearDampingRatioMultiplier);
	SET_SPARSE_DATA(LinearExtraDampingMultiplier);
	SET_SPARSE_DATA(MaxForceMultiplier);
	SET_SPARSE_DATA(AngularStrengthMultiplier);
	SET_SPARSE_DATA(AngularDampingRatioMultiplier);
	SET_SPARSE_DATA(AngularExtraDampingMultiplier);
	SET_SPARSE_DATA(MaxTorqueMultiplier);
}

//======================================================================================================================
void FPhysicsControlModifierData::UpdateFromSparseData(const FPhysicsControlModifierSparseData& SparseData)
{
	SET_SPARSE_DATA(MovementType);
	SET_SPARSE_DATA(CollisionType);
	SET_SPARSE_DATA(GravityMultiplier);
	SET_SPARSE_DATA(PhysicsBlendWeight);
	SET_SPARSE_DATA(KinematicTargetSpace);
	SET_SPARSE_DATA(bUpdateKinematicFromSimulation);
	SET_SPARSE_DATA(bEnableCCD);
}

//======================================================================================================================
FArchive& operator<<(FArchive& Ar, FPhysicsControlData& Data)
{
	Ar << Data.LinearStrength;
	Ar << Data.LinearDampingRatio;
	Ar << Data.LinearExtraDamping;
	Ar << Data.MaxForce;
	Ar << Data.AngularStrength;
	Ar << Data.AngularDampingRatio;
	Ar << Data.AngularExtraDamping;
	Ar << Data.MaxTorque;
	Ar << Data.LinearTargetVelocityMultiplier;
	Ar << Data.AngularTargetVelocityMultiplier;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) < FPhysicsControlObjectVersion::RemoveSkeletalAnimationVelocityMultiplier)
	{
		float SkeletalAnimationVelocityMultiplier = 0.0f;
		Ar << SkeletalAnimationVelocityMultiplier;
	}
	Ar << Data.CustomControlPoint;
	Ar << Data.bEnabled;
	Ar << Data.bUseCustomControlPoint;
	Ar << Data.bUseSkeletalAnimation;
	Ar << Data.bDisableCollision;
	Ar << Data.bOnlyControlChildObject;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlDataDriveForceMode)
	{
		Ar << Data.bUseAccelerationDriveMode;
	}
	return Ar;
}

FArchive& operator <<(FArchive& Ar, FPhysicsControlModifierData& Data)
{
	Ar << Data.MovementType;
	Ar << Data.CollisionType;
	Ar << Data.GravityMultiplier;
	Ar << Data.PhysicsBlendWeight;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) < FPhysicsControlObjectVersion::RemoveUseSkeletalAnimation)
	{
		bool bUseSkeletalAnimation = false;
		Ar << bUseSkeletalAnimation;
	}
	else
	{
		Ar << Data.KinematicTargetSpace;
	}
	Ar << Data.bUpdateKinematicFromSimulation;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::EnableCCD)
	{
		Ar << Data.bEnableCCD;
	}
	return Ar;
}



#undef SET_SPARSE_DATA
#undef INTERPOLATE_PARAM
#undef SELECT_PARAM
#undef SET_ENABLED_PARAM
