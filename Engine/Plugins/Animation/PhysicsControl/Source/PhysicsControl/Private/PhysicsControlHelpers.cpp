// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlHelpers.h"
#include "PhysicsControlData.h"
#include "PhysicsEngine/ConstraintInstance.h"

#include "Components/PrimitiveComponent.h"
#include "Components/SkeletalMeshComponent.h"

namespace UE
{
namespace PhysicsControl
{

//======================================================================================================================
void ConvertStrengthToSpringParams(
	FVector& OutSpring, FVector& OutDamping, 
	const FVector& InStrength, const FVector& InDampingRatio, const FVector& InExtraDamping)
{
	ConvertStrengthToSpringParams(OutSpring.X, OutDamping.X, InStrength.X, InDampingRatio.X, InExtraDamping.X);
	ConvertStrengthToSpringParams(OutSpring.Y, OutDamping.Y, InStrength.Y, InDampingRatio.Y, InExtraDamping.Y);
	ConvertStrengthToSpringParams(OutSpring.Z, OutDamping.Z, InStrength.Z, InDampingRatio.Z, InExtraDamping.Z);
}

inline double GetLinearDriveStiffness(const FLinearDriveConstraint& InDrive)
{
	return (
		(InDrive.XDrive.bEnablePositionDrive ? InDrive.XDrive.Stiffness : 0.0) + 
		(InDrive.YDrive.bEnablePositionDrive ? InDrive.YDrive.Stiffness : 0.0) +
		(InDrive.ZDrive.bEnablePositionDrive ? InDrive.ZDrive.Stiffness : 0.0) 
		) / 3.0;
}
inline double GetLinearDriveDamping(const FLinearDriveConstraint& InDrive)
{
	return (
		(InDrive.XDrive.bEnableVelocityDrive ? InDrive.XDrive.Damping : 0.0) +
		(InDrive.YDrive.bEnableVelocityDrive ? InDrive.YDrive.Damping : 0.0) + 
		(InDrive.ZDrive.bEnableVelocityDrive ? InDrive.ZDrive.Damping : 0.0) 
		) / 3.0;
}
inline double GetLinearDriveMaxForce(const FLinearDriveConstraint& InDrive)
{
	return FVector(InDrive.XDrive.MaxForce, InDrive.YDrive.MaxForce, InDrive.ZDrive.MaxForce).Length();
}

inline double GetAngularDriveStiffness(const FAngularDriveConstraint& InDrive)
{
	if (InDrive.AngularDriveMode == EAngularDriveMode::SLERP)
	{
		return InDrive.SlerpDrive.bEnablePositionDrive ? InDrive.SlerpDrive.Stiffness : 0.0;
	}
	else
	{
		return (
			(InDrive.TwistDrive.bEnablePositionDrive ? InDrive.TwistDrive.Stiffness : 0.0) + 
			(InDrive.SwingDrive.bEnablePositionDrive ? InDrive.SwingDrive.Stiffness : 0.0)
			) / 2.0;
	}
}
inline double GetAngularDriveDamping(const FAngularDriveConstraint& InDrive)
{
	if (InDrive.AngularDriveMode == EAngularDriveMode::SLERP)
	{
		return InDrive.SlerpDrive.bEnableVelocityDrive ? InDrive.SlerpDrive.Damping : 0.0;
	}
	else
	{
		return (
			(InDrive.TwistDrive.bEnableVelocityDrive ? InDrive.TwistDrive.Damping : 0.0) +
			(InDrive.SwingDrive.bEnableVelocityDrive ? InDrive.SwingDrive.Damping : 0.0)
			) / 2.0;
	}
}
inline double GetAngularDriveMaxTorque(const FAngularDriveConstraint& InDrive)
{
	if (InDrive.AngularDriveMode == EAngularDriveMode::SLERP)
	{
		return InDrive.SlerpDrive.MaxForce;
	}
	else
	{
		return FVector2D(InDrive.TwistDrive.MaxForce, InDrive.SwingDrive.MaxForce).Length();
	}
}

//======================================================================================================================
void ConvertConstraintProfileToControlData(
	FPhysicsControlData& OutControlData, const FConstraintProfileProperties& InProfileProperties)
{
	ConvertSpringToStrengthParams(
		OutControlData.LinearStrength, 
		OutControlData.LinearDampingRatio, 
		OutControlData.LinearExtraDamping,
		GetLinearDriveStiffness(InProfileProperties.LinearDrive), 
		GetLinearDriveDamping(InProfileProperties.LinearDrive));

	OutControlData.MaxForce = float(GetLinearDriveMaxForce(InProfileProperties.LinearDrive));

	ConvertSpringToStrengthParams(
		OutControlData.AngularStrength,
		OutControlData.AngularDampingRatio,
		OutControlData.AngularExtraDamping,
		GetAngularDriveStiffness(InProfileProperties.AngularDrive),
		GetAngularDriveDamping(InProfileProperties.AngularDrive));

	OutControlData.MaxTorque = float(GetAngularDriveMaxTorque(InProfileProperties.AngularDrive));

	OutControlData.bEnabled = true;
}

//======================================================================================================================
FBodyInstance* GetBodyInstance(UPrimitiveComponent* Component, const FName BoneName)
{
	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Component))
	{
		return SkeletalMeshComponent->GetBodyInstance(BoneName);
	}
	if (Component)
	{
		return Component->GetBodyInstance();
	}
	return nullptr;
}

//======================================================================================================================
FName GetPhysicalParentBone(USkeletalMeshComponent* SkeletalMeshComponent, FName BoneName)
{
	while (true)
	{
		FName ParentBoneName = SkeletalMeshComponent->GetParentBone(BoneName);
		if (ParentBoneName.IsNone() || ParentBoneName == BoneName)
		{
			return FName();
		}
		if (const FBodyInstance* ParentBodyInstance = GetBodyInstance(SkeletalMeshComponent, ParentBoneName))
		{
			return ParentBoneName;
		}
		BoneName = ParentBoneName;
	}
}

} // namespace PhysicsControl
} // namespace UE

