// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlLimbData.h"

//======================================================================================================================

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsControlLimbData)
FPhysicsControlCharacterSetupData& FPhysicsControlCharacterSetupData::operator+=(
	const FPhysicsControlCharacterSetupData& Other)
{
	LimbSetupData += Other.LimbSetupData;
	DefaultWorldSpaceControlData = Other.DefaultWorldSpaceControlData;
	DefaultParentSpaceControlData = Other.DefaultParentSpaceControlData;
	DefaultBodyModifierData = Other.DefaultBodyModifierData;
	return *this;
}

