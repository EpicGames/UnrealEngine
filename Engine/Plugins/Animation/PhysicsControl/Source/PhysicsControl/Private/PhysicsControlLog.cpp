// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlLog.h"

PHYSICSCONTROL_API DEFINE_LOG_CATEGORY(LogPhysicsControl);
