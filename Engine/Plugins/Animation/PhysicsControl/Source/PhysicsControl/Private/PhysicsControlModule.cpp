// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlModule.h"
#include "PhysicsControlObjectVersion.h"
#include "UObject/DevObjectVersion.h"

#define LOCTEXT_NAMESPACE "FPhysicsControlModule"

// Unique version id
const FGuid FPhysicsControlObjectVersion::GUID(0xE68A5C1A, 0x42C3CA0D, 0xB01441B2, 0x28C1F2DD);
// Register custom version with Core
FDevVersionRegistration GRegisterPhysicsControlObjectVersion(
	FPhysicsControlObjectVersion::GUID, FPhysicsControlObjectVersion::LatestVersion, TEXT("Dev-PhysicsControl"));

void FPhysicsControlModule::StartupModule()
{
}

void FPhysicsControlModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FPhysicsControlModule, PhysicsControl)

