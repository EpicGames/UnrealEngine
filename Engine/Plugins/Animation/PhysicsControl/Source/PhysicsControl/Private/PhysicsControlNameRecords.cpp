// Copyright Epic Games, Inc. All Rights Reserved.
#include "PhysicsControlNameRecords.h"

//======================================================================================================================
void AddUniqueNamesToSet(TConstArrayView<FName> Names, const FName SetName, TMap<FName, TArray<FName>>& SetCollection)
{
	if (!SetName.IsNone())
	{
		TArray<FName>& SetElementNames = SetCollection.FindOrAdd(SetName);
		for (const FName Name : Names)
		{
			SetElementNames.AddUnique(Name);
		}
	}
}

//======================================================================================================================
void FPhysicsControlNameRecords::AddControl(FName Name, FName SetName, bool bAddToAll)
{
	if (!SetName.IsNone())
	{
		ControlSets.FindOrAdd(SetName).AddUnique(Name);
	}
	if (bAddToAll)
	{
		ControlSets.FindOrAdd(TEXT("All")).AddUnique(Name);
	}
}

//======================================================================================================================
void FPhysicsControlNameRecords::AddControl(FName Name, TConstArrayView<FName> SetNames, bool bAddToAll)
{
	for (const FName SetName : SetNames)
	{
		if (!SetName.IsNone())
		{
			ControlSets.FindOrAdd(SetName).AddUnique(Name);
		}
	}
	if (bAddToAll)
	{
		ControlSets.FindOrAdd(TEXT("All")).AddUnique(Name);
	}
}


//======================================================================================================================
void FPhysicsControlNameRecords::AddControls(TConstArrayView<FName> ControlNames, const FName SetName, bool bAddToAll)
{
	AddUniqueNamesToSet(ControlNames, SetName, ControlSets);
	if (bAddToAll)
	{
		AddUniqueNamesToSet(ControlNames, "All", ControlSets);
	}
}

//======================================================================================================================
void FPhysicsControlNameRecords::RemoveControl(FName Name)
{
	for (TPair<FName, TArray<FName>>& Set : ControlSets)
	{
		Set.Value.Remove(Name);
	}
}

//======================================================================================================================
const TArray<FName>& FPhysicsControlNameRecords::GetControlNamesInSet(FName SetName) const
{
	const TArray<FName>* Names = ControlSets.Find(SetName);
	if (Names)
	{
		return *Names;
	}
	static TArray<FName> FailureResult;
	return FailureResult;
}

//======================================================================================================================
void FPhysicsControlNameRecords::AddBodyModifier(FName Name, FName SetName, bool bAddToAll)
{
	if (!SetName.IsNone())
	{
		BodyModifierSets.FindOrAdd(SetName).AddUnique(Name);
	}
	if (bAddToAll)
	{
		BodyModifierSets.FindOrAdd(TEXT("All")).AddUnique(Name);
	}
}

//======================================================================================================================
void FPhysicsControlNameRecords::AddBodyModifier(FName Name, TConstArrayView<FName> SetNames, bool bAddToAll)
{
	for (FName SetName : SetNames)
	{
		if (!SetName.IsNone())
		{
			BodyModifierSets.FindOrAdd(SetName).AddUnique(Name);
		}
	}
	if (bAddToAll)
	{
		BodyModifierSets.FindOrAdd(TEXT("All")).AddUnique(Name);
	}
}

//======================================================================================================================
void FPhysicsControlNameRecords::AddBodyModifiers(
	TConstArrayView<FName> BodyModifierNames, const FName SetName, bool bAddToAll)
{
	AddUniqueNamesToSet(BodyModifierNames, SetName, BodyModifierSets);
	if (bAddToAll)
	{
		AddUniqueNamesToSet(BodyModifierNames, TEXT("All"), BodyModifierSets);
	}
}

//======================================================================================================================
void FPhysicsControlNameRecords::RemoveBodyModifier(FName Name)
{
	for (TPair<FName, TArray<FName>>& Set : BodyModifierSets)
	{
		Set.Value.Remove(Name);
	}
}

//======================================================================================================================
const TArray<FName>& FPhysicsControlNameRecords::GetBodyModifierNamesInSet(FName SetName) const
{
	const TArray<FName>* Names = BodyModifierSets.Find(SetName);
	if (Names)
	{
		return *Names;
	}
	static TArray<FName> FailureResult;
	return FailureResult;
}

//======================================================================================================================
void FPhysicsControlNameRecords::Reset()
{
	ControlSets.Reset();
	BodyModifierSets.Reset();
}

//======================================================================================================================
TArray<FName> ExpandName(const FName InName, const TMap<FName, TArray<FName>>& SetNames)
{
	TArray<FName> OutputNames;
	if (const TArray<FName>* const FoundSet = SetNames.Find(InName))
	{
		OutputNames.Append(*FoundSet);
	}
	else
	{
		OutputNames.Add(InName);
	}
	return OutputNames;
}

//======================================================================================================================
TArray<FName> ExpandNames(const TArray<FName>& InNames, const TMap<FName, TArray<FName>>& SetNames)
{
	TArray<FName> OutputNames;
	for (const FName Name : InNames)
	{
		OutputNames.Append(ExpandName(Name, SetNames));
	}
	return OutputNames;
}
