// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlOperatorNameGeneration.h"

#include "AnimNode_RigidBodyWithControl.h"
#include "PhysicsControlLimbData.h"
#include "PhysicsControlRecord.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"

#include "ReferenceSkeleton.h"

namespace UE
{
namespace PhysicsControl
{

//======================================================================================================================
FName FindParentBodyBoneName(
	const FName BoneName, const FReferenceSkeleton& RefSkeleton, const UPhysicsAsset* PhysicsAsset)
{
	FName ParentBoneName;

	if (PhysicsAsset)
	{
		const int32 StartBoneIndex = RefSkeleton.FindBoneIndex(BoneName);

		if (StartBoneIndex != INDEX_NONE)
		{
			const int32 ParentBodyIndex = PhysicsAsset->FindParentBodyIndex(RefSkeleton, StartBoneIndex);

			if (PhysicsAsset->SkeletalBodySetups.IsValidIndex(ParentBodyIndex) && 
				PhysicsAsset->SkeletalBodySetups[ParentBodyIndex])
			{
				ParentBoneName = PhysicsAsset->SkeletalBodySetups[ParentBodyIndex]->BoneName;
			}
		}
	}

	return ParentBoneName;
}

//======================================================================================================================
template<typename TOperatorFunctorType> void CreateAdditionalBodyModifiers(
	const TMap<FName, FPhysicsBodyModifierCreationData>& CreationSpecifiers, 
	FPhysicsControlNameRecords&                          NameRecords, 
	TOperatorFunctorType&                                OperatorFunctor)
{
	for (TMap<FName, FPhysicsBodyModifierCreationData>::ElementType CreationPair : CreationSpecifiers)
	{
		const FName Name = CreationPair.Key;
		const FPhysicsBodyModifierCreationData& Specifier = CreationPair.Value;
		if (Name.IsNone())
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"CreateAdditionalBodyModifiers: Failed to make body modifier for %ls", 
				*Specifier.Modifier.BoneName.ToString());
		}
		else
		{
			UE_LOGF(LogPhysicsControl, Verbose,
				"Made modifier %ls for %ls",
				*Name.ToString(),
				*Specifier.Modifier.BoneName.ToString());

			NameRecords.AddBodyModifier(Name, Specifier.Sets);
		}
	}
}

//======================================================================================================================
template<typename TOperatorFunctorType> void CreateAdditionalControls(
	const TMap<FName, FPhysicsControlCreationData>& CreationSpecifiers, 
	FPhysicsControlNameRecords&                     NameRecords, 
	TOperatorFunctorType&                           OperatorFunctor)
{
	for (TMap<FName, FPhysicsControlCreationData>::ElementType CreationPair : CreationSpecifiers)
	{
		const FName Name = CreationPair.Key;
		const FPhysicsControlCreationData& Specifier = CreationPair.Value;
		if (Name.IsNone())
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"CreateAdditionalControls: Failed to make control between %ls and %ls",
				*Specifier.Control.ParentBoneName.ToString(), *Specifier.Control.ChildBoneName.ToString());
		}
		else
		{
			UE_LOGF(LogPhysicsControl, Verbose,
				"Made control %ls between %ls and %ls",
				*Name.ToString(),
				*Specifier.Control.ParentBoneName.ToString(),
				*Specifier.Control.ChildBoneName.ToString());
			NameRecords.AddControl(Name, Specifier.Sets);
		}
	}
}

//======================================================================================================================
template<typename TFunctor> void CreateControlsFromLimbBones(
	const FName                     LimbName,
	const FPhysicsControlLimbBones& LimbBones,
	const EPhysicsControlType       ControlType,
	const FReferenceSkeleton&       RefSkeleton,
	const UPhysicsAsset*            PhysicsAsset,
	const FPhysicsControlData&      ControlData,
	FPhysicsControlNameRecords&     NameRecords,
	TFunctor&                       CreateOperationLambda)
{
	for (const FName ChildBoneName : LimbBones.BoneNames)
	{
		FName ParentBoneName;

		if (ControlType == EPhysicsControlType::ParentSpace)
		{
			ParentBoneName = FindParentBodyBoneName(ChildBoneName, RefSkeleton, PhysicsAsset);

			if (ParentBoneName.IsNone())
			{
				// This happens for the pelvis, for example - we only create parent space
				// controls if there's a parent!
				continue;
			}
		}

		const FName ControlName = UE::PhysicsControl::GetUniqueControlName(
			ParentBoneName, ChildBoneName, NameRecords.GetControlNamesInSet(TEXT("All")), TEXT(""));
		if (ControlName.IsNone())
		{
			UE_LOGF(LogPhysicsControl, Warning, "Unable to find a suitable Control name for bones %ls and %ls",
				*ParentBoneName.ToString(), *ChildBoneName.ToString());
			return;
		}

		bool result = CreateOperationLambda(ControlName, ParentBoneName, ChildBoneName, ControlData);

		if (result)
		{
			NameRecords.AddControl(ControlName, LimbName);
			NameRecords.AddControl(ControlName, GetPhysicsControlTypeName(ControlType));
			NameRecords.AddControl(ControlName, FName(
				GetPhysicsControlTypeName(ControlType).ToString().Append("_").Append(LimbName.ToString())));
		}
		else
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"Failed to create control for %ls", *ChildBoneName.ToString());
		}
	}
}

//======================================================================================================================
template<typename TFunctor> void CreateBodyModifiersFromLimbBones(
	const FName                        LimbName,
	const FPhysicsControlLimbBones&    LimbBones,
	const FPhysicsControlModifierData& ModifierData,
	FPhysicsControlNameRecords&        NameRecords,
	TFunctor&                          CreateOperationLambda)
{
	for (const FName BoneName : LimbBones.BoneNames)
	{
		const FName BodyModifierName = UE::PhysicsControl::GetUniqueBodyModifierName(
			BoneName, NameRecords.GetBodyModifierNamesInSet(TEXT("All")), TEXT(""));
		if (BodyModifierName.IsNone())
		{
			UE_LOGF(LogPhysicsControl, Warning, "Unable to find a suitable Body Modifier name for bone %ls",
				*BoneName.ToString());
			return;
		}

		bool result = CreateOperationLambda(BodyModifierName, BoneName, ModifierData);

		if (result)
		{
			NameRecords.AddBodyModifier(BodyModifierName, LimbName);
		}
		else
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"Failed to create body modifier %ls", *BodyModifierName.ToString());
		}
	}
}

//======================================================================================================================
template<typename TControlFunctorType, typename TBodyModifierFunctorType>
void ForEachPotentialOperator(
	const FPhysicsControlCharacterSetupData&           CharacterSetupData,
	const FPhysicsControlAndBodyModifierCreationDatas& AdditionalControlsAndBodyModifiers,
	const TMap<FName, FPhysicsControlLimbBones>        AllLimbBones, 
	const FReferenceSkeleton&                          RefSkeleton, 
	const UPhysicsAsset*                               PhysicsAsset, 
	FPhysicsControlNameRecords&                        NameRecords, 
	TControlFunctorType&                               ControlFunctor, 
	TBodyModifierFunctorType&                          BodyModifierFunctor)
{
	for (const TMap<FName, FPhysicsControlLimbBones>::ElementType& LimbBoneEntry : AllLimbBones)
	{
		const FName LimbName = LimbBoneEntry.Key;
		const FPhysicsControlLimbBones& LimbBones = LimbBoneEntry.Value;
		if (LimbBones.bCreateWorldSpaceControls)
		{
			CreateControlsFromLimbBones(
				LimbName, LimbBones, EPhysicsControlType::WorldSpace, RefSkeleton, PhysicsAsset, 
				CharacterSetupData.DefaultWorldSpaceControlData, NameRecords, ControlFunctor);
		}
		if (LimbBones.bCreateParentSpaceControls)
		{
			CreateControlsFromLimbBones(
				LimbName, LimbBones, EPhysicsControlType::ParentSpace, RefSkeleton, PhysicsAsset, 
				CharacterSetupData.DefaultParentSpaceControlData, NameRecords, ControlFunctor);
		}
		if (LimbBones.bCreateBodyModifiers)
		{
			CreateBodyModifiersFromLimbBones(
				LimbName, LimbBones, CharacterSetupData.DefaultBodyModifierData, NameRecords, BodyModifierFunctor);
		}
	}

	// Find names for any additional controls that have been requested	
	CreateAdditionalBodyModifiers(AdditionalControlsAndBodyModifiers.Modifiers, NameRecords, BodyModifierFunctor);
	CreateAdditionalControls(AdditionalControlsAndBodyModifiers.Controls, NameRecords, ControlFunctor);
}

//======================================================================================================================
// Slightly annoying to have to add the names individually, but we want to check they exist
template<typename TBodyModifierNameContainerType, typename TControlNameContainerType>
void CreateAdditionalSets_Implementation(
	const FPhysicsControlSetUpdates&      AdditionalSets, 
	const TBodyModifierNameContainerType& BodyModifierNames, 
	const TControlNameContainerType&      ControlNames, 
	FPhysicsControlNameRecords&           NameRecords)
{
	for (const FPhysicsControlSetUpdate& Set : AdditionalSets.ControlSetUpdates)
	{
		TArray<FName> Names = ExpandNames(Set.Names, NameRecords.ControlSets);

		for (FName Name : Names)
		{
			if (ControlNames.Contains(Name))
			{
				NameRecords.AddControl(Name, Set.SetName);
			}
			else
			{
				UE_LOGF(LogPhysicsControl, Warning,
					"CreateAdditionalSets: Failed to find control with name %ls to add to set %ls",
					*Name.ToString(), *Set.SetName.ToString());
			}
		}
	}

	for (const FPhysicsControlSetUpdate& Set : AdditionalSets.ModifierSetUpdates)
	{
		TArray<FName> Names = ExpandNames(Set.Names, NameRecords.BodyModifierSets);

		for (FName Name : Names)
		{
			if (BodyModifierNames.Contains(Name))
			{
				NameRecords.AddBodyModifier(Name, Set.SetName);
			}
			else
			{
				UE_LOGF(LogPhysicsControl, Warning,
					"CreateAdditionalSets: Failed to find body modifier with name %ls to add to set %ls",
					*Name.ToString(), *Set.SetName.ToString());
			}
		}
	}
}

//======================================================================================================================
// Note - Output limb bones are not in the order specified in the skeleton - would be better if they were
TMap<FName, FPhysicsControlLimbBones> GetLimbBones(
	const TArray<FPhysicsControlLimbSetupData>& LimbSetupDatas,
	const FReferenceSkeleton&                   RefSkeleton,
	const UPhysicsAsset*                        PhysicsAsset)
{
	TMap<FName, FPhysicsControlLimbBones> Result;

	if (!PhysicsAsset)
	{
		UE_LOGF(LogPhysicsControl, Warning, "Physics asset missing");
		return Result;
	}

	TSet<FName> AllBones;

	for (const FPhysicsControlLimbSetupData& LimbSetup : LimbSetupDatas)
	{
		FPhysicsControlLimbBones& LimbBones = Result.Add(LimbSetup.LimbName);

		LimbBones.bFirstBoneIsAdditional = false;
		LimbBones.bCreateWorldSpaceControls = LimbSetup.bCreateWorldSpaceControls;
		LimbBones.bCreateParentSpaceControls = LimbSetup.bCreateParentSpaceControls;
		LimbBones.bCreateBodyModifiers = LimbSetup.bCreateBodyModifiers;

		if (LimbSetup.bIncludeParentBone)
		{
			const FName ParentBoneName = FindParentBodyBoneName(LimbSetup.StartBone, RefSkeleton, PhysicsAsset);

			if (!ParentBoneName.IsNone())
			{
				LimbBones.BoneNames.Add(ParentBoneName);
				AllBones.Add(ParentBoneName);
				LimbBones.bFirstBoneIsAdditional = true;
			}
		}

		TArray<int32> ChildBodyIndices;
		PhysicsAsset->GetBodyIndicesBelow(ChildBodyIndices, LimbSetup.StartBone, RefSkeleton);

		for (int32 ChildBodyIndex : ChildBodyIndices)
		{
			const FName BoneName = PhysicsAsset->SkeletalBodySetups[ChildBodyIndex]->BoneName;

			if (!AllBones.Find(BoneName))
			{
				LimbBones.BoneNames.Add(BoneName);
				AllBones.Add(BoneName);
			}
		}
	}

	return Result;
}

//======================================================================================================================
void CollectOperatorNames(
	const FPhysicsControlCharacterSetupData&           CharacterSetupData,
	const FPhysicsControlAndBodyModifierCreationDatas& AdditionalControlsAndBodyModifiers,
	const TMap<FName, FPhysicsControlLimbBones>        AllLimbBones,
	const FReferenceSkeleton&                          RefSkeleton, 
	const UPhysicsAsset*                               PhysicsAsset, 
	TSet<FName>&                                       BodyModifierNames, 
	TSet<FName>&                                       ControlNames, 
	FPhysicsControlNameRecords&                        NameRecords)
{
	auto CollectControlName = [&ControlNames](
		const FName ControlName, const FName ParentBoneName, 
		const FName ChildBoneName, const FPhysicsControlData& Data) -> bool 
		{
			ControlNames.Add(ControlName); 
			return true; 
		};
	auto CollectBodyModifierName = [&BodyModifierNames](
		const FName BodyModifierName, const FName BoneName, const FPhysicsControlModifierData& Data) -> bool
		{
			BodyModifierNames.Add(BodyModifierName);
			return true;
		};

	ForEachPotentialOperator(
		CharacterSetupData, AdditionalControlsAndBodyModifiers, AllLimbBones, RefSkeleton,
		PhysicsAsset, NameRecords, CollectControlName, CollectBodyModifierName);
}

//======================================================================================================================
void CreateOperatorsForNode(
	FAnimNode_RigidBodyWithControl*                  Node,
	const FPhysicsControlCharacterSetupData&         CharacterSetupData,
	const FPhysicsControlAndBodyModifierCreationDatas& AdditionalControlsAndBodyModifiers,
	const TMap<FName, FPhysicsControlLimbBones>      AllLimbBones,
	const FReferenceSkeleton&                        RefSkeleton, 
	const UPhysicsAsset*                             PhysicsAsset, 
	FPhysicsControlNameRecords&                      NameRecords)
{
	auto CreateControl = [Node](
		const FName ControlName, const FName ParentBoneName, 
		const FName ChildBoneName, const FPhysicsControlData& Data) -> bool
		{ 
			return Node->CreateNamedControl(ControlName, ParentBoneName, ChildBoneName, Data); 
		};
	auto CreateBodyModifier = [Node](
		const FName BodyModifierName, const FName BoneName, const FPhysicsControlModifierData& Data) -> bool
		{ 
			return Node->CreateNamedBodyModifier(BodyModifierName, BoneName, Data); 
		};

	ForEachPotentialOperator(
		CharacterSetupData, AdditionalControlsAndBodyModifiers, AllLimbBones, RefSkeleton,
		PhysicsAsset, NameRecords, CreateControl, CreateBodyModifier);
}

//======================================================================================================================
void CreateAdditionalSets(
	const FPhysicsControlSetUpdates& AdditionalSets, 
	const TSet<FName>&               BodyModifierNames, 
	const TSet<FName>&               ControlNames, 
	FPhysicsControlNameRecords&      NameRecords)
{
	CreateAdditionalSets_Implementation(AdditionalSets, BodyModifierNames, ControlNames, NameRecords);
}

//======================================================================================================================
void CreateAdditionalSets(
	const FPhysicsControlSetUpdates&             AdditionalSets, 
	const TMap<FName, FRigidBodyModifierRecord>& BodyModifierRecords, 
	const TMap<FName, FRigidBodyControlRecord>&  Controls, 
	FPhysicsControlNameRecords&                  NameRecords)
{
	CreateAdditionalSets_Implementation(AdditionalSets, BodyModifierRecords, Controls, NameRecords);
}

//======================================================================================================================
void CreateAdditionalSets(
	const FPhysicsControlSetUpdates&          AdditionalSets,
	const TMap<FName, FPhysicsBodyModifierRecord>&  BodyModifierRecords,
	const TMap<FName, FPhysicsControlRecord>& Controls,
	FPhysicsControlNameRecords&               NameRecords)
{
	CreateAdditionalSets_Implementation(AdditionalSets, BodyModifierRecords, Controls, NameRecords);
}

} // namespace PhysicsControl
} // namespace UE
