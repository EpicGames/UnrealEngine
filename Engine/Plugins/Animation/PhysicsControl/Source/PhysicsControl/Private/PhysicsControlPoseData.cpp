// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlPoseData.h"
#include "Components/SkeletalMeshComponent.h"

namespace UE
{
namespace PhysicsControl
{

//======================================================================================================================
bool DetectTeleport(
	const FVector& OldPosition, const FQuat& OldOrientation,
	const FVector& NewPosition, const FQuat& NewOrientation,
	float DistanceThreshold, float RotationThreshold)
{
	if (DistanceThreshold > 0)
	{
		const double Distance = FVector::Distance(OldPosition, NewPosition);
		if (Distance > DistanceThreshold)
		{
			return true;
		}
	}
	if (RotationThreshold > 0)
	{
		const double Radians = OldOrientation.AngularDistance(NewOrientation);
		if (FMath::RadiansToDegrees(Radians) > RotationThreshold)
		{
			return true;
		}
	}
	return false;
}

//======================================================================================================================
bool DetectTeleport(const FTransform& OldTM, const FTransform& NewTM,
	float DistanceThreshold, float RotationThreshold)
{
	return DetectTeleport(OldTM.GetTranslation(), OldTM.GetRotation(), NewTM.GetTranslation(), NewTM.GetRotation(),
		DistanceThreshold, RotationThreshold);
}

//======================================================================================================================
void FPhysicsControlPoseData::Update(
	USkeletalMeshComponent* SkeletalMesh, 
	const float             Dt,
	const float             TeleportDistanceThreshold, 
	const float             TeleportRotationThreshold)
{
	DeltaTime = Dt;
	const FTransform SkeletalMeshComponentTM = SkeletalMesh->GetComponentTransform();
	const TArray<FTransform>& TMs = SkeletalMesh->GetEditableComponentSpaceTransforms();
	const int32 NumTMs = TMs.Num();
	if (NumTMs == BoneDatas.Num())
	{
		for (int32 Index = 0; Index != NumTMs; ++Index)
		{
			BoneDatas[Index].PreviousTM = BoneDatas[Index].CurrentTM;
			FTransform TM = TMs[Index] * SkeletalMeshComponentTM;
			BoneDatas[Index].CurrentTM = TM;
		}
		bHasJustTeleported = DetectTeleport(
			ComponentTM, SkeletalMeshComponentTM, TeleportDistanceThreshold, TeleportRotationThreshold);
	}
	else
	{
		BoneDatas.Empty(NumTMs);
		for (const FTransform& BoneTM : TMs)
		{
			FTransform TM = BoneTM * SkeletalMeshComponentTM;
			BoneDatas.Emplace(TM, TM);
		}
		bHasJustTeleported = true;
	}
	ComponentTM = SkeletalMeshComponentTM;
}

//======================================================================================================================
FVector FBoneData::CalculateLinearVelocity(float Dt) const
{
	if (Dt <= 0)
	{
		return FVector::ZeroVector;
	}

	return (CurrentTM.GetTranslation() - PreviousTM.GetTranslation()) / Dt;
}

//======================================================================================================================
FVector FBoneData::CalculateAngularVelocity(float Dt) const
{
	if (Dt <= 0)
	{
		return FVector::ZeroVector;
	}

	// Note that quats multiply in the opposite order to TMs, and must be in the same hemisphere.
	const FQuat DeltaQ = (CurrentTM.GetRotation() * PreviousTM.GetRotation().Inverse()).GetShortestArcWith(FQuat::Identity);
	return DeltaQ.ToRotationVector() / Dt;
}

} // namespace PhysicsControl
} // namespace UE

