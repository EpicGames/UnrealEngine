// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlRecord.h"
#include "PhysicsControlLog.h"
#include "PhysicsControlHelpers.h"

#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/PhysicsAsset.h"

//======================================================================================================================
void FPhysicsControlRecord::ResetConstraint()
{
	if (ConstraintInstance.IsValid())
	{
		ConstraintInstance->TermConstraint();
	}
	ConstraintInstance.Reset();
}

//======================================================================================================================
FVector FPhysicsControlRecord::GetControlPoint() const
{
	if (PhysicsControl.ControlData.bUseCustomControlPoint)
	{
		return PhysicsControl.ControlData.CustomControlPoint;
	}

	const FBodyInstance* ChildBodyInstance = UE::PhysicsControl::GetBodyInstance(
		ChildComponent.Get(), PhysicsControl.ChildBoneName);

	return ChildBodyInstance ? ChildBodyInstance->GetMassSpaceLocal().GetTranslation() : FVector::ZeroVector;
}

//======================================================================================================================
bool FPhysicsControlRecord::InitConstraint(
	UObject* ConstraintDebugOwner, FName ControlName, bool bWarnAboutInvalidNames)
{
	if (!ConstraintInstance.IsValid())
	{
		ConstraintInstance = MakeShared<FConstraintInstance>();
	}
	check(ConstraintInstance.IsValid());

	FBodyInstance* ParentBody = UE::PhysicsControl::GetBodyInstance(
		ParentComponent.Get(), PhysicsControl.ParentBoneName);
	FBodyInstance* ChildBody = UE::PhysicsControl::GetBodyInstance(
		ChildComponent.Get(), PhysicsControl.ChildBoneName);

	if (ParentComponent.IsValid() && !PhysicsControl.ParentBoneName.IsNone() && !ParentBody)
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"Failed to find expected parent body %ls when making constraint for control %ls",
				*PhysicsControl.ParentBoneName.ToString(),
				*ControlName.ToString());
		}
		return false;
	}
	if (ChildComponent.IsValid() && !PhysicsControl.ChildBoneName.IsNone() && !ChildBody)
	{
		if (bWarnAboutInvalidNames)
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"Failed to find expected child body %ls when making constraint for control %ls",
				*PhysicsControl.ChildBoneName.ToString(),
				*ControlName.ToString());
		}
		return false;
	}

	ConstraintInstance->InitConstraint(ChildBody, ParentBody, 1.0f, ConstraintDebugOwner);
	ConstraintInstance->SetDisableCollision(PhysicsControl.ControlData.bDisableCollision);
	// These things won't change so set them once here
	ConstraintInstance->SetLinearXMotion(ELinearConstraintMotion::LCM_Free);
	ConstraintInstance->SetLinearYMotion(ELinearConstraintMotion::LCM_Free);
	ConstraintInstance->SetLinearZMotion(ELinearConstraintMotion::LCM_Free);
	ConstraintInstance->SetAngularSwing1Motion(EAngularConstraintMotion::ACM_Free);
	ConstraintInstance->SetAngularSwing2Motion(EAngularConstraintMotion::ACM_Free);
	ConstraintInstance->SetAngularTwistMotion(EAngularConstraintMotion::ACM_Free);
	ConstraintInstance->SetAngularDriveMode(EAngularDriveMode::SLERP);

	ConstraintInstance->SetOrientationDriveSLERP(true);
	ConstraintInstance->SetAngularVelocityDriveSLERP(true);
	ConstraintInstance->SetLinearPositionDrive(true, true, true);
	ConstraintInstance->SetLinearVelocityDrive(true, true, true);

	UpdateConstraintControlPoint();

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_PhysicsControl_InitJointConstraint);

		// Resolve and cache the ragdoll joint constraint index for this bone pair once,
		// so each tick we can use GetConstraintInstanceByIndex instead of FindConstraintIndex.
		// JointConstraintPhysAsset is stored alongside so UpdateControls can detect SetPhysicsAsset calls.
		JointConstraintIndex = INDEX_NONE;
		JointConstraintPhysAsset = nullptr;
		if (USkeletalMeshComponent* ChildSKM = Cast<USkeletalMeshComponent>(ChildComponent.Get()))
		{
			// Only look up a joint constraint when parent and child are on the same SKM (or the
			// parent is null/world-space). A cross-SKM control has no shared physics asset, so
			// ParentBoneName would not correspond to a constraint in ChildSKM's physics asset.
			USkeletalMeshComponent* ParentSKM = Cast<USkeletalMeshComponent>(ParentComponent.Get());
			if (ParentSKM == nullptr || ParentSKM == ChildSKM)
			{
				if (UPhysicsAsset* PhysAsset = ChildSKM->GetPhysicsAsset())
				{
					JointConstraintIndex = PhysAsset->FindConstraintIndex(PhysicsControl.ChildBoneName, PhysicsControl.ParentBoneName);
					JointConstraintPhysAsset = PhysAsset;
				}
			}
		}
	}

	return true;
}

//======================================================================================================================
bool FPhysicsControlRecord::RefreshJointConstraintIndex(
	USkeletalMeshComponent* ChildSKM, UPhysicsAsset* PhysAsset)
{
	if (PhysAsset != JointConstraintPhysAsset.Get())
	{
		// Physics asset changed (or was never set): re-resolve. The cross-SKM guard ensures we
		// never look up a constraint in ChildSKM's physics asset using a parent bone that belongs
		// to a different SKM; for world-space or same-SKM controls the parent pointer is null or
		// equal to ChildSKM.
		USkeletalMeshComponent* ParentSKM = Cast<USkeletalMeshComponent>(ParentComponent.Get());
		JointConstraintIndex = (ParentSKM == nullptr || ParentSKM == ChildSKM)
			? PhysAsset->FindConstraintIndex(
				PhysicsControl.ChildBoneName,
				PhysicsControl.ParentBoneName)
			: INDEX_NONE;
		JointConstraintPhysAsset = PhysAsset;
	}
	return JointConstraintIndex != INDEX_NONE;
}

//======================================================================================================================
// Note that, by default, the constraint frames are simply identity. We only modify Frame1, which 
// corresponds to the child frame. Frame2 will always be identity, because we never change it.
void FPhysicsControlRecord::UpdateConstraintControlPoint()
{
	if (ConstraintInstance.IsValid())
	{
		// Constraints are child then parent
		FTransform Frame1 = ConstraintInstance->GetRefFrame(EConstraintFrame::Frame1);
		Frame1.SetTranslation(GetControlPoint());
		ConstraintInstance->SetRefFrame(EConstraintFrame::Frame1, Frame1);
	}
}

//======================================================================================================================
void FPhysicsControlRecord::ResetControlPoint()
{
	PhysicsControl.ControlData.bUseCustomControlPoint = false;
	UpdateConstraintControlPoint();
}

