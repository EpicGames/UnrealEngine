// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigidBodyControlData.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsActorHandle.h"

//======================================================================================================================

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigidBodyControlData)
FRigidBodyControlRecord::FRigidBodyControlRecord(
	const FPhysicsControl& InControl, ImmediatePhysics::FJointHandle* InJointHandle)
	: Control(InControl)
	, ControlData(Control.ControlData)
	, ChildBodyIndex(-1)
	, ParentBodyIndex(-1)
	, JointHandle(InJointHandle)
{
}

//======================================================================================================================
void FRigidBodyControlRecord::ResetCurrent(bool bResetTarget)
{
	ControlData = Control.ControlData;
	ControlMultiplier = Control.ControlMultiplier;

	if (bResetTarget)
	{
		ControlTarget = FRigidBodyControlTarget();
	}
}

//======================================================================================================================
FVector FRigidBodyControlRecord::GetControlPoint(const ImmediatePhysics::FActorHandle* ChildActorHandle) const
{
	if (ControlData.bUseCustomControlPoint)
	{
		return ControlData.CustomControlPoint;
	}
	return ChildActorHandle->GetLocalCoMLocation();
}

//======================================================================================================================
FRigidBodyModifierRecord::FRigidBodyModifierRecord(
	const FPhysicsBodyModifier& InModifier, ImmediatePhysics::FActorHandle* InActorHandle)
	: Modifier(InModifier)
	, ModifierData(Modifier.ModifierData)
	, ActorHandle(InActorHandle)
{
}

//======================================================================================================================
void FRigidBodyModifierRecord::ResetCurrent()
{
	ModifierData = Modifier.ModifierData;
}

