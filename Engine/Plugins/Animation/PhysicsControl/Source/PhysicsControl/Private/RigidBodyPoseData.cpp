// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigidBodyPoseData.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimInstanceProxy.h"

namespace UE
{
namespace PhysicsControl
{

//======================================================================================================================
void FRigidBodyPoseData::Update(
	FComponentSpacePoseContext&    ComponentSpacePoseContext,
	const TArray<FOutputBoneData>& OutputBoneData,
	const ESimulationSpace         SimulationSpace,
	const FBoneReference&          BaseBoneRef,
	const FGraphTraversalCounter&  InUpdateCounter)
{
	ExpectedUpdateCounter = UpdateCounter;
	ExpectedUpdateCounter.Increment();
	UpdateCounter = InUpdateCounter;

	const FTransform CompWorldSpaceTM = ComponentSpacePoseContext.AnimInstanceProxy->GetComponentTransform();
	const FBoneContainer& BoneContainer = ComponentSpacePoseContext.Pose.GetPose().GetBoneContainer();
	const FTransform BaseBoneTM = ComponentSpacePoseContext.Pose.GetComponentSpaceTransform(
		BaseBoneRef.GetCompactPoseIndex(BoneContainer));

	for (const FOutputBoneData& OutputData : OutputBoneData)
	{
		// It is very unusual, but possible that BodyIndex is invalid - in particular that it is too
		// big. This can happen when OutputBoneData has changed in size and we haven't been
		// reinitialized. In this edge case, we could simply refuse to calculate TMs, but there's no
		// harm in simply expanding our cache array and continuing to function. See UE-214162
		const int32 BodyIndex = OutputData.BodyIndex;
		if (BodyIndex >= 0)
		{
			if (BodyIndex >= BoneTMs.Num())
			{
				// This could cause multiple re-allocations if we keep finding a body index that is
				// too big, but the situation will be so rare that it's not a significant problem
				// (and would only happen for one frame).
				BoneTMs.SetNumUninitialized(BodyIndex + 1);
			}
			const FTransform& ComponentSpaceTM = 
				ComponentSpacePoseContext.Pose.GetComponentSpaceTransform(OutputData.CompactPoseBoneIndex);
			const FTransform BodyTM = ConvertCSTransformToSimSpace(
				SimulationSpace, ComponentSpaceTM, CompWorldSpaceTM, BaseBoneTM);
			BoneTMs[BodyIndex] = BodyTM;
		}
	}
}

} // namespace PhysicsControl
} // namespace UE


