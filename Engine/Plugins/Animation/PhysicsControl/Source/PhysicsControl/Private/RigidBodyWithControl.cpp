// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_RigidBodyWithControl.h"

#include "PhysicsControlOperatorNameGeneration.h"
#include "PhysicsControlLog.h"
#include "PhysicsControlAsset.h"
#include "PhysicsControlHelpers.h"

#include "Animation/AnimInstance.h"
#include "Animation/AnimInstanceProxy.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/PBDJointConstraintTypes.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsActorHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsAdapters.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsJointHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsSimulation.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "Chaos/ChaosConstraintSettings.h"
#include "Logging/StructuredLog.h"

DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_InitControlsAndBodyModifiers"), STAT_RigidBodyNodeWithControl_InitControlsAndBodyModifiers, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_LogControlsModifiersAndSets"), STAT_RigidBodyNodeWithControl_LogControlsModifiersAndSets, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_ApplyControlAndModifierUpdatesAndParametersToRecords"), STAT_RigidBodyNodeWithControl_ApplyControlAndModifierUpdatesAndParametersToRecords, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_ApplyControlsAndModifierDatas"), STAT_RigidBodyNodeWithControl_ApplyControlsAndModifierDatas, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_ApplyControlsAndModifiers"), STAT_RigidBodyNodeWithControl_ApplyControlsAndModifiers, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_ApplyForcesAndTorques"), STAT_RigidBodyNodeWithControl_ApplyForcesAndTorques, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_ApplyControls"), STAT_RigidBodyNodeWithControl_ApplyControls, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_ApplyModifiers"), STAT_RigidBodyNodeWithControl_ApplyModifiers, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_ApplyKinematicTargets"), STAT_RigidBodyNodeWithControl_ApplyKinematicTargets, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("RigidBodyNodeWithControl_ApplyCurrentConstraintProfile"), STAT_RigidBodyNodeWithControl_ApplyCurrentConstraintProfile, STATGROUP_Anim);

constexpr int32 ConstraintChildIndex = 0;
constexpr int32 ConstraintParentIndex = 1;

//======================================================================================================================
void SetPhysicsJointEnabled(ImmediatePhysics::FJointHandle& JointHandle, const bool bIsEnabled)
{
	if (ImmediatePhysics::FJointHandle::FChaosConstraintHandle* ConstraintHandle = JointHandle.GetConstraint())
	{
		ConstraintHandle->SetConstraintEnabled(bIsEnabled);

		if (!bIsEnabled)
		{
			// The call above disables the constraint, but if any actor in the simulation is
			// flagged as dirty, it gets re-enabled! See UE-204006 TODO remove this workaround
			// when the bug has been fixed.
			ConstraintHandle->SetDriveParams(
				Chaos::FVec3::ZeroVector, Chaos::FVec3::ZeroVector, Chaos::FVec3::ZeroVector, 
				Chaos::FVec3::ZeroVector, Chaos::FVec3::ZeroVector, Chaos::FVec3::ZeroVector);
		}
	}
}

//======================================================================================================================
static void SetRecordParameters(
	const FName Name, const FPhysicsControlModifierSparseData& Data, TMap<FName, FRigidBodyModifierRecord>& Records)
{
	if (FRigidBodyModifierRecord* const RecordSearchResult = Records.Find(Name))
	{
		RecordSearchResult->ModifierData.UpdateFromSparseData(Data);
	}
	else
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetRecordParameters: Failed to find modifier with name %ls", *Name.ToString());
	}
}

//======================================================================================================================
static void SetRecordParameters(
	const FName Name, const FPhysicsControlSparseData& Data, TMap<FName, FRigidBodyControlRecord>& Records)
{
	if (FRigidBodyControlRecord* const RecordSearchResult = Records.Find(Name))
	{
		RecordSearchResult->ControlData.UpdateFromSparseData(Data);
	}
	else
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetRecordParameters: Failed to find control with name %ls", *Name.ToString());
	}
}

//======================================================================================================================
static void SetRecordParameters(
	const FName Name, const FPhysicsControlSparseMultiplier& Data, TMap<FName, FRigidBodyControlRecord>& Records)
{
	if (FRigidBodyControlRecord* const RecordSearchResult = Records.Find(Name))
	{
		RecordSearchResult->ControlMultiplier.UpdateFromSparseData(Data);
	}
	else
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"SetRecordParameters: Failed to find control with name %ls", *Name.ToString());
	}
}

//======================================================================================================================
template<typename TRecord, typename TParameters> void ApplyControlAndModifierParametersToRecords(
	TMap<FName, TRecord>& Records, const TArray<TParameters>& AllParameters, const TMap<FName, TArray<FName>>& Sets)
{
	for (const TParameters& Parameters : AllParameters)
	{
		// Find the list of control records in the target set.
		const TArray<FName>* const SetSearchResult = Sets.Find(Parameters.Name);

		if (SetSearchResult)
		{
			for (const FName& Name : *SetSearchResult)
			{
				SetRecordParameters(Name, Parameters.Data, Records);
			}
		}
		else
		{
			// No Set found with a matching name - try to find a control with a matching name.
			SetRecordParameters(Parameters.Name, Parameters.Data, Records);
		}
	}
}

//======================================================================================================================
FName MapConstraintsBehaviorTypeToString(const MapConstraintsBehaviorType InType)
{
	if (InType == MapConstraintsBehaviorType::AuthoredSkeleton) { return FName("AuthoredSkeleton"); }
	if (InType == MapConstraintsBehaviorType::DefaultTransform) { return FName("DefaultTransform"); }
	
	return FName("None");
}

//======================================================================================================================
void UpdateBodyFromModifierData(
	ImmediatePhysics::FActorHandle&    ActorHandle,
	ImmediatePhysics::FSimulation&     PhysicsSimulation,
	const FPhysicsControlModifierData& ModifierData,
	const FVector&                     SimSpaceGravity)
{
	if (ModifierData.MovementType != EPhysicsMovementType::Default)
	{
		// Note that there's an early out if there's no change needed, so this should be OK.
		PhysicsSimulation.SetIsKinematic(
			&ActorHandle,
			ModifierData.MovementType != EPhysicsMovementType::Simulated);
	}

	ActorHandle.SetCCDEnabled(ModifierData.bEnableCCD);

	// Note that the actual kinematic targets will be set separately, since they need to be set
	// for all kinematics whether or not they were under a modifier.

	// Scale gravity
	const float GravityMultiplier = ModifierData.GravityMultiplier;
	if (GravityMultiplier != 1 && ActorHandle.IsGravityEnabled())
	{
		const float Mass = (float)ActorHandle.GetMass();
		const FVector AntiGravityForce = SimSpaceGravity * (Mass * (GravityMultiplier - 1.0f));
		ActorHandle.AddForce(AntiGravityForce);
	}

	// Set collision. This has an early out, and doesn't flag actors as dirty etc if nothing has changed.
	PhysicsSimulation.SetHasCollision(
		&ActorHandle, CollisionEnabledHasPhysics(ModifierData.CollisionType));

	// This simply sets a flag - doesn't trigger any work to call it when it hasn't changed.
	ActorHandle.SetUpdateKinematicFromSimulation(ModifierData.bUpdateKinematicFromSimulation);
}

//======================================================================================================================
ImmediatePhysics::FJointHandle* CreatePhysicsJoint(
	ImmediatePhysics::FSimulation&  Simulation,
	ImmediatePhysics::FActorHandle& ChildActorHandle,
	ImmediatePhysics::FActorHandle& ParentActorHandle)
{
	ImmediatePhysics::FJointHandle* JointHandle = nullptr;

	Chaos::FPBDJointSettings Settings;

	Settings.LinearMotionTypes = {
		Chaos::EJointMotionType::Free, Chaos::EJointMotionType::Free, Chaos::EJointMotionType::Free };
	Settings.AngularMotionTypes = {
		Chaos::EJointMotionType::Free, Chaos::EJointMotionType::Free, Chaos::EJointMotionType::Free };

	Settings.bLinearPositionDriveEnabled = { true, true, true };
	Settings.bLinearVelocityDriveEnabled = { true, true, true };
	Settings.LinearDriveForceMode = Chaos::EJointForceMode::Acceleration;

	Settings.bAngularSLerpPositionDriveEnabled = true;
	Settings.bAngularSLerpVelocityDriveEnabled = true;

	Settings.bAngularTwistPositionDriveEnabled = false;
	Settings.bAngularTwistVelocityDriveEnabled = false;
	Settings.bAngularSwingPositionDriveEnabled = false;
	Settings.bAngularSwingVelocityDriveEnabled = false;
	Settings.AngularDriveForceMode = Chaos::EJointForceMode::Acceleration;

	// For control, we shouldn't be in situations where mass conditioning is needed.
	Settings.bMassConditioningEnabled = false;

	Settings.bUseLinearSolver = true;
	// It's not our job to change collision settings - that should come from the physics asset.
	// However, the naming of this is unclear - if collisions are disabled in the physics asset,
	// trust that this doesn't enable them.
	Settings.bCollisionEnabled = true;

	FVector ChildCoMPositionOffset = ChildActorHandle.GetLocalCoMTransform().GetLocation();
	Settings.ConnectorTransforms[ConstraintChildIndex].SetLocation(ChildCoMPositionOffset);

	Settings.Sanitize();
	JointHandle = Simulation.CreateJoint(ImmediatePhysics::FJointSetup(Settings, &ChildActorHandle, &ParentActorHandle));

	if (JointHandle)
	{
		SetPhysicsJointEnabled(*JointHandle, false); // Disable constraints by default.
	}

	return JointHandle;
}

//======================================================================================================================
const FTransform FAnimNode_RigidBodyWithControl::GetBodyTransform(const int32 BodyIndex) const
{
	if (Bodies.IsValidIndex(BodyIndex))
	{
		return Bodies[BodyIndex]->GetWorldTransform();
	}
	return FTransform::Identity;
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::CreateWorldSpaceControlRootBody(UPhysicsAsset* const PhysicsAsset)
{
	// Note that this never moves - it's just defined as being an actor at the root of the world
	WorldSpaceControlActorHandle = Simulation->CreateActor(
		ImmediatePhysics::MakeStaticActorSetup(nullptr, FTransform()));
	if (WorldSpaceControlActorHandle)
	{
		WorldSpaceControlActorHandle->SetName(FName("WorldSpaceControlHandle"));
		Simulation->SetHasCollision(WorldSpaceControlActorHandle, false);
	}
	else
	{
		UE_LOGF(LogPhysicsControl, Error, "Failed to create world space control root actor");
	}
}

//======================================================================================================================
int32 FAnimNode_RigidBodyWithControl::AddBody(ImmediatePhysics::FActorHandle* const BodyHandle)
{
	const int32 BodyIndex = Bodies.Add(BodyHandle);

	if (BodyHandle != nullptr)
	{
		BodyNameToIndexMap.Add(BodyHandle->GetName(), BodyIndex);
	}

	return BodyIndex;
}

//======================================================================================================================
int32 FAnimNode_RigidBodyWithControl::FindBodyIndexFromBoneName(const FName BoneName) const
{
	const int32* const FoundIndex = BodyNameToIndexMap.Find(BoneName);

	return (FoundIndex != nullptr) ? *FoundIndex : INDEX_NONE;
}

//======================================================================================================================
ImmediatePhysics::FActorHandle* FAnimNode_RigidBodyWithControl::FindBodyFromBoneName(const FName BoneName) const
{
	ImmediatePhysics::FActorHandle* BodyHandle = nullptr;
	const int32 BodyIndex = FindBodyIndexFromBoneName(BoneName);
	if (BodyIndex != INDEX_NONE)
	{
		BodyHandle = Bodies[BodyIndex];
	}
	return BodyHandle;
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::UpdateBodyIndicesInControlRecord(FRigidBodyControlRecord& ControlRecord)
{
	ControlRecord.ChildBodyIndex = FindBodyIndexFromBoneName(ControlRecord.Control.ChildBoneName);
	ControlRecord.ParentBodyIndex = ControlRecord.Control.ParentBoneName.IsNone() ?
		-1 : FindBodyIndexFromBoneName(ControlRecord.Control.ParentBoneName);
}

//======================================================================================================================
bool FAnimNode_RigidBodyWithControl::CreateNamedControl(
	const FName                ControlName, 
	const FName                ParentBoneName, 
	const FName                ChildBoneName, 
	const FPhysicsControlData& ControlData)
{
	FPhysicsControl Control;
	Control.ParentBoneName = ParentBoneName;
	Control.ChildBoneName = ChildBoneName;
	Control.ControlData = ControlData;

	ImmediatePhysics::FJointHandle* JointHandle = nullptr;
	ImmediatePhysics::FActorHandle* ParentBodyHandle = nullptr;

	if (ParentBoneName.IsNone())
	{
		// A parent actor is needed. Without it, the constraint doesn't work (though there's no error)
		ParentBodyHandle = WorldSpaceControlActorHandle;
	}
	else
	{
		ParentBodyHandle = FindBodyFromBoneName(ParentBoneName);
	}
	ImmediatePhysics::FActorHandle* const ChildBodyHandle = FindBodyFromBoneName(ChildBoneName);
	if (Simulation && ChildBodyHandle && ParentBodyHandle)
	{
		JointHandle = CreatePhysicsJoint(*Simulation, *ChildBodyHandle, *ParentBodyHandle);
	}

	if (!JointHandle)
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"Unable to create world space control constraint for bone %ls", *ChildBoneName.ToString());
		return false;
	}

	ControlRecords.Add(ControlName, FRigidBodyControlRecord(Control, JointHandle));
	return true;
}


//======================================================================================================================
FName FAnimNode_RigidBodyWithControl::CreateControl(
	const FName ParentBoneName, const FName ChildBoneName, const FPhysicsControlData& ControlData)
{
	FName ControlName = GetUniqueControlName(ParentBoneName, ChildBoneName);
	if (CreateNamedControl(ControlName, ParentBoneName, ChildBoneName, ControlData))
	{
		return ControlName;
	}
	return FName();
}

//======================================================================================================================
FName FAnimNode_RigidBodyWithControl::GetBodyFromBoneName(const FName BoneName) const
{
	if (const FName* Name = BoneToBodyNameMap.Find(BoneName))
	{
		return *Name;
	}
	return BoneName;
}

//======================================================================================================================
FName FAnimNode_RigidBodyWithControl::GetUniqueBodyModifierName(const FName BoneName) const
{
	const FName UniqueName = UE::PhysicsControl::GetUniqueBodyModifierName(
		GetBodyFromBoneName(BoneName), ModifierRecords, TEXT(""));

	if (UniqueName.IsNone())
	{
		UE_LOGF(LogPhysicsControl, Warning, "Unable to find a suitable Body Modifier name");
	}

	return UniqueName;
}

//======================================================================================================================
FName FAnimNode_RigidBodyWithControl::GetUniqueControlName(const FName ParentBoneName, const FName ChildBoneName) const
{
	const FName UniqueName = UE::PhysicsControl::GetUniqueControlName(
		GetBodyFromBoneName(ParentBoneName), GetBodyFromBoneName(ChildBoneName), ControlRecords, TEXT(""));

	if (UniqueName.IsNone())
	{
		UE_LOGF(LogPhysicsControl, Warning, "Unable to find a suitable Control name");
	}

	return UniqueName;
}

//======================================================================================================================
bool FAnimNode_RigidBodyWithControl::CreateNamedBodyModifier(
	const FName ModifierName, const FName BoneName, const FPhysicsControlModifierData& ModifierData)
{
	ImmediatePhysics::FActorHandle* const ActorHandle = FindBodyFromBoneName(BoneName);
	if (ActorHandle)
	{
		FPhysicsBodyModifier BodyModifier(BoneName, ModifierData);
		FRigidBodyModifierRecord& Modifier = ModifierRecords.Add(
			ModifierName, FRigidBodyModifierRecord(BodyModifier, ActorHandle));
		return true;
	}
	return false;
}

//======================================================================================================================
FName FAnimNode_RigidBodyWithControl::CreateBodyModifier(FName BoneName, const FPhysicsControlModifierData& ModifierData)
{
	FName ModifierName = GetUniqueBodyModifierName(BoneName);
	if (CreateNamedBodyModifier(ModifierName, BoneName, ModifierData))
	{
		return ModifierName;
	}
	return FName();
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::InitControlsAndBodyModifiers(const FReferenceSkeleton& RefSkeleton)
{
	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_InitControlsAndBodyModifiers);

	check(ControlRecords.IsEmpty()); // Controls should not exist when this function is called.

	FPhysicsControlCharacterSetupData SetupData;
	if (IsValid(PhysicsControlAsset))
	{
		SetupData = PhysicsControlAsset->CharacterSetupData;
	}
	if (bEnableCharacterSetupData)
	{
		SetupData += CharacterSetupData;
	}

	// These functions will create the base set of controls and modifiers from SetupData
	TMap<FName, FPhysicsControlLimbBones> AllLimbBones =
		UE::PhysicsControl::GetLimbBones(SetupData.LimbSetupData, RefSkeleton, GetPhysicsAsset());

	FPhysicsControlAndBodyModifierCreationDatas AdditionalControlAndBodyModifierCreationDatas;
	if (IsValid(PhysicsControlAsset))
	{
		AdditionalControlAndBodyModifierCreationDatas = PhysicsControlAsset->AdditionalControlsAndModifiers;
	}
	AdditionalControlAndBodyModifierCreationDatas += AdditionalControlsAndBodyModifiers;

	// An "operator" is a control or a body modifier. This will also add them to sets etc.
	UE::PhysicsControl::CreateOperatorsForNode(
		this, SetupData, AdditionalControlAndBodyModifierCreationDatas,
		AllLimbBones, RefSkeleton, GetPhysicsAsset(), NameRecords);

	for (TMap<FName, FRigidBodyControlRecord>::ElementType& NameRecordPair : ControlRecords)
	{
		FRigidBodyControlRecord& ControlRecord = NameRecordPair.Value;
		UpdateBodyIndicesInControlRecord(ControlRecord);
	}

	// Create any additional sets that have been requested
	if (IsValid(PhysicsControlAsset))
	{
		UE::PhysicsControl::CreateAdditionalSets(
			PhysicsControlAsset->AdditionalSets, ModifierRecords, ControlRecords, NameRecords);
	}
	UE::PhysicsControl::CreateAdditionalSets(AdditionalSets, ModifierRecords, ControlRecords, NameRecords);

	// Apply control and modifier parameters on a single-use basis
	ApplyControlAndBodyModifierDatas(
		InitialControlAndBodyModifierUpdates.ControlParameters,
		InitialControlAndBodyModifierUpdates.ControlMultiplierParameters,
		InitialControlAndBodyModifierUpdates.ModifierParameters);

	// Tell the poor user what we've done
	LogControlsModifiersAndSets();
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::DestroyControlsAndBodyModifiers()
{
	// This is needed because deleting the joint handle doesn't actually remove the constraint from
	// the simulation.
	for (TMap<FName, FRigidBodyControlRecord>::ElementType& NameRecordPair : ControlRecords)
	{
		ImmediatePhysics::FJointHandle* JointHandle = NameRecordPair.Value.JointHandle;
		if (JointHandle)
		{
			Simulation->DestroyJoint(JointHandle);
		}
	}

	ControlRecords.Reset();
	ModifierRecords.Reset();
	NameRecords.Reset();
	bHaveSetupControls = false;
	CurrentControlProfile = FName();
}

//======================================================================================================================
// TODO This isn't ideal as it only dumps out the "original" values, not including the updates
void FAnimNode_RigidBodyWithControl::LogControlsModifiersAndSets()
{
#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_LogControlsModifiersAndSets);

#define RBWC_LOG_LEVEL Verbose

	UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "Controls:");
	for (const TMap<FName, FRigidBodyControlRecord>::ElementType& NameRecordPair : ControlRecords)
	{
		UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "  %ls:", *NameRecordPair.Key.ToString());
		const FRigidBodyControlRecord& Record = NameRecordPair.Value;
		UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "    Parent bone: %ls -> Child bone: %ls", 
			*Record.Control.ParentBoneName.ToString(), *Record.Control.ChildBoneName.ToString());
		UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "    Enabled %d",
			Record.IsEnabled());
		UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "    Linear: Strength %f DampingRatio %f ExtraDamping %f",
			Record.Control.ControlData.LinearStrength, 
			Record.Control.ControlData.LinearDampingRatio, 
			Record.Control.ControlData.LinearExtraDamping);
		UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "    Angular: Strength %f DampingRatio %f ExtraDamping %f",
			Record.Control.ControlData.AngularStrength, 
			Record.Control.ControlData.AngularDampingRatio, 
			Record.Control.ControlData.AngularExtraDamping);
	}

	UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "Body Modifiers:");
	for (const TMap<FName, FRigidBodyModifierRecord>::ElementType& NameRecordPair : ModifierRecords)
	{
		UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "  %ls:", *NameRecordPair.Key.ToString());
		const FRigidBodyModifierRecord& Record = NameRecordPair.Value;
		UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "    Bone: %ls Body: %ls",
			*Record.Modifier.BoneName.ToString(), *GetBodyFromBoneName(Record.Modifier.BoneName).ToString());
		UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "    Movement: %ls GravityMultiplier: %f",
			*GetPhysicsMovementTypeName(Record.Modifier.ModifierData.MovementType).ToString(),
			Record.Modifier.ModifierData.GravityMultiplier);
	}

	UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "Control sets:");
	for (const TMap<FName, TArray<FName>>::ElementType& Pair : NameRecords.ControlSets)
	{
		UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "  %ls:", *Pair.Key.ToString());
		const TArray<FName>& Names = Pair.Value;
		for (FName Name : Names)
		{
			UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "    %ls", *Name.ToString());
		}
	}

	UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "Body Modifier sets:");
	for (const TMap<FName, TArray<FName>>::ElementType& Pair : NameRecords.BodyModifierSets)
	{
		UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "  %ls:", *Pair.Key.ToString());
		const TArray<FName>& Names = Pair.Value;
		for (FName Name : Names)
		{
			UE_LOGF(LogPhysicsControl, RBWC_LOG_LEVEL, "    %ls", *Name.ToString());
		}
	}
#undef RBWC_LOG_LEVEL

#endif // !UE_BUILD_TEST && !UE_BUILD_SHIPPING
}

//======================================================================================================================
template<typename TOut>
static void ConvertStrengthToSpringParams(
	TOut& OutSpring, TOut& OutDamping,
	double InStrength, double InDampingRatio, double InExtraDamping)
{
	const double ClampedStrength = FMath::Max(InStrength, 0.0);
	TOut AngularFrequency = TOut(ClampedStrength * UE_DOUBLE_TWO_PI);
	TOut Stiffness = AngularFrequency * AngularFrequency;
	OutSpring = Stiffness;
	OutDamping = TOut(InExtraDamping + 2 * InDampingRatio * AngularFrequency);
}

//======================================================================================================================
bool UpdateDriveSpringDamperSettings(
	ImmediatePhysics::FJointHandle&   JointHandle,
	const Chaos::FPBDJointSettings&   Settings, 
	const FPhysicsControlData&        Data,
	const FPhysicsControlMultiplier&  Multiplier)
{
	Chaos::FPBDJointConstraintHandle* Constraint = JointHandle.GetConstraint();
	if (!Constraint)
	{
		return false;
	}
	float AngularSpring;
	float AngularDamping;
	const float MaxTorque = Data.MaxTorque * Multiplier.MaxTorqueMultiplier;

	FVector LinearSpring;
	FVector LinearDamping;
	const FVector MaxForce = Data.MaxForce * Multiplier.MaxForceMultiplier;

	UE::PhysicsControl::ConvertStrengthToSpringParams(
		AngularSpring, AngularDamping,
		Data.AngularStrength * Multiplier.AngularStrengthMultiplier,
		Data.AngularDampingRatio * Multiplier.AngularDampingRatioMultiplier,
		Data.AngularExtraDamping * Multiplier.AngularExtraDampingMultiplier);
	UE::PhysicsControl::ConvertStrengthToSpringParams(
		LinearSpring, LinearDamping,
		Data.LinearStrength * Multiplier.LinearStrengthMultiplier,
		Data.LinearDampingRatio * Multiplier.LinearDampingRatioMultiplier,
		Data.LinearExtraDamping * Multiplier.LinearExtraDampingMultiplier);

	if (Multiplier.MaxTorqueMultiplier <= 0)
	{
		AngularSpring = 0;
		AngularDamping = 0;
	}
	if (Multiplier.MaxForceMultiplier.X <= 0)
	{
		LinearSpring.X = 0;
		LinearDamping.X = 0;
	}
	if (Multiplier.MaxForceMultiplier.Y <= 0)
	{
		LinearSpring.Y = 0;
		LinearDamping.Y = 0;
	}
	if (Multiplier.MaxForceMultiplier.Z <= 0)
	{
		LinearSpring.Z = 0;
		LinearDamping.Z = 0;
	}

	const Chaos::EJointForceMode DriveForceMode = Data.bUseAccelerationDriveMode ?
		Chaos::EJointForceMode::Acceleration : Chaos::EJointForceMode::Force;
	Constraint->SetLinearDriveForceMode(DriveForceMode);
	Constraint->SetAngularDriveForceMode(DriveForceMode);

	Constraint->SetDriveParams(
		LinearSpring, LinearDamping, MaxForce,
		Chaos::FVec3(AngularSpring), Chaos::FVec3(AngularDamping), Chaos::FVec3(MaxTorque));

	bool bHaveAngular = (AngularSpring + AngularDamping) > 0;
	bool bHaveLinear = (LinearSpring + LinearDamping).GetMax() > 0;
	bool bHaveAnything = bHaveLinear || bHaveAngular;

	SetPhysicsJointEnabled(JointHandle, bHaveAnything);
	return bHaveAnything;
}

//======================================================================================================================
static UE::PhysicsControl::FPosQuat CalculateTargetTM(
	const Chaos::FPBDJointSettings&               JointSettings, 
	const UE::PhysicsControl::FRigidBodyPoseData& PoseData,
	const int32                                   ParentBodyIndex, 
	const int32                                   ChildBodyIndex)
{
	if (!ensure(PoseData.IsValidIndex(ChildBodyIndex)))
	{
		return UE::PhysicsControl::FPosQuat(JointSettings.ConnectorTransforms[ConstraintChildIndex]);
	}
	
	const UE::PhysicsControl::FPosQuat ChildTargetTM =
		PoseData.GetTM(ChildBodyIndex) *
		UE::PhysicsControl::FPosQuat(JointSettings.ConnectorTransforms[ConstraintChildIndex]);
	
	if (ParentBodyIndex >= 0)
	{
		if (!ensure(PoseData.IsValidIndex(ParentBodyIndex)))
		{
			return ChildTargetTM;
		}

		const UE::PhysicsControl::FPosQuat ParentTargetTM =
		PoseData.GetTM(ParentBodyIndex) *
		UE::PhysicsControl::FPosQuat(JointSettings.ConnectorTransforms[ConstraintParentIndex]);
		return ParentTargetTM.Inverse() * ChildTargetTM;
	}
	return ChildTargetTM;
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::ApplyControl(FRigidBodyControlRecord& ControlRecord, float DeltaTime)
{
	using namespace ImmediatePhysics;
	FJointHandle* JointHandle = ControlRecord.JointHandle;

	if (JointHandle)
	{
		Chaos::FPBDJointConstraintHandle* Constraint = JointHandle->GetConstraint();
		if (Constraint)
		{
			if (!PoseData.UpdateCounter.HasEverBeenUpdated() || 
				ControlRecord.ExpectedUpdateCounter.Get() != PoseData.UpdateCounter.Get())
			{
				// If we missed some intermediate updates, then we don't want to use the previous
				// positions etc to calculate velocities. This will mean velocity/damping will be
				// incorrect for one frame, but that's probably OK.
				DeltaTime = 0.0f;
			}

			Constraint->SetCollisionEnabled(!ControlRecord.Control.ControlData.bDisableCollision);
			Constraint->SetParentInvMassScale(ControlRecord.Control.ControlData.bOnlyControlChildObject ? 0 : 1);

			const Chaos::FPBDJointSettings& JointSettings = Constraint->GetSettings();
			if (UpdateDriveSpringDamperSettings(
				*JointHandle, JointSettings, ControlRecord.ControlData, ControlRecord.ControlMultiplier))
			{
				const FActorHandle* ChildActorHandle = JointHandle->GetActorHandles()[ConstraintChildIndex];
				const FActorHandle* ParentActorHandle = JointHandle->GetActorHandles()[ConstraintParentIndex];

				if (ChildActorHandle && ParentActorHandle)
				{

					// TODO
					// - cache settings / previous input parameters to avoid unnecessary repeating
					//   calculations and making physics API calls every update.

					// Update the target point on the child
					Constraint->SetChildConnectorLocation(ControlRecord.GetControlPoint(ChildActorHandle));

					checkSlow(FindBodyIndexFromBoneName(ControlRecord.Control.ChildBoneName) == ControlRecord.ChildBodyIndex);
					checkSlow((ControlRecord.Control.ParentBoneName.IsNone() ? -1 : 
						FindBodyIndexFromBoneName(ControlRecord.Control.ParentBoneName)) == ControlRecord.ParentBodyIndex);

					UE::PhysicsControl::FPosQuat TargetTM(
						ControlRecord.ControlTarget.TargetOrientation, 
						ControlRecord.ControlTarget.TargetPosition);

					if (ControlRecord.ControlData.bUseSkeletalAnimation)
					{
						UE::PhysicsControl::FPosQuat AnimTargetTM = CalculateTargetTM(
							JointSettings, PoseData, ControlRecord.ParentBodyIndex, ControlRecord.ChildBodyIndex);
						TargetTM = AnimTargetTM * TargetTM;
					}

					Constraint->SetLinearDrivePositionTarget(TargetTM.GetTranslation());
					Constraint->SetAngularDrivePositionTarget(TargetTM.GetRotation());

					if ((DeltaTime * ControlRecord.ControlData.LinearTargetVelocityMultiplier) != 0)
					{
						FVector Velocity = 
							(TargetTM.GetTranslation() - ControlRecord.PreviousTargetTM.GetTranslation()) / DeltaTime;
						Constraint->SetLinearDriveVelocityTarget(
							Velocity * ControlRecord.ControlData.LinearTargetVelocityMultiplier);
					}
					else
					{
						Constraint->SetLinearDriveVelocityTarget(Chaos::FVec3(0));
					}

					if ((DeltaTime * ControlRecord.ControlData.AngularTargetVelocityMultiplier) != 0)
					{
						// Note that quats multiply in the opposite order to TMs, and must be in the same hemisphere.
						const FQuat Q = TargetTM.GetRotation();
						FQuat PrevQ = ControlRecord.PreviousTargetTM.GetRotation();
						PrevQ.EnforceShortestArcWith(Q);
						const FQuat DeltaQ = Q * PrevQ.Inverse();
						const FVector AngularVelocity = DeltaQ.ToRotationVector() / DeltaTime;

						Constraint->SetAngularDriveVelocityTarget( 
							AngularVelocity * ControlRecord.ControlData.AngularTargetVelocityMultiplier);
					}
					else
					{
						Constraint->SetAngularDriveVelocityTarget(Chaos::FVec3(0));
					}


					ControlRecord.PreviousTargetTM = TargetTM;
					ControlRecord.ExpectedUpdateCounter = PoseData.UpdateCounter;
					ControlRecord.ExpectedUpdateCounter.Increment();
				}
				else
				{
					// Note that if we don't have any strength, then we don't calculate the targets.
					// However, make sure that we don't apply velocities using the wrong calculation
					// when the strength/damping is increased in the future
				}
			}
		}
	}
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::ApplyModifier(
	const FRigidBodyModifierRecord& BodyModifierRecord, 
	const FVector&                  SimSpaceGravity)
{
	if (BodyModifierRecord.ActorHandle && Simulation)
	{
		UpdateBodyFromModifierData(
			*BodyModifierRecord.ActorHandle, *Simulation, BodyModifierRecord.ModifierData, SimSpaceGravity);
	}
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::ApplyControlAndModifierUpdatesAndParametersToRecords(
	const FPhysicsControlControlAndModifierUpdates&    Updates,
	const FPhysicsControlControlAndModifierParameters& Parameters)
{
	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_ApplyControlAndModifierUpdatesAndParametersToRecords);

	// Apply control and modifier parameters on a single-use basis
	ApplyControlAndBodyModifierDatas(Updates.ControlUpdates, Updates.ControlMultiplierUpdates, Updates.ModifierUpdates);

	// This goes through the records, resetting the update parts.
	// Then the update structures get adjusted based on the parameters.
	// The results don't get applied to the actual constraints yet - that happens in ApplyControlsAndModifiers
	for (TMap<FName, FRigidBodyControlRecord>::ElementType& NameRecordPair : ControlRecords)
	{
		FName ControlName = NameRecordPair.Key;
		if (const FRigidBodyControlTarget* ControlTarget = ControlTargets.Targets.Find(ControlName))
		{
			NameRecordPair.Value.ControlTarget = *ControlTarget;
			NameRecordPair.Value.ResetCurrent(false);
		}
		else
		{
			NameRecordPair.Value.ResetCurrent(true);
		}
	}

	for (TMap<FName, FRigidBodyModifierRecord>::ElementType& NameRecordPair : ModifierRecords)
	{
		NameRecordPair.Value.ResetCurrent();
	}

	::ApplyControlAndModifierParametersToRecords(
		ControlRecords, Parameters.ControlParameters, NameRecords.ControlSets);
	::ApplyControlAndModifierParametersToRecords(
		ControlRecords, Parameters.ControlMultiplierParameters, NameRecords.ControlSets);
	::ApplyControlAndModifierParametersToRecords(
		ModifierRecords, Parameters.ModifierParameters, NameRecords.BodyModifierSets);
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::ApplyControlAndBodyModifierDatas(
	const TArray<FPhysicsControlNamedControlParameters>&           InControlParameters,
	const TArray<FPhysicsControlNamedControlMultiplierParameters>& InControlMultiplierParameters,
	const TArray<FPhysicsControlNamedModifierParameters>&          InModifierParameters)
{
	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_ApplyControlsAndModifierDatas);

	// This updates the "original" controls and modifiers based on the parameters.
	for (const FPhysicsControlNamedControlParameters& ControlParameters : InControlParameters)
	{
		const FPhysicsControlSparseData& ControlData = ControlParameters.Data;
		TArray<FName> Names = ExpandName(ControlParameters.Name, NameRecords.ControlSets);
		for (FName Name : Names)
		{
			if (FRigidBodyControlRecord* ControlRecord = ControlRecords.Find(Name))
			{
				ControlRecord->Control.ControlData.UpdateFromSparseData(ControlData);
			}
			else
			{
				UE_LOGF(LogPhysicsControl, Warning,
					"ApplyControlAndBodyModifierDatas: Failed to find control with name %ls", *Name.ToString());
			}
		}
	}

	for (const FPhysicsControlNamedControlMultiplierParameters& MultiplierParameters : InControlMultiplierParameters)
	{
		const FPhysicsControlSparseMultiplier& MultiplierData = MultiplierParameters.Data;
		TArray<FName> Names = ExpandName(MultiplierParameters.Name, NameRecords.ControlSets);
		for (FName Name : Names)
		{
			if (FRigidBodyControlRecord* ControlRecord = ControlRecords.Find(Name))
			{
				ControlRecord->Control.ControlMultiplier.UpdateFromSparseData(MultiplierData);
			}
			else
			{
				UE_LOGF(LogPhysicsControl, Warning,
					"ApplyControlAndBodyModifierDatas: Failed to find control with name %ls", *Name.ToString());
			}
		}
	}

	for (const FPhysicsControlNamedModifierParameters& ModifierParameters : InModifierParameters)
	{
		const FPhysicsControlModifierSparseData& ModifierData = ModifierParameters.Data;
		TArray<FName> Names = ExpandName(ModifierParameters.Name, NameRecords.BodyModifierSets);
		for (FName Name : Names)
		{
			if (FRigidBodyModifierRecord* ModifierRecord = ModifierRecords.Find(Name))
			{
				ModifierRecord->Modifier.ModifierData.UpdateFromSparseData(ModifierData);
			}
			else
			{
				UE_LOGF(LogPhysicsControl, Warning,
					"ApplyControlAndBodyModifierDatas: Failed to find modifier with name %ls", *Name.ToString());
			}
		}
	}
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::ApplyControlsAndModifiers(const FVector& SimSpaceGravity, float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_ApplyControlsAndModifiers);

	// If we've skipped a frame then we need to avoid doing any velocity calculations. Simplest
	// method is to set DeltaTime to zero.
	{
		if (!PoseData.UpdateCounter.HasEverBeenUpdated() || 
			PoseData.UpdateCounter.Get() != PoseData.ExpectedUpdateCounter.Get())
		{
			DeltaTime = 0.0f;
		}
	}

	if (!PoseData.IsEmpty())
	{
		// Apply Controls.
		{
			SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_ApplyControls);

			for (TMap<FName, FRigidBodyControlRecord>::ElementType& NameRecordPair : ControlRecords)
			{
				FRigidBodyControlRecord& ControlRecord = NameRecordPair.Value;
				if (ControlRecord.IsEnabled())
				{
					ApplyControl(ControlRecord, DeltaTime);
				}
				else
				{
					if (ControlRecord.JointHandle)
					{
						SetPhysicsJointEnabled(*ControlRecord.JointHandle, false);
					}
					// Ensure the control doesn't use a bogus velocity calculation when it is re-enabled
					ControlRecord.ExpectedUpdateCounter.Reset();
				}
			}
		}

		// Apply Body Modifiers.
		{
			SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_ApplyModifiers);
			for (const TMap<FName, FRigidBodyModifierRecord>::ElementType& NameRecordPair : ModifierRecords)
			{
				ApplyModifier(NameRecordPair.Value, SimSpaceGravity);
			}
		}
	}
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::ApplyForceAndTorque(
	const FPhysicsControlForceAndTorqueData& ForceAndTorqueData,
	const FRigidBodyModifierRecord&          ModifierRecord,
	const FTransform&                        ComponentToWorld, 
	const FTransform&                        BaseBoneTM)
{
	if (ModifierRecord.ActorHandle && ModifierRecord.ActorHandle->IsSimulated())
	{
		ImmediatePhysics::FActorHandle* ActorHandle = ModifierRecord.ActorHandle;

		// Need to convert the requested force/torque into simulation space
		FVector LinearSS;
		FVector AngularSS;

		switch (ForceAndTorqueData.ForceAndTorqueSpace)
		{
		case EPhysicsControlSpace::World:
			LinearSS = WorldVectorToSpaceNoScale(
				SimulationSpace, ForceAndTorqueData.Linear, ComponentToWorld, BaseBoneTM);
			AngularSS = WorldVectorToSpaceNoScale(
				SimulationSpace, ForceAndTorqueData.Angular, ComponentToWorld, BaseBoneTM);
			break;
		case EPhysicsControlSpace::Component:
			LinearSS = ConvertCSVectorToSimSpace(
				SimulationSpace, ForceAndTorqueData.Linear, ComponentToWorld, BaseBoneTM);
			AngularSS = ConvertCSVectorToSimSpace
				(SimulationSpace, ForceAndTorqueData.Angular, ComponentToWorld, BaseBoneTM);
			break;
		case EPhysicsControlSpace::Body:
			LinearSS = ActorHandle->GetWorldTransform().GetRotation() * ForceAndTorqueData.Linear;
			AngularSS = ActorHandle->GetWorldTransform().GetRotation() * ForceAndTorqueData.Angular;
			break;
		default:
			LinearSS = FVector::ZeroVector;
			AngularSS = FVector::ZeroVector;
			break;
		}

		// Apply it differently depending on the location
		ImmediatePhysics_Shared::EForceType FT = ConvertForceType(ForceAndTorqueData.Type);
		switch (ForceAndTorqueData.LocationSpace)
		{
		case EPhysicsControlSpace::World:
		{
			FVector SimSpaceLocation = WorldPositionToSpace(
				SimulationSpace, ForceAndTorqueData.Location, ComponentToWorld, BaseBoneTM);
			ActorHandle->AddForceAtLocation(LinearSS, SimSpaceLocation, FT);
		}
		break;
		case EPhysicsControlSpace::Component:
		{
			FVector SimSpaceLocation = ConvertCSPositionToSimSpace(
				SimulationSpace, ForceAndTorqueData.Location, ComponentToWorld, BaseBoneTM);
			ActorHandle->AddForceAtLocation(LinearSS, SimSpaceLocation, FT);
		}
		break;
		case EPhysicsControlSpace::Body:
		{
			FVector SimSpaceLocation = ActorHandle->GetWorldTransform().TransformPositionNoScale(
				ActorHandle->GetLocalCoMLocation() + ForceAndTorqueData.Location);
			ActorHandle->AddForceAtLocation(LinearSS, SimSpaceLocation, FT);
		}
		break;
		default:
			break;
		}
		// Torque doesn't get applied at a location
		ActorHandle->AddTorque(AngularSS, FT);
	}
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::ApplyForcesAndTorques(
	const FTransform& ComponentToWorld, const FTransform& BaseBoneTM)
{
	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_ApplyForcesAndTorques);
	for (TMap<FName, FPhysicsControlForceAndTorqueData>::ElementType& ForceAndTorquePair : ForcesAndTorques.ForcesAndTorques)
	{
		FName Name = ForceAndTorquePair.Key;
		const FPhysicsControlForceAndTorqueData& ForceAndTorqueData = ForceAndTorquePair.Value;

		const TArray<FName>& Names = NameRecords.GetBodyModifierNamesInSet(Name);
		if (Names.IsEmpty())
		{
			if (FRigidBodyModifierRecord* ModifierRecord = ModifierRecords.Find(Name))
			{
				ApplyForceAndTorque(ForceAndTorqueData, *ModifierRecord, ComponentToWorld, BaseBoneTM);
			}
		}
		else
		{
			for (const FName& ModifierName : Names)
			{
				if (FRigidBodyModifierRecord* ModifierRecord = ModifierRecords.Find(ModifierName))
				{
					ApplyForceAndTorque(ForceAndTorqueData, *ModifierRecord, ComponentToWorld, BaseBoneTM);
				}
			}
		}
	}
}

//======================================================================================================================
// Note that this will be called AFTER normal kinematic targets have been set
void FAnimNode_RigidBodyWithControl::ApplyKinematicTargets(
	const FTransform& CompWorldSpaceTM, const FTransform& BaseBoneTM)
{
	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_ApplyKinematicTargets);

	if (PoseData.IsEmpty())
	{
		return;
	}

	// Apply any kinematic targets.
	for (TMap<FName, FRigidBodyKinematicTarget>::ElementType& KinematicTargetPair : KinematicTargets.Targets)
	{
		FName BodyModifierName = KinematicTargetPair.Key;
		FRigidBodyKinematicTarget& Target = KinematicTargetPair.Value;
		FRigidBodyModifierRecord* ModifierRecord = ModifierRecords.Find(BodyModifierName);
		if (!ModifierRecord || !ModifierRecord->ActorHandle)
		{
			continue;
		}

		ImmediatePhysics::FActorHandle* ActorHandle = ModifierRecord->ActorHandle;
		// TODO It might be worth storing this index in a way that doesn't need a lookup
		const int32 BodyIndex = FindBodyIndexFromBoneName(ModifierRecord->Modifier.BoneName);
		if (!ActorHandle->GetIsKinematic() || BodyIndex == INDEX_NONE)
		{
			continue;
		}
		if (!ensure(PoseData.IsValidIndex(BodyIndex)))
		{
			continue;
		}

		// Bone transform in simulation space, and the target offset packaged as a transform.
		const FTransform BoneRelSimTM = PoseData.GetTM(BodyIndex).ToTransform();
		const FTransform OffsetTM(Target.TargetOrientation, Target.TargetPosition);

		FTransform TargetRelSimTM;
		bool bApplyTarget = true;

		switch (ModifierRecord->ModifierData.KinematicTargetSpace)
		{
		case EPhysicsControlKinematicTargetSpace::OffsetInBoneSpace:
		{
			// Offset is in bone-local space; compose into sim space.
			TargetRelSimTM = OffsetTM * BoneRelSimTM;
			break;
		}
		case EPhysicsControlKinematicTargetSpace::OffsetInWorldSpace:
		{
			// Translation offset is along world axes; rotation offset is pre-multiplied in world.
			// Round-trip via world to keep semantics identical to UPhysicsControlComponent::ApplyKinematicTarget.
			const FTransform SimRelWorldTM = SpaceToWorldTransform(SimulationSpace, CompWorldSpaceTM, BaseBoneTM);
			const FTransform BoneRelWorldTM = BoneRelSimTM * SimRelWorldTM;
			FTransform TargetRelWorldTM = BoneRelWorldTM;
			TargetRelWorldTM.AddToTranslation(OffsetTM.GetTranslation());
			TargetRelWorldTM.SetRotation(OffsetTM.GetRotation() * BoneRelWorldTM.GetRotation());
			TargetRelSimTM = TargetRelWorldTM.GetRelativeTransform(SimRelWorldTM);
			break;
		}
		case EPhysicsControlKinematicTargetSpace::OffsetInComponentSpace:
		{
			// Same shape as OffsetInWorldSpace but in component-space axes.
			const FTransform SimRelWorldTM = SpaceToWorldTransform(SimulationSpace, CompWorldSpaceTM, BaseBoneTM);
			const FTransform BoneRelWorldTM = BoneRelSimTM * SimRelWorldTM;
			const FTransform BoneRelCompTM = BoneRelWorldTM.GetRelativeTransform(CompWorldSpaceTM);
			FTransform TargetRelCompTM = BoneRelCompTM;
			TargetRelCompTM.AddToTranslation(OffsetTM.GetTranslation());
			TargetRelCompTM.SetRotation(OffsetTM.GetRotation() * BoneRelCompTM.GetRotation());
			TargetRelSimTM = ConvertCSTransformToSimSpace(SimulationSpace, TargetRelCompTM, CompWorldSpaceTM, BaseBoneTM);
			break;
		}
		case EPhysicsControlKinematicTargetSpace::World:
		{
			// Target is the desired world transform. Convert directly to sim space.
			const FTransform TargetRelCompTM = OffsetTM.GetRelativeTransform(CompWorldSpaceTM);
			TargetRelSimTM = ConvertCSTransformToSimSpace(SimulationSpace, TargetRelCompTM, CompWorldSpaceTM, BaseBoneTM);
			break;
		}
		case EPhysicsControlKinematicTargetSpace::Component:
		{
			// Target is the desired transform in component space.
			TargetRelSimTM = ConvertCSTransformToSimSpace(SimulationSpace, OffsetTM, CompWorldSpaceTM, BaseBoneTM);
			break;
		}
		case EPhysicsControlKinematicTargetSpace::IgnoreTarget:
		{
			// Just track the bone; ignore the supplied target.
			TargetRelSimTM = BoneRelSimTM;
			break;
		}
		default:
			// Unknown space - skip rather than apply a wrong transform.
			UE_LOGF(LogPhysicsControl, Warning,
				"RigidBodyWithControl: unknown kinematic target space (%d) for modifier %ls",
				(int32)ModifierRecord->ModifierData.KinematicTargetSpace, *BodyModifierName.ToString());
			bApplyTarget = false;
			break;
		}

		if (bApplyTarget)
		{
			ActorHandle->SetKinematicTarget(TargetRelSimTM);
		}
	}
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::ApplyCurrentConstraintProfile()
{
	using namespace Chaos;

	SCOPE_CYCLE_COUNTER(STAT_RigidBodyNodeWithControl_ApplyCurrentConstraintProfile);

	// Go through each joint (in the ragdoll that's been created) in turn...
	for (int32 JointIndex = 0; JointIndex != Joints.Num(); ++JointIndex)
	{
		if (ImmediatePhysics::FJointHandle* JointHandle = Joints[JointIndex])
		{
			ImmediatePhysics::FActorHandle* ParentActorHandle = JointHandle->GetActorHandles()[ConstraintParentIndex];
			ImmediatePhysics::FActorHandle* ChildActorHandle = JointHandle->GetActorHandles()[ConstraintChildIndex];

			// We need to associate this with the constraint setup to get the profile. At the moment
			// we have to do a brute force search, because our joints will not necessarily be in the
			// same order. TODO store the map from joint indices to constraint setups in the physics asset.
			FName ParentActorName = ParentActorHandle ? ParentActorHandle->GetName() : FName();
			FName ChildActorName = ChildActorHandle ? ChildActorHandle->GetName() : FName();

			for (UPhysicsConstraintTemplate* ConstraintSetup : PhysicsAssetToUse->ConstraintSetup)
			{
				// All sorts of problems with comparing names
				if (ConstraintSetup->DefaultInstance.GetParentBoneName() == ParentActorName &&
					ConstraintSetup->DefaultInstance.GetChildBoneName() == ChildActorName)
				{
					FPBDJointConstraintHandle* ConstraintHandle = JointHandle->GetConstraint();

					const FConstraintProfileProperties& Profile = 
						ConstraintSetup->GetConstraintProfilePropertiesOrDefault(ConstraintProfile);

					FPBDJointSettings JointSettings = ConstraintHandle->GetSettings();
					
					ImmediatePhysics::UpdateJointSettingsFromConstraintProfile(Profile, JointSettings);

					ConstraintHandle->SetSettings(JointSettings);
				}
			}
		}
	}
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::InvokeControlProfile(FName ControlProfileName)
{
	CurrentControlProfile = ControlProfileName;
	ApplyCurrentControlProfile();
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::ApplyCurrentControlProfile()
{
	// We shouldn't have a hitch here, since the asset (if set) will already have been loaded 
	if (IsValid(PhysicsControlAsset))
	{
		const FPhysicsControlControlAndModifierUpdates* Updates = 
			PhysicsControlAsset->Profiles.Find(CurrentControlProfile);
		if (Updates)
		{
			ApplyControlAndBodyModifierDatas(
				Updates->ControlUpdates, Updates->ControlMultiplierUpdates, Updates->ModifierUpdates);
		}
		else
		{
			UE_LOGF(LogPhysicsControl, Warning,
				"ApplyCurrentControlProfile: Profile %ls not found", *CurrentControlProfile.ToString());
		}
	}
	else
	{
		UE_LOGF(LogPhysicsControl, Warning,
			"ApplyCurrentControlProfile: No control profile asset loaded");
	}
}

//======================================================================================================================
void FAnimNode_RigidBodyWithControl::TransformConstraintsToMatchSkeletalMesh(
	const USkeletalMesh*             SkeletalMeshAsset,
	const MapConstraintsBehaviorType PositionBehavior,
	const MapConstraintsBehaviorType OrientationBehavior,
	TArray<FConstraintInstance*>&    ConstraintInstances)
{
	// Bone1 = Child
	// Bone2 = Parent 

	if (SkeletalMeshAsset != nullptr)
	{
		const bool bAuthoredTransformRequired = 
			((PositionBehavior == MapConstraintsBehaviorType::AuthoredSkeleton) || 
				(OrientationBehavior == MapConstraintsBehaviorType::AuthoredSkeleton)) && 
			(PhysicsAssetAuthoredSkeletalMesh != nullptr) && 
			(SkeletalMeshAsset != PhysicsAssetAuthoredSkeletalMesh);
		const bool bDefaultTransformRequired = 
			(PositionBehavior == MapConstraintsBehaviorType::DefaultTransform) || 
			(OrientationBehavior == MapConstraintsBehaviorType::DefaultTransform);

		if (bAuthoredTransformRequired || bDefaultTransformRequired)
		{
#if !NO_LOGGING
			const FString AuthoredSkeletalMeshName = 
				(PhysicsAssetAuthoredSkeletalMesh) ? PhysicsAssetAuthoredSkeletalMesh->GetName() : FString("UNDEFINED");
			UE_LOGFMT(LogPhysicsControl, Log,
				"Modify Constraint parent transforms to match the current skeleton \"{0}\". Settings: Authored Skeleton {1}, Position set from {2}, Orientation set from {3}",
				SkeletalMeshAsset->GetName(), AuthoredSkeletalMeshName, MapConstraintsBehaviorTypeToString(PositionBehavior), MapConstraintsBehaviorTypeToString(OrientationBehavior));
#endif

			for (FConstraintInstance* ConstraintInstance : ConstraintInstances)
			{
				const FReferenceSkeleton& SkeletalMeshReferenceSkeleton = SkeletalMeshAsset->GetRefSkeleton();
				const FName ChildBoneName = ConstraintInstance->ConstraintBone1;
				const FName ParentBoneName = ConstraintInstance->ConstraintBone2;

				// This function might be overkill, but it handles the case that there are sketal
				// bones missing in the physics hierarchy.
				const FTransform CurrentChildRelParentTM = CalculateRelativeBoneTransform(
					ChildBoneName, ParentBoneName, SkeletalMeshReferenceSkeleton);

				FTransform AuthoredCurrentRefFrame;
				if (bAuthoredTransformRequired)
				{
					const FTransform OriginalChildRelParentTM = CalculateRelativeBoneTransform(
						ChildBoneName, ParentBoneName, 
						PhysicsAssetAuthoredSkeletalMesh->GetRefSkeleton());

					// Find the transform that maps the parent-bone-relative-to-the-child-bone transform
					// in the original skeleton to the parent-bone-relative-to-the-child-bone transform
					// in the current skeleton.
					// Should be equivalent to CurrentChildRelParentTM * OriginalChildRelParentTM.Inverse()
					const FTransform OriginalToCurrentParentRelChildTM =
						CurrentChildRelParentTM.GetRelativeTransform(OriginalChildRelParentTM);

					// Update the constraints transform relative to the parent bone.
					const FTransform OriginalRefFrame = ConstraintInstance->GetRefFrame(EConstraintFrame::Frame2);
					AuthoredCurrentRefFrame = OriginalToCurrentParentRelChildTM * OriginalRefFrame;
				}

#if !NO_LOGGING
				const FTransform LogPreviousConstraintTransformRelParent = ConstraintInstance->GetRefFrame(EConstraintFrame::Frame2);
#endif

				if (PositionBehavior == MapConstraintsBehaviorType::AuthoredSkeleton)
				{
					ConstraintInstance->SetRefPosition(EConstraintFrame::Frame2, AuthoredCurrentRefFrame.GetTranslation());
				}
				else if (PositionBehavior == MapConstraintsBehaviorType::DefaultTransform)
				{
					ConstraintInstance->SetRefPosition(EConstraintFrame::Frame2, CurrentChildRelParentTM.GetTranslation());
				}

				if (OrientationBehavior == MapConstraintsBehaviorType::AuthoredSkeleton)
				{
					ConstraintInstance->SetRefOrientation(EConstraintFrame::Frame2, 
						AuthoredCurrentRefFrame.GetUnitAxis(EAxis::X), AuthoredCurrentRefFrame.GetUnitAxis(EAxis::Y));
				}
				else if (OrientationBehavior == MapConstraintsBehaviorType::DefaultTransform)
				{
					ConstraintInstance->SetRefOrientation(EConstraintFrame::Frame2, 
						CurrentChildRelParentTM.GetUnitAxis(EAxis::X), CurrentChildRelParentTM.GetUnitAxis(EAxis::Y));
				}

#if !NO_LOGGING
				UE_LOGFMT(LogPhysicsControl, Log,
					"Constraint {0} - {1}  - transform relative to parent was {2} now {3}.",
					ChildBoneName.ToString(),
					ParentBoneName.ToString(),
					LogPreviousConstraintTransformRelParent.ToString(),
					ConstraintInstance->GetRefFrame(EConstraintFrame::Frame2).ToString());
#endif
			}
		}
	}
}


