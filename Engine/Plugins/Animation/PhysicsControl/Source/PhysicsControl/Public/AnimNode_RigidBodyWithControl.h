// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "Components/SkeletalMeshComponent.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsDeclares.h"
#include "PhysicsControlLimbData.h"
#include "PhysicsControlNameRecords.h"
#include "PhysicsProxy/PerSolverFieldSystem.h"
#include "RigidBodyControlData.h"
#include "RigidBodyPoseData.h"
#include "Tasks/Task.h"

 // Use the simulation space functions from the RBAN
#include "BoneControllers/AnimNode_RigidBody.h" 

#include "AnimNode_RigidBodyWithControl.generated.h"

#define UE_API PHYSICSCONTROL_API

struct FBodyInstance;
struct FConstraintInstance;

class FEvent;
class UPhysicsControlAsset;
class UPhysicsAsset;

extern TAutoConsoleVariable<int32> CVarEnableRigidBodyNodeWithControl;
extern TAutoConsoleVariable<int32> CVarEnableRigidBodyNodeWithControlSimulation;
extern TAutoConsoleVariable<int32> CVarRigidBodyNodeWithControlLODThreshold;

struct FSimulationSpaceData
{
	FTransform Transform;
	FVector    LinearVel;
	FVector    AngularVel;
	FVector    LinearAcc;
	FVector    AngularAcc;
};

UENUM(BlueprintType)
enum class MapConstraintsBehaviorType : uint8
{
	AuthoredSkeleton UMETA(DisplayName = "From Authored Skeletal Mesh", ToolTip = "Apply any difference in the relative transform of the constraint's parent and child bones between the authored and current skeleton to the constraint's transform relative to the parent bone. This is normally the best option and should effectively keep the joint in the authored location relative to the parent bone."),
	DefaultTransform UMETA(DisplayName = "From Child Bone Transform (default)", ToolTip = "Set constraints transform relative to the parent bone to match the child bone in the current skeleton."),
	None UMETA(DisplayName = "Nothing", ToolTip = "Nothing"),
};

/**
 * A helper function that lets any immediate simulation user update their bodies in the same way that
 * RigidBodyWithControl does.
 */
PHYSICSCONTROL_API void UpdateBodyFromModifierData(
	ImmediatePhysics::FActorHandle&    ActorHandle,
	ImmediatePhysics::FSimulation&     PhysicsSimulation,
	const FPhysicsControlModifierData& ModifierData,
	const FVector&                     SimSpaceGravity);

// Adjusts the constraint spring drive settings to reflect the control data. Also enabled or
// disables the underlying constraint based on whether there is any strength/damping. Returns true
// if there is some control, false if the drive has no effect
PHYSICSCONTROL_API bool UpdateDriveSpringDamperSettings(
	ImmediatePhysics::FJointHandle&  JointHandle,
	const Chaos::FPBDJointSettings&  Settings,
	const FPhysicsControlData&       Data,
	const FPhysicsControlMultiplier& Multiplier);

// Creates a physics joint in the form that's useful for controls.
PHYSICSCONTROL_API ImmediatePhysics::FJointHandle* CreatePhysicsJoint(
	ImmediatePhysics::FSimulation&  Simulation,
	ImmediatePhysics::FActorHandle& ChildActorHandle,
	ImmediatePhysics::FActorHandle& ParentActorHandle);

PHYSICSCONTROL_API void SetPhysicsJointEnabled(
	ImmediatePhysics::FJointHandle& JointHandle, const bool bIsEnabled);

// Deprecated versions of the functions above

UE_DEPRECATED(5.8, "Use the version of UpdateBodyFromModifierData taking references instead instead")
PHYSICSCONTROL_API inline void UpdateBodyFromModifierData(
	ImmediatePhysics::FActorHandle*    ActorHandle,
	ImmediatePhysics::FSimulation*     PhysicsSimulation,
	const FPhysicsControlModifierData& ModifierData,
	const FVector&                     SimSpaceGravity)
{
	if (ActorHandle && PhysicsSimulation)
	{
		UpdateBodyFromModifierData(*ActorHandle, *PhysicsSimulation, ModifierData, SimSpaceGravity);
	}
}

UE_DEPRECATED(5.8, "Use the version of UpdateDriveSpringDamperSettings taking references instead instead")
PHYSICSCONTROL_API inline bool UpdateDriveSpringDamperSettings(
	ImmediatePhysics::FJointHandle* JointHandle,
	const Chaos::FPBDJointSettings& Settings,
	const FPhysicsControlData& Data,
	const FPhysicsControlMultiplier& Multiplier)
{
	if (JointHandle)
	{
		return UpdateDriveSpringDamperSettings(*JointHandle, Settings, Data, Multiplier);
	}
	return false;
}

UE_DEPRECATED(5.8, "Use the version of CreatePhysicsJoint taking references instead instead")
PHYSICSCONTROL_API inline ImmediatePhysics::FJointHandle* CreatePhysicsJoint(
	ImmediatePhysics::FSimulation* Simulation,
	ImmediatePhysics::FActorHandle* ChildActorHandle,
	ImmediatePhysics::FActorHandle* ParentActorHandle)
{
	if (Simulation && ChildActorHandle && ParentActorHandle)
	{
		return CreatePhysicsJoint(*Simulation, *ChildActorHandle, *ParentActorHandle);
	}
	return nullptr;
}

UE_DEPRECATED(5.8, "Use the version of SetPhysicsJointEnabled taking references instead instead")
PHYSICSCONTROL_API inline void SetPhysicsJointEnabled(ImmediatePhysics::FJointHandle* JointHandle, const bool bIsEnabled)
{
	if (JointHandle)
	{
		SetPhysicsJointEnabled(*JointHandle, bIsEnabled);
	}
}

/**
 * Controller that simulates physics based on the physics asset of the skeletal mesh component
 */
USTRUCT()
struct FAnimNode_RigidBodyWithControl : public FAnimNode_SkeletalControlBase
{
	GENERATED_BODY()

	UE_API FAnimNode_RigidBodyWithControl();
	UE_API ~FAnimNode_RigidBodyWithControl();

	// FAnimNode_Base interface
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void UpdateComponentPose_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void EvaluateComponentPose_AnyThread(FComponentSpacePoseContext& Output) override;
	UE_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	UE_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	virtual bool NeedsOnInitializeAnimInstance() const override { return true; }
	UE_API virtual void PreUpdate(const UAnimInstance* InAnimInstance) override;
	UE_API virtual void UpdateInternal(const FAnimationUpdateContext& Context) override;
	virtual bool HasPreUpdate() const override { return true; }
	UE_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	UE_API virtual bool NeedsDynamicReset() const override;
	UE_API virtual void ResetDynamics(ETeleportType InTeleportType) override;
	UE_API virtual int32 GetLODThreshold() const override;
	// End of FAnimNode_SkeletalControlBase interface

	UE_API virtual void AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName = NAME_None);

	// TEMP: Exposed for use in PhAT as a quick way to get drag handles working with Chaos
	virtual ImmediatePhysics::FSimulation* GetSimulation() { return Simulation; }

	/**
	 * Set the override physics asset. This will automatically trigger a physics re-init if the 
	 * override physics asset changes. Users can get access to this in the Animation Blueprint 
	 * via the Animation Node Functions.
	 */
	UE_API void SetOverridePhysicsAsset(UPhysicsAsset* PhysicsAsset);

	/**
	 * Invoke the profile stored in the control profile asset, and sets the current profile. Note that 
	 * this produces permanent changes to the control and modifier values.
	 */
	UE_API void InvokeControlProfile(FName ControlProfileName);

	UPhysicsAsset* GetPhysicsAsset() const { return PhysicsAssetToUse; }

public:
	/**
	 * Physics asset to use. If empty use the skeletal mesh's default physics asset. Note that this can be
	 * changed at runtime using SetOverridePhysicsAsset in the Animation Node Functions
	 */
	UPROPERTY(EditAnywhere, Category = Settings)
	TObjectPtr<UPhysicsAsset> OverridePhysicsAsset;

	/** 
	 * Use the skeletal mesh physics asset as default in case set to True. The Override Physics Asset 
	 * will always have priority over this. 
	 */
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bDefaultToSkeletalMeshPhysicsAsset = true;

private:
	/** 
	 * Get the physics asset candidate to be used while respecting the bDefaultToSkeletalMeshPhysicsAsset 
	 * and the priority to the override physics asset. 
	 */
	UE_API UPhysicsAsset* GetPhysicsAssetToBeUsed(const UAnimInstance* InAnimInstance) const;

	FTransform PreviousCompWorldSpaceTM;
	FTransform CurrentTransform;
	FTransform PreviousTransform;
	
	UPhysicsAsset* PhysicsAssetToUse;
public:

	/** Override gravity*/
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, editcondition = "bOverrideWorldGravity"))
	FVector OverrideWorldGravity;

	/** Applies a uniform external force in world space. This allows for easily faking inertia of movement while still simulating in component space for example */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinShownByDefault))
	FVector ExternalForce;

	/** When using non-world-space sim, this controls how much of the components world-space acceleration is passed on to the local-space simulation. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	FVector ComponentLinearAccScale;

	/** When using non-world-space sim, this applies a 'drag' to the bodies in the local space simulation, based on the components world-space velocity. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	FVector ComponentLinearVelScale;

	/** When using non-world-space sim, this is an overall clamp on acceleration derived from ComponentLinearAccScale and ComponentLinearVelScale, to ensure it is not too large. */
	UPROPERTY(EditAnywhere, Category = Settings)
	FVector	ComponentAppliedLinearAccClamp;

	/**
	 * Settings for the system which passes motion of the simulation's space
	 * into the simulation. This allows the simulation to pass a
	 * fraction of the world space motion onto the bodies which allows Bone-Space
	 * and Component-Space simulations to react to world-space movement in a
	 * controllable way.
	 * This system is a superset of the functionality provided by ComponentLinearAccScale,
	 * ComponentLinearVelScale, and ComponentAppliedLinearAccClamp. In general
	 * you should not have both systems enabled.
	 */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	FSimSpaceSettings SimSpaceSettings;

	/**
	 * Scale of cached bounds (vs. actual bounds) used for obtaining the list of objects we might collide with.
	 * Increasing this may improve performance, but overlaps may not work as well.
	 * (A value of 1.0 effectively disables cached bounds).
	 */
	UPROPERTY(EditAnywhere, Category = CollisionSettings, meta = (ClampMin="1.0", ClampMax="2.0"))
	float CachedBoundsScale;

	/**
	 * This can be used to force the cached collision objects to be updated every frame, even when we are not 
	 * moving. This can be expensive, but may be needed if we wish to detect collisions with objects that are 
	 * moving, otherwise the cache will only be updated if/when we move.
	 */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	bool bUpdateCacheEveryFrame;

	/** Matters if SimulationSpace is BaseBone */
	UPROPERTY(EditAnywhere, Category = Settings)
	FBoneReference BaseBoneRef;

	/** The channel we use to find static geometry to collide with */
	UPROPERTY(EditAnywhere, Category = CollisionSettings, meta = (editcondition = "bEnableWorldGeometry"))
	TEnumAsByte<ECollisionChannel> OverlapChannel;

	/** What space to simulate the bodies in. This affects how velocities are generated */
	UPROPERTY(EditAnywhere, Category = Settings)
	ESimulationSpace SimulationSpace;

	/** 
	 * Whether or not to calculate velocities for world geometry. Note that if the simulation space is
	 * not set to world, then even static objects "should" have velocities calculated. There is a cost 
	 * associated with calculating velocities, but without them there may be more penetration between
	 * the simulated objects and the world. 
	 */
	UPROPERTY(EditAnywhere, Category = CollisionSettings, meta = (editcondition = "bEnableWorldGeometry"))
	uint8 bCalculateVelocitiesForWorldGeometry : 1;

	/** Whether to allow collisions between two bodies joined by a constraint  */
	UPROPERTY(EditAnywhere, Category = CollisionSettings)
	uint8 bForceDisableCollisionBetweenConstraintBodies : 1;

	/** If true, kinematic objects will be added to the simulation at runtime to represent any cloth colliders defined for the parent object. */
	UPROPERTY(EditAnywhere, Category = CollisionSettings)
	uint8 bUseExternalClothCollision : 1;

	/** 
	 * If true, constraints will be made even when both objects are kinematic, in case they are subsequently 
	 * made dynamic. There may be a small performance benefit to disabling this if you know this won't happen. 
	 */
	UPROPERTY(EditAnywhere, Category = Settings)
	uint8 bMakeKinematicConstraints : 1;

private:
	ETeleportType ResetSimulatedTeleportType;

public:
	UPROPERTY(EditAnywhere, Category = CollisionSettings)
	uint8 bEnableWorldGeometry : 1;

	UPROPERTY(EditAnywhere, Category = CollisionSettings)
	uint8 bWorldGeometryCollisionEnabledHasPhysics : 1;
	
	UPROPERTY(EditAnywhere, Category = CollisionSettings)
	uint8 bWorldGeometryCollisionEnabledHasQuery : 1;
	
	UPROPERTY(EditAnywhere, Category = CollisionSettings)
	uint8 bWorldGeometryCollisionEnabledHasProbe : 1;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	uint8 bOverrideWorldGravity : 1;

	/** 
		When simulation starts, transfer previous bone velocities (from animation)
		to make transition into simulation seamless.
	*/
	UPROPERTY(EditAnywhere, Category = Settings, meta=(PinHiddenByDefault))
	uint8 bTransferBoneVelocities : 1;

	/**
		When simulation starts, freeze incoming pose.
		This is useful for ragdolls, when we want the simulation to take over.
		It prevents non simulated bones from animating.
	*/
	UPROPERTY(EditAnywhere, Category = Settings)
	uint8 bFreezeIncomingPoseOnStart : 1;

	/**
	* Configure the way constraint positions can be modified at runtime to match the current Skeletal Mesh.
	*/
	UPROPERTY(EditAnywhere, Category = PhysicsAssetConditioning, meta = (DisplayName = "Set Position Relative to Parent", DisplayAfter = "PhysicsAssetAuthoredSkeletalMesh"))
	MapConstraintsBehaviorType PhysicsAssetConditioningConstraintPosition;
	
	/**
	* Configure the way constraint orientations can be modified at runtime to match the current Skeletal Mesh.
	*/
	UPROPERTY(EditAnywhere, Category = PhysicsAssetConditioning, meta = (DisplayName = "Set Orientation Relative to Parent", DisplayAfter = "PhysicsAssetAuthoredSkeletalMesh"))
	MapConstraintsBehaviorType PhysicsAssetConditioningConstraintOrientation;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	uint8 bModifyConstraintTransformsToMatchSkeleton_DEPRECATED : 1;
#endif


	/**
		For world-space simulations, if the magnitude of the component's 3D scale is less than 
		WorldSpaceMinimumScale, do not update the node.
	*/
	UPROPERTY(EditAnywhere, Category = Settings)
	float WorldSpaceMinimumScale;

	/**
		If the node is not evaluated for this amount of time (seconds), either because a lower LOD was in use 
		for a while or the component was not visible, reset the simulation to the default pose on the next 
		evaluation. Set to 0 to disable time-based reset.
	*/
	UPROPERTY(EditAnywhere, Category = Settings)
	float EvaluationResetTime;

	/**
	 * If false, then controls will not be created. Note that this can be exposed as a pin/bound, and then control 
	 * creation can be deferred to when the value is set to true as the node runs.
	 */
	UPROPERTY(EditAnywhere, Category = ControlSetup, meta = (PinHiddenByDefault))
	bool bEnableControls;

	/** 
	 * Physics control asset to use. This is optional, but if it has been set, then it will be used 
	 * during creation of controls/body modifiers. Note that this can be modified at runtime in the Animation 
	 * Blueprint via the Animation Node functions.
	 */
	UPROPERTY(EditAnywhere, Category = ControlSetup)
	TObjectPtr<UPhysicsControlAsset> PhysicsControlAsset;

	/**
	 * A map of bone names to "body" names, the latter being used to assign names to controls/modifiers. 
	 * This is optional - so if there is no mapping for a bone, then its name will be used directly when 
	 * creating controls. The two main benefits of this are (1) to generate consistently named controls 
	 * even on different skeletons and (2) to make it easier to refer to individual controls, without needing 
	 * to refer to the skeleton.
	 */
	UPROPERTY(EditAnywhere, Category = ControlSetup, meta = (PinHiddenByDefault))
	TMap<FName, FName> BoneToBodyNameMap;

	/**
	 * Setup data for creating the main controls (world- and parent-space) and modifiers, based on splitting the 
	 * skeleton up into limbs.
	 */
	UPROPERTY(EditAnywhere, Category = ControlSetup, meta = (PinHiddenByDefault, editcondition = "bEnableCharacterSetupData"))
	FPhysicsControlCharacterSetupData CharacterSetupData;

	UPROPERTY(EditAnywhere, Category = ControlSetup, meta = (InlineEditConditionToggle))
	bool bEnableCharacterSetupData = true;

	/**
	 * Controls and modifiers that should be created, in addition to those made as part of the limb setup.
	 */
	UPROPERTY(EditAnywhere, Category = ControlSetup, meta = (PinHiddenByDefault))
	FPhysicsControlAndBodyModifierCreationDatas AdditionalControlsAndBodyModifiers;

	/**
	 * Allows additional sets of controls or modifiers to be created, and existing sets to be modified
	 */
	UPROPERTY(EditAnywhere, Category = ControlSetup, meta = (PinHiddenByDefault))
	FPhysicsControlSetUpdates AdditionalSets;

	/**
	 * An initial set of controls that should be applied immediately after setup. This allows individual or 
	 * sets of controls/modifiers etc to be adjusted. Note that these will then be "baked" into the 
	 * controls.
	 */
	UPROPERTY(EditAnywhere, Category = ControlSetup, meta = (PinHiddenByDefault))
	FPhysicsControlControlAndModifierParameters InitialControlAndBodyModifierUpdates;

	/**
	 * Controls that should be applied each frame, and can be expected to change. Note that if these
	 * stop being passed in then the controls and modifiers will return to their normal/original state.
	 */
	UPROPERTY(EditAnywhere, Category = Controls, meta = (PinShownByDefault))
	FPhysicsControlControlAndModifierParameters ControlAndModifierParameters;

	/**
	 * Updates to controls that can be applied. Note that these update the normal/original state.
	 */
	UPROPERTY(EditAnywhere, Category = Controls, meta = (PinShownByDefault))
	FPhysicsControlControlAndModifierUpdates ControlAndModifierUpdates;

	/**
	 * Targets that should be applied to the controls. 
	 */
	UPROPERTY(EditAnywhere, Category = Controls, meta = (PinShownByDefault))
	FRigidBodyControlTargets ControlTargets;

	/**
	 * Targets that should be applied to kinematic bodies that are under the influence of a body modifier. 
	 */
	UPROPERTY(EditAnywhere, Category = Controls, meta = (PinShownByDefault))
	FRigidBodyKinematicTargets KinematicTargets;

	/**
	 * Forces/torques that should be applied to dynamic bodies that are under the influence of a body modifier.
	 */
	UPROPERTY(EditAnywhere, Category = Controls, meta = (PinShownByDefault))
	FRigidBodyForcesAndTorques ForcesAndTorques;

	/** 
	 * These options configure the way constraint transforms can be modified at runtime to match the current Skeletal
	 * Mesh. This can be useful when applying a single RBwC node to assets with different skeletons.
	 *
	 * When the "From authored skeletal mesh" option is selected, the Position and/or Orientation of each constraint
	 * relative to its parent bone will be changed to account for any difference in relative bone transforms 
	 * between that Skeletal Mesh and the current one. This means that if bone positions or orientations have changed
	 * between the authored Skeletal Mesh and the one currently being animated then the constraint positions and
	 * orientations will be updated to match. The authored Skeletal Mesh should normally be set to the same skeletal 
	 * mesh that was used when creating the physics asset. WARNING This option will do nothing if the authored 
	 * skeleton is not defined or cannot be found when the node is initialized at runtime.
	 *
	 * When the "To default" option is selected the Position and/or Orientation of each constraint relative to its 
	 * parent bone will be set to the Position and/or Orientation of the constraint's child bone relative to its parent
	 * bone. This is the same as the default transform applied in the Physics Asset editing tool when a constraint is
	 * created or 'snapped' to the default transform.
	 *
	 * [WARNING]
	 * The "To default" option can produce undesirable results and should be used with caution but may be useful
	 * for simple assets or to minimize sudden changes in bone positions that may arise if the
	 * authored skeleton is unavailable during the early stages of initialization. In most cases its probably more
	 * useful to only set constraint positions to the default.
	 */
	UPROPERTY(EditAnywhere, Category = PhysicsAssetConditioning, meta = (DisplayName = "Authored Skeletal Mesh"))
	TObjectPtr<USkeletalMesh> PhysicsAssetAuthoredSkeletalMesh;

	/**
	 * The constraint profile to use on all the joints in the physics asset. Will be applied when 
	 * the profile name changes.
	 */
	UPROPERTY(EditAnywhere, Category = Controls, meta = (PinHiddenByDefault))
	FName ConstraintProfile;

	/**
	 * The control profile to use. Will be applied when the profile name changes. To force it to be invoked again, 
	 * either change the profile to something else for a frame, or call InvokeControlProfile.
	 */
	UPROPERTY(EditAnywhere, Category = Controls, meta = (PinHiddenByDefault))
	FName ControlProfile;

	/**
	 * Expands the shape bounds to detect potentially active contacts when moving. Increasing this
	 * (typically up to 1) will increase the number of inactive contacts, which will reduce the
	 * likelihood of penetration, but will also increase solver cost.
	 */
	UPROPERTY(EditAnywhere, Category = CollisionSettings, meta = (ClampMin = 0))
	float BoundsVelocityMultiplier = 0.0f;

	/**
	 * The maximum margin added due to movement. Reducing this can prevent excessive numbers of inactive
	 * contacts being generated.
	 */
	UPROPERTY(EditAnywhere, Category = CollisionSettings, meta = (ClampMin = 0))
	float MaxVelocityBoundsExpansion = 0.0f;

	/**
	 * Allows the solver to use the EnableCCD setting on the bodies
	 */
	UPROPERTY(EditAnywhere, Category = CollisionSettings)
	bool bAllowCCD = false;

private:
	uint8 bEnabled : 1;
	uint8 bSimulationStarted : 1;
	uint8 bCheckForBodyTransformInit : 1;
	uint8 bHaveSetupControls : 1;

public:
	UE_API void PostSerialize(const FArchive& Ar);

	UE_API const int32 GetNumBodies() const;

	const FPhysicsControlNameRecords& GetNameRecords() const { return NameRecords; }

private:

	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	UE_API void InitPhysics(const UAnimInstance* InAnimInstance);
	UE_API void UpdateWorldGeometry(const UWorld& World, const USkeletalMeshComponent& SKC);
	UE_API void UpdateWorldForces(const FTransform& ComponentToWorld, const FTransform& BaseBoneTM, const float DeltaSeconds);

public:
	UE_API const FTransform GetBodyTransform(const int32 BodyIndex) const;

	UE_API bool CreateNamedControl(
		const FName ControlName, const FName ParentBoneName, const FName ChildBoneName, const FPhysicsControlData& ControlData);
	UE_API FName CreateControl(const FName ParentBoneName, const FName ChildBoneName, const FPhysicsControlData& ControlData);

	UE_API bool CreateNamedBodyModifier(
		const FName ModifierName, const FName BoneName, const FPhysicsControlModifierData& ModifierData);
	UE_API FName CreateBodyModifier(const FName BoneName, const FPhysicsControlModifierData& ModifierData);

protected:

	// Add a new physics body and update the body name to index map
	UE_API int32 AddBody(ImmediatePhysics::FActorHandle* const BodyHandle); 
	UE_API int32 FindBodyIndexFromBoneName(const FName BoneName) const;
	UE_API ImmediatePhysics::FActorHandle* FindBodyFromBoneName(const FName BoneName) const;
	UE_API void UpdateBodyIndicesInControlRecord(FRigidBodyControlRecord& ControlRecord);

	UE_API ImmediatePhysics::FJointHandle* CreateConstraint(
		ImmediatePhysics::FActorHandle* const ChildBodyHandle, ImmediatePhysics::FActorHandle* const ParentBodyHandle);
	UE_API void CreateWorldSpaceControlRootBody(UPhysicsAsset* const PhysicsAsset);

	UE_API void ApplyControl(FRigidBodyControlRecord& ControlRecord, float DeltaTime);
	UE_API void ApplyModifier(const FRigidBodyModifierRecord& BodyModifierRecord, const FVector& SimSpaceGravity);

	UE_API void ApplyControlsAndModifiers(const FVector& SimSpaceGravity, float DeltaTime);

	UE_API void ApplyForcesAndTorques(
		const FTransform& ComponentToWorld, const FTransform& BaseBoneTM);

	UE_API void ApplyForceAndTorque(
		const FPhysicsControlForceAndTorqueData& ForceAndTorqueData,
		const FRigidBodyModifierRecord& ModifierRecord,
		const FTransform& ComponentToWorld,
		const FTransform& BaseBoneTM);

	// This applies kinematic targets to any that have been set through the body modifiers
	UE_API void ApplyKinematicTargets(const FTransform& CompWorldSpaceTM, const FTransform& BaseBoneTM);

	// This applies the desired constraint profile, if necessary
	UE_API void ApplyCurrentConstraintProfile();

	// This applies the desired control profile, if necessary
	UE_API void ApplyCurrentControlProfile();

	UE_API FName GetBodyFromBoneName(const FName BoneName) const;

	UE_API FName GetUniqueBodyModifierName(const FName BoneName) const;

	UE_API FName GetUniqueControlName(const FName ParentBoneName, const FName ChildBoneName) const;

	UE_API void SetupControls(USkeletalMeshComponent* const SkeletalMeshComponent);

	// This will walk through the skeleton, create controls and body modifiers, and create the sets.
	UE_API void InitControlsAndBodyModifiers(const FReferenceSkeleton& RefSkeleton);

	UE_API void DestroyControlsAndBodyModifiers();

	// Helper to log overything that has been created to help the user
	UE_API void LogControlsModifiersAndSets();

	// Applies the overrides to the underlying controls and modifiers
	UE_API void ApplyControlAndBodyModifierDatas(
		const TArray<FPhysicsControlNamedControlParameters>& InControlParameters,
		const TArray<FPhysicsControlNamedControlMultiplierParameters>& InControlMultiplierParameters,
		const TArray<FPhysicsControlNamedModifierParameters>& InModifierParameters);

	// Applies the parameters to control and modifier records
	UE_API void ApplyControlAndModifierUpdatesAndParametersToRecords(
		const FPhysicsControlControlAndModifierUpdates&    Updates,
		const FPhysicsControlControlAndModifierParameters& Parameters);

	UE_API void InitializeNewBodyTransformsDuringSimulation(
		FComponentSpacePoseContext& Output, const FTransform& ComponentTransform, const FTransform& BaseBoneTM);

	UE_API void InitSimulationSpace(const FTransform& ComponentToWorld, const FTransform& BoneToComponent);

	// Calculate simulation space transform, velocity etc to pass into the solver
	UE_API void CalculateSimulationSpaceMotion(
		const float Dt,
		const FSimSpaceSettings& Settings,
		FVector& SpaceLinearVel,
		FVector& SpaceAngularVel,
		FVector& SpaceLinearAcc,
		FVector& SpaceAngularAcc);

	// Returns true if constraint transforms in the physics asset should be modified to match the current skeleton.
	UE_API bool ShouldApplyPhysicsAssetConditioning() const;

	// Modify Constraint transforms relative to the parent bone to correct for the difference
	// between the Skeleton used to create the Physics asset and the current skeleton.
	UE_API void TransformConstraintsToMatchSkeletalMesh(
		const USkeletalMesh* const SkeletalMeshAsset,
		const MapConstraintsBehaviorType PositionBehavior,
		const MapConstraintsBehaviorType OrientationBehavior,
		TArray<FConstraintInstance*>& ConstraintInstances);

	// Gather cloth collision sources from the supplied Skeletal Mesh and add a kinematic actor
	// representing each one of them to the sim.
	UE_API void CollectClothColliderObjects(const USkeletalMeshComponent* SkeletalMeshComp);
	
	// Remove all cloth collider objects from the sim.
	UE_API void RemoveClothColliderObjects();

	// Update the sim-space transforms of all cloth collider objects.
	UE_API void UpdateClothColliderObjects();

	// Gather nearby world objects and add them to the sim
	UE_API void CollectWorldObjects();

	// Flag invalid world objects to be removed from the sim
	UE_API void ExpireWorldObjects();

	// Remove simulation objects that are flagged as expired
	UE_API void PurgeExpiredWorldObjects();

	// Update sim-space transforms of world objects
	UE_API void UpdateWorldObjects(const FVector& LinearVel, const FVector& AngularVel, const float DeltaSeconds);

	// Advances the simulation by a given timestep
	UE_API void RunPhysicsSimulation(float DeltaSeconds, const FVector& SimSpaceGravity);

	// Waits for the deferred simulation task to complete if it's not already finished
	UE_API void FlushDeferredSimulationTask();

	// Destroy the simulation and free related structures
	UE_API void DestroyPhysicsSimulation();

public:

	/* 
	 * Whether the physics simulation runs synchronously with the node's evaluation or is run in the 
	 * background until the next frame. Note that running with deferred timing can improve performance, 
	 * but there will be a frame's lag in the output (relative to the input animation), and there can
	 * be problems with handling collisions.
	 */
	UPROPERTY(EditAnywhere, Category=Settings, AdvancedDisplay)
	ESimulationTiming SimulationTiming;

private:

	float WorldTimeSeconds;
	float LastEvalTimeSeconds;

	float AccumulatedDeltaTime;
	float AnimPhysicsMinDeltaTime;
	bool bSimulateAnimPhysicsAfterReset;
	/** This should only be used for removing the delegate during termination. Do NOT use this for any per frame work */
	TWeakObjectPtr<USkeletalMeshComponent> SkelMeshCompWeakPtr;

	ImmediatePhysics::FSimulation* Simulation;
	FPhysicsAssetSolverSettings SolverSettings;
	FSolverIterations SolverIterations;	// to be deprecated

	friend class FRigidBodyNodeWithControlSimulationTask;
	UE::Tasks::FTask SimulationTask;

	TMap<FName, int32> BodyNameToIndexMap;
	ImmediatePhysics::FActorHandle* WorldSpaceControlActorHandle;

	struct FBodyAnimData
	{
		FBodyAnimData()
			: TransferedBoneAngularVelocity(ForceInit)
			, TransferedBoneLinearVelocity(ForceInitToZero)
			, bIsSimulated(false)
			, bBodyTransformInitialized(false)
		{}

		FVector TransferedBoneAngularVelocity;
		FVector TransferedBoneLinearVelocity;

		bool bIsSimulated : 1;
		bool bBodyTransformInitialized : 1;
	};

	struct FWorldObject
	{
		FWorldObject() : ActorHandle(nullptr), LastSeenTick(0), bExpired(false) {}
		FWorldObject(
			ImmediatePhysics::FActorHandle*     InActorHandle, 
			int32                               InLastSeenTick, 
			TWeakObjectPtr<UPrimitiveComponent> InComponent)
			: Component(InComponent), ActorHandle(InActorHandle), LastSeenTick(InLastSeenTick), bExpired(false){}

		TWeakObjectPtr<UPrimitiveComponent> Component;
		ImmediatePhysics::FActorHandle* ActorHandle;
		int32 LastSeenTick;
		bool bExpired;
	};

	TArray<UE::PhysicsControl::FOutputBoneData> OutputBoneData;
	// Note that the Bodies and Joints arrays will be the same size - i.e. there will be a
	// correspondence between every joint and body. The joint will correspond to the parent of the
	// body. Note that some joints will not be set.
	TArray<ImmediatePhysics::FActorHandle*> Bodies;
	TArray<ImmediatePhysics::FJointHandle*> Joints;
	TArray<int32> SkeletonBoneIndexToBodyIndex;
	TArray<FBodyAnimData> BodyAnimData;

	friend class UAnimGraphNode_RigidBodyWithControl;

	// Each update we cache the incoming pose transforms in whatever space the simulation is running in
	UE::PhysicsControl::FRigidBodyPoseData PoseData;

	// Map of control records - they will be referenced by name
	TMap<FName, FRigidBodyControlRecord> ControlRecords;

	// Map of body modifier records - they will be referenced by name
	TMap<FName, FRigidBodyModifierRecord> ModifierRecords;

	// Details about sets etc
	FPhysicsControlNameRecords NameRecords;

	FName CurrentConstraintProfile;

	FName CurrentControlProfile;

	TArray<USkeletalMeshComponent::FPendingRadialForces> PendingRadialForces;

	FPerSolverFieldSystem PerSolverField;

	// Information required to identify and update a kinematic object representing a cloth collision source in the sim.
	struct FClothCollider
	{
		FClothCollider(
			ImmediatePhysics::FActorHandle* const InActorHandle, 
			const USkeletalMeshComponent* const   InSkeletalMeshComponent, 
			const uint32                          InBoneIndex)
			: ActorHandle(InActorHandle)
			, SkeletalMeshComponent(InSkeletalMeshComponent)
			, BoneIndex(InBoneIndex)
		{}

		ImmediatePhysics::FActorHandle* ActorHandle; // Identifies the physics actor in the sim.
		TWeakObjectPtr<const USkeletalMeshComponent> SkeletalMeshComponent; // Parent skeleton.
		uint32 BoneIndex; // Bone within parent skeleton that drives physics actors transform.
	};

	// List of cloth colliders in the sim that represent objects collected from other parts of this character.
	TArray<FClothCollider> ClothColliders; 
	
	// List of actors in the sim that represent objects collected from the world. Note that the key
	// should never be dereferenced, as it is not a smart pointer, and might have been deleted.
	// Instead it should just be used for comparisons, and the actual pointer to the component is
	// stored in the WorldObject.
	TMap<const UPrimitiveComponent*, FWorldObject> ComponentsInSim;
	int32 ComponentsInSimTick;

	FVector WorldSpaceGravity;

	float TotalMass;

	// Bounds used to gather world objects copied into the simulation
	FSphere CachedBounds;

	FCollisionQueryParams QueryParams;

	// Used by CollectWorldObjects and UpdateWorldGeometry in the Task Thread. Typically, World
	// should never be accessed off the Game Thread. However, since we're just doing overlaps this
	// should be OK.
	const UWorld* UnsafeWorld;

	// Used by CollectWorldObjects and UpdateWorldGeometry in Task Thread. Only used for a pointer comparison.
	const AActor* UnsafeOwner;

	// The Skeletal Mesh parsed during the last physics initialization
	const USkeletalMesh* SkeletalMeshAtLastPhysicsInit = nullptr;

	FBoneContainer CapturedBoneVelocityBoneContainer;
	FCSPose<FCompactHeapPose> CapturedBoneVelocityPose;
	FCSPose<FCompactHeapPose> CapturedFrozenPose;
	FBlendedHeapCurve CapturedFrozenCurves;

	FVector PreviousComponentLinearVelocity;

	// Used by the world-space to simulation-space motion transfer system in Component- or Bone-Space sims
	FTransform SimulationSpaceTransform;
	FTransform PreviousSimulationSpaceTransform;
	FTransform PreviousPreviousSimulationSpaceTransform;
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FAnimNode_RigidBodyWithControl> : public TStructOpsTypeTraitsBase2<FAnimNode_RigidBodyWithControl>
{
	enum
	{
		WithPostSerialize = true,
	};
};
#endif

#undef UE_API
