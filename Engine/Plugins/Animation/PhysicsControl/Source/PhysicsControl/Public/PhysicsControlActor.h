// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "PhysicsControlData.h"
#include "PhysicsControlActor.generated.h"

class UPhysicsControlComponent;
struct FPhysicsControl;
struct FPhysicsControlLimbSetupData;

// Structure that determines a Physics Control used during initialization of the physics control actor
USTRUCT(BlueprintType)
struct FInitialPhysicsControl
{
	GENERATED_BODY()

	FInitialPhysicsControl() {}

	// The owner of the mesh that will be doing the driving. Blank/non-existent means it will happen in world space
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TWeakObjectPtr<AActor> ParentActor;

	// The mesh that will be doing the driving. If this is blank but there is an actor, then we'll attempt to
	// use the root component. If that doesn't work then it will happen in world space.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName ParentMeshComponentName;

	// If the parent mesh component is skeletal, then the name of the skeletal mesh bone that will be doing
	// the driving.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName ParentBoneName;

	// The owner of the mesh that the control will be driving
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TWeakObjectPtr<AActor> ChildActor;

	// The mesh that the control will be driving. If this is blank but there is an actor, then we'll attempt
	// to use the root component
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName ChildMeshComponentName;

	// If the child mesh component is skeletal, then the name of the skeletal mesh bone that the control
	// will be driving.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName ChildBoneName;

	// Strength and damping parameters. Can be modified at any time, but will sometimes have
	// been set once during initialization
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsControlData ControlData;

	// Multiplier for the ControlData. This will typically be modified dynamically, and also expose the ability
	// to set directional strengths
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsControlMultiplier ControlMultiplier;

	// The position/orientation etc targets for the controls. These are procedural/explicit control targets -
	// skeletal meshes have the option to use skeletal animation as well, in which case these targets are
	// expressed as relative to that animation.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsControlTarget ControlTarget;
};

// Structure that determines a Body Modifier used during initialization of the physics control actor
USTRUCT(BlueprintType)
struct FInitialBodyModifier
{
	GENERATED_BODY()

	FInitialBodyModifier() {}

	// The owner of the mesh that that we will modify
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TWeakObjectPtr<AActor> Actor;

	// The mesh that will be modify. If this is blank but there is an actor, then we'll attempt to
	// use the root component.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName MeshComponentName;

	// If the mesh component is skeletal, then the name of the skeletal mesh bone to modify
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName BoneName;

	// How the body should move etc
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsControlModifierData BodyModifierData;

	// The target position when kinematic. Note that this is applied on top of any animation
	// target if bUseSkeletalAnimation is set.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FVector KinematicTargetPosition = FVector::ZeroVector;

	// The target orientation when kinematic. Note that this is applied on top of any animation
	// target if bUseSkeletalAnimation is set.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FRotator KinematicTargetOrientation = FRotator::ZeroRotator;
};

// Structure that holds the data necessary to set up a default set of limb controls for a character
USTRUCT(BlueprintType)
struct FInitialCharacterControls
{
	GENERATED_BODY()

	PHYSICSCONTROL_API FInitialCharacterControls();
	PHYSICSCONTROL_API ~FInitialCharacterControls();

	// The owner of the character skeletal mesh
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TWeakObjectPtr<AActor> CharacterActor;

	// The skeletal mesh that will have controls set up. If this is blank but there is an actor, then we'll attempt to
	// use the root component.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName SkeletalMeshComponentName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TArray<FPhysicsControlLimbSetupData> LimbSetupData;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsControlData WorldSpaceControlData;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsControlData ParentSpaceControlData;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsControlModifierData BodyModifierData;
};

//
UCLASS(meta = (BlueprintSpawnableComponent), ClassGroup = Physics)
class UPhysicsControlInitializerComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

public:
	PHYSICSCONTROL_API virtual void BeginPlay() override;

public:
	// This creates all the controls specified in the Initial properties. You can call it explicitly,
	// or you can opt to have it called at BeginPlay using CreateControlsAtBeginPlay.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	PHYSICSCONTROL_API void CreateControls(UPhysicsControlComponent* PhysicsControlComponent);

	// This can be filled in to automatically create whole-character controls (by specifying limbs etc) for
	// a skeletal mesh during the BeginPlay event.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControls)
	FInitialCharacterControls InitialCharacterControls;

	// This can be filled in to automatically create controls during the BeginPlay event. If a control
	// already exists with the name (e.g. created as part of InitialCharacterControls) then it will be updated.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControls)
	TMap<FName, FInitialPhysicsControl> InitialControls;

	// This can be filled in to automatically create body modifiers during the BeginPlay event. If a body modifier
	// already exists with the name (e.g. created as part of InitialCharacterControls) then it will be updated.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControls)
	TMap<FName, FInitialBodyModifier> InitialBodyModifiers;

	// If set, then CreateControls will be called in our BeginPlay event, attempting to find a
	// PhysicsControlComponent in our parent actor.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControls)
	bool bCreateControlsAtBeginPlay = true;

private:
	void CreateInitialCharacterControls(UPhysicsControlComponent* ControlComponent);
	void CreateOrUpdateInitialControls(UPhysicsControlComponent* ControlComponent);
	void CreateOrUpdateInitialBodyModifiers(UPhysicsControlComponent* ControlComponent);
};

// Actor that holds a PhysicsControlComponent which is then used to apply controls/body modifiers to
// other actors from "outside", and a PhysicsControlInitializerComponent which provides the ability
// to create those controls automatically
UCLASS(ConversionRoot, MinimalAPI, ComponentWrapperClass)
class APhysicsControlActor : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(Category = ControlActor, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UPhysicsControlComponent> ControlComponent;

	UPROPERTY(Category = ControlActor, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UPhysicsControlInitializerComponent> ControlInitializerComponent;
};


