// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsControlData.h"
#include "PhysicsControlLimbData.h"

#include "Interfaces/Interface_PreviewMeshProvider.h"

#include "PhysicsControlAsset.generated.h"

#define UE_API PHYSICSCONTROL_API

class USkeletalMesh;

// Asset for storing Physics Control Profiles. The asset will contain data that define:
// - Controls and body modifiers to be created on a mesh
// - Sets referencing those controls and body modifiers
// - Full profiles containing settings for all the controls/modifiers
// - Sparse profiles containing partial sets of settings for specific controls/modifiers
//
// It will also be desirable to support "inheritance" - so a generic profile can be made, and then
// customized for certain characters or scenarios.
UCLASS(MinimalAPI, BlueprintType)
class UPhysicsControlAsset : public UObject, public IInterface_PreviewMeshProvider
{
	GENERATED_BODY()

public:
	UE_API UPhysicsControlAsset();

	// Data that have been compiled from a combination of inherited and "My" data.

	// We can define controls in the form of limbs etc here
	UPROPERTY()
	FPhysicsControlCharacterSetupData CharacterSetupData;

	// Additional controls and modifiers. If these have the same name as one that's already
	// created, they'll just override it.
	UPROPERTY()
	FPhysicsControlAndBodyModifierCreationDatas AdditionalControlsAndModifiers;

	// Additional control and body modifier sets
	UPROPERTY()
	FPhysicsControlSetUpdates AdditionalSets;

	// Initial updates to apply immediately after controls and modifiers are created
	UPROPERTY()
	TArray<FPhysicsControlControlAndModifierUpdates> InitialControlAndModifierUpdates;

	// The named profiles, which are essentially control and modifier updates
	UPROPERTY()
	TMap<FName, FPhysicsControlControlAndModifierUpdates> Profiles;

public:
	// Data that will then be compiled down into the runtime data

#if WITH_EDITORONLY_DATA
	// Whether editing the profiles will automatically compile.
	UPROPERTY(EditAnywhere, Category = ProfileEditing)
	bool bAutoCompileProfiles = true;

	// Whether to automatically invoke profiles that have been edited (and have auto-compiled) when simulating
	UPROPERTY(EditAnywhere, Category = ProfileEditing, Meta = (EditCondition="AutoCompileProfiles"))
	bool bAutoInvokeProfiles = true;

	// Whether editing the setup data will automatically compile.
	UPROPERTY(EditAnywhere, Category = SetupEditing)
	bool bAutoCompileSetup = true;

	// Whether to automatically reinitialize following editing of setup data (when auto-compiling) when simulating
	UPROPERTY(EditAnywhere, Category = SetupEditing, Meta = (EditCondition="AutoCompileSetup"))
	bool bAutoReinitSetup = true;

	// Whether to automatically re-invoke the previously invoked profile after automatically running the setup
	UPROPERTY(EditAnywhere, Category = SetupEditing, Meta = (EditCondition="AutoCompileSetup"))
	bool bAutoInvokeProfileAfterSetup = true;

	// A profile asset to inherit from (can be null). If set, we will just add/modify data in that
	UPROPERTY(EditAnywhere, Category = Inheritance)
	TSoftObjectPtr<UPhysicsControlAsset> ParentAsset;

	// Additional profile assets from which profiles (not the setup data, extra sets etc) will be added
	// to this asset.
	UPROPERTY(EditAnywhere, Category = Inheritance)
	TArray<TSoftObjectPtr<UPhysicsControlAsset>> AdditionalProfileAssets;

	// The PhysicsAsset that this control asset is targeting. This will also determine the preview
	// mesh. You will need to close/re-open the Physics Control Asset editor in order to see the
	// effect of changing it.
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category = PreviewMesh)
	TSoftObjectPtr<UPhysicsAsset> PhysicsAsset;

public:
	// "My" runtime data - i.e. the data that will be combined with what has been inherited
	// We should have custom UI that displays this combined with the inherited data

	// We can define controls in the form of limbs etc here
	UPROPERTY(EditAnywhere, Category = Setup, meta=(DisplayName="Character Setup Data"))
	FPhysicsControlCharacterSetupData MyCharacterSetupData;

	// Additional controls and modifiers. If these have the same name as one that's
	// already created, they'll just override it.
	UPROPERTY(EditAnywhere, Category = Setup, meta = (DisplayName = "Additional Controls and Modifiers"))
	FPhysicsControlAndBodyModifierCreationDatas MyAdditionalControlsAndModifiers;

	// Additional control and body modifier sets
	UPROPERTY(EditAnywhere, Category = Setup, meta = (DisplayName = "Additional Sets"))
	FPhysicsControlSetUpdates MyAdditionalSets;

	// Initial updates to apply immediately after controls and modifiers are created
	UPROPERTY(EditAnywhere, Category = Setup, meta = (DisplayName = "Initial Control and Modifier Updates"))
	TArray<FPhysicsControlControlAndModifierUpdates> MyInitialControlAndModifierUpdates;

	// The named profiles, which are essentially control and modifier updates
	UPROPERTY(EditAnywhere, Category = Profiles, meta = (DisplayName = "Profiles"))
	TMap<FName, FPhysicsControlControlAndModifierUpdates> MyProfiles;
#endif

public:
	// Buttons/actions in the editor

#if WITH_EDITOR
	DECLARE_EVENT_OneParam(FPhysicsControlAssetEditor, FOnControlAssetCompiled, bool);
	FOnControlAssetCompiled& OnControlAssetCompiled() { return OnControlAssetCompiledDelegate; }

	// Shows all the controls etc that would be made
	UFUNCTION(CallInEditor, Category = Actions)
	UE_API void ShowCompiledData() const;

	// Collapses inherited and authored profiles etc to make a profile asset that can be read without
	// need for subsequent processing.
	UFUNCTION(CallInEditor, Category = Actions)
	UE_API void Compile();

	// Returns true if compilation would change any of our compiled data. Note that this is potentially slow as
	// it simply compiles and compares the result with the data we already have.
	UFUNCTION(CallInEditor, Category = Actions)
	UE_API bool IsCompilationNeeded() const;

	// Returns a list of all the profiles that need compilation
	UFUNCTION(CallInEditor, Category = Actions)
	UE_API TArray<FName> GetDirtyProfiles() const;

	// Returns true if the setup data need compilation such that the controls etc need to be re-initialized.
	UFUNCTION(CallInEditor, Category = Actions)
	UE_API bool IsSetupDirty() const;


	// Combines and returns data from our parent and ourself
	UE_API FPhysicsControlCharacterSetupData GetCharacterSetupData() const;

	// Combines and returns data from our parent and ourself
	UE_API FPhysicsControlAndBodyModifierCreationDatas GetAdditionalControlsAndModifiers() const;

	// Combines and returns data from our parent and ourself
	UE_API FPhysicsControlSetUpdates GetAdditionalSets() const;

	// Combines and returns data from our parent and ourself
	UE_API TArray<FPhysicsControlControlAndModifierUpdates> GetInitialControlAndModifierUpdates() const;

	// Combines and returns data from our parent and ourself
	UE_API TMap<FName, FPhysicsControlControlAndModifierUpdates> GetProfiles() const;

protected:
	FOnControlAssetCompiled OnControlAssetCompiledDelegate;

#endif

public:

	// IInterface_PreviewMeshProvider interface
	UE_API virtual void SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty = true) override;
	UE_API virtual USkeletalMesh* GetPreviewMesh() const override;
	// END IInterface_PreviewMeshProvider interface

#if WITH_EDITOR
	// This loads the asset if necessary
	UE_API UPhysicsAsset* GetPhysicsAsset() const;
	UE_API void SetPhysicsAsset(UPhysicsAsset* InPhysicsAsset);

	static UE_API const FName GetPreviewMeshPropertyName();
#endif
};

#undef UE_API
