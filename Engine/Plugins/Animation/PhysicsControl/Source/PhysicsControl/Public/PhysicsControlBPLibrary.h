// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if 1

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "SkeletalControlLibrary.h"

#include "PhysicsControlBPLibrary.generated.h"

struct FRigidBodyControlParameters;
struct FAnimNode_RigidBodyWithControl;

USTRUCT(BlueprintType)
struct FRigidBodyWithControlReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_RigidBodyWithControl FInternalNodeType;
};


UCLASS(MinimalAPI)
class UPhysicsControlBPLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	// Add a single control parameter to a container of parameters.
	UFUNCTION(BlueprintPure, Category = "Animation|PhysicsControl", meta = (BlueprintThreadSafe, AutoCreateRefTerm = "LinearStrengthMultiplier, LinearExtraDampingMultiplier, AngularStrengthMultiplier, AngularExtraDampingMultiplier, bEnabled"))
	static void AddControlParameters(
		UPARAM(ref, DisplayName = "Parameters") FPhysicsControlControlAndModifierParameters& InParameters, 
		UPARAM(DisplayName = "Parameters") FPhysicsControlControlAndModifierParameters&      OutParameters,
		const FName                                                                          Name,
		const FPhysicsControlSparseData&                                                     ControlData);

	// Add an array of control parameters (all with the same data) to a container of parameters.
	UFUNCTION(BlueprintPure, Category = "Animation|PhysicsControl", meta = (BlueprintThreadSafe, AutoCreateRefTerm = "LinearStrengthMultiplier, LinearExtraDampingMultiplier, AngularStrengthMultiplier, AngularExtraDampingMultiplier, bEnabled"))
	static void AddMultipleControlParameters(
		UPARAM(ref, DisplayName = "Parameters") FPhysicsControlControlAndModifierParameters& InParameters, 
		UPARAM(DisplayName = "Parameters") FPhysicsControlControlAndModifierParameters&      OutParameters,
		const TArray<FName>&                                                                 Names,
		const FPhysicsControlSparseData&                                                     ControlData);

	// Add a single body modifier parameter to a container of parameters.
	UFUNCTION(BlueprintPure, Category = "Animation|PhysicsControl", meta = (BlueprintThreadSafe))
	static void AddModifierParameters(
		UPARAM(ref, DisplayName = "Parameters") FPhysicsControlControlAndModifierParameters& InParameters,
		UPARAM(DisplayName = "Parameters") FPhysicsControlControlAndModifierParameters&      OutParameters,
		const FName                                                                          Name, 
		const FPhysicsControlModifierSparseData&                                             ModifierData);

	// Add an array of body modifier parameters to a container of parameter.
	UFUNCTION(BlueprintPure, Category = "Animation|PhysicsControl", meta = (BlueprintThreadSafe))
	static void AddMultipleModifierParameters(
		UPARAM(ref, DisplayName = "Parameters") FPhysicsControlControlAndModifierParameters& InParameters, 
		UPARAM(DisplayName = "Parameters") FPhysicsControlControlAndModifierParameters&      OutParameters,
		const TArray<FName>&                                                                 Names,
		const FPhysicsControlModifierSparseData&                                             ModifierData);

	// Returns the linear interpolation of two sets of parameters. Any parameters that exist in one
	// of the input sets but not the other will be added to the output with a weight of 1.
	UFUNCTION(BlueprintPure, Category = "Animation|PhysicsControl", meta = (BlueprintThreadSafe))
	static void BlendParameters(
		UPARAM(ref, DisplayName = "From Parameters") FPhysicsControlControlAndModifierParameters& InParametersA,
		UPARAM(ref, DisplayName = "To Parameters") FPhysicsControlControlAndModifierParameters&   InParametersB,
		UPARAM(DisplayName = "Weight") const float                                                InInterpolationWeight,
		UPARAM(DisplayName = "Parameters") FPhysicsControlControlAndModifierParameters&           OutParameters);
	
	// Adds control parameters to the output parameters for each control name in the supplied array.
	// The values in each control parameters added will be a linear interpolation of the two
	// supplied Parameters, blending from the start parameters to the end parameters across the
	// elements in the array. Note that this is most likely only useful when the control names are
	// in order - for example going down a limb.
	UFUNCTION(BlueprintPure, Category = "Animation|PhysicsControl", meta = (BlueprintThreadSafe))
	static void BlendControlParametersThroughSet(
		UPARAM(ref, DisplayName = "Parameters") FPhysicsControlControlAndModifierParameters&       InParameters,
		UPARAM(ref, DisplayName = "Start Parameters") const FPhysicsControlNamedControlParameters& InStartControlParameters,
		UPARAM(ref, DisplayName = "End Parameters") const FPhysicsControlNamedControlParameters&   InEndControlParameters,
		const TArray<FName>&                                                                       ControlNames,
		FPhysicsControlControlAndModifierParameters&                                               OutParameters);

	// Adds body modifier parameters to the output parameters for each modifier name in the supplied
	// array. The values in each modifier parameters added will be a linear interpolation of the two
	// supplied parameters, blending from the start parameters to the end parameters across the
	// elements in the array. Note that this is most likely only useful when the modifier names are
	// in order - for example going down a limb.
	UFUNCTION(BlueprintPure, Category = "Animation|PhysicsControl", meta = (BlueprintThreadSafe))
	static void BlendModifierParametersThroughSet(
		UPARAM(ref) FPhysicsControlControlAndModifierParameters&                        InParameters,
		UPARAM(ref) const FPhysicsControlNamedModifierParameters&                       InStartModifierParameters,
		UPARAM(ref) const FPhysicsControlNamedModifierParameters&                       InEndModifierParameters,
		const TArray<FName>&                                                            ModifierNames,
		UPARAM(DisplayName = "Parameters") FPhysicsControlControlAndModifierParameters& OutParameters);

	// Get a Rigid Body With Control node reference from an anim node reference 
	UFUNCTION(BlueprintCallable, Category = "Animation|PhysicsControl", meta = (BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static FRigidBodyWithControlReference ConvertToRigidBodyWithControl(
		const FAnimNodeReference&           Node,
		EAnimNodeReferenceConversionResult& Result);

	// Get a Rigid Body With Control node from an anim node (pure)
	UFUNCTION(BlueprintPure, Category = "Animation|PhysicsControl", meta = (BlueprintThreadSafe, DisplayName = "Convert to Rigid Body With Control"))
	static void ConvertToRigidBodyWithControlPure(
		const FAnimNodeReference&       Node, 
		FRigidBodyWithControlReference& RigidBodyWithControl, 
		bool&                           Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		RigidBodyWithControl = ConvertToRigidBodyWithControl(Node, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
	}

	// Set the physics asset on the Rigid Body With Control anim graph node.
	UFUNCTION(BlueprintCallable, Category = "Animation|PhysicsControl", meta = (BlueprintThreadSafe))
	static FRigidBodyWithControlReference SetOverridePhysicsAsset(const FRigidBodyWithControlReference& Node, UPhysicsAsset* PhysicsAsset);

	// Get the names of all the controls in a specified set managed by this Rigid Body With Control node.
	UFUNCTION(BlueprintPure, Category = "Animation|PhysicsControl", meta = (BlueprintThreadSafe))
	static TArray<FName> GetControlNamesInSet(
		UPARAM(DisplayName = "Node") const FRigidBodyWithControlReference& RigidBodyWithControl, 
		const FName                                                        SetName);

	// Get the names of all the body modifiers in a specified set managed by this Rigid Body With
	// Control node.
	UFUNCTION(BlueprintPure, Category = "Animation|PhysicsControl", meta = (BlueprintThreadSafe))
	static TArray<FName> GetBodyModifierNamesInSet(
		UPARAM(DisplayName = "Node") const FRigidBodyWithControlReference& RigidBodyWithControl,
		const FName                                                        SetName);

	// Disables collision between two physics bodies at runtime. Works for two bones on the same
	// skeletal mesh, a skeletal-mesh bone vs. a separate primitive, or two unrelated primitives.
	//
	// Notes:
	// - Both bodies must be in the same physics scene / solver.
	// - Both bodies must be dynamic or kinematic; static particles are not supported by the
	//   underlying Chaos ignore-collision manager.
	// - The effect is transient: it does not survive body re-creation (LOD swap, SetSkeletalMesh,
	//   SetPhysicsAsset, physics-state teardown). To restore the disable after such an event, call
	//   the function again.
	//
	// Returns true on success; false on validation failure (null component, missing bone,
	// mismatched solvers, static body, etc.) - a warning is logged in that case.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	static bool DisableCollisionBetweenBodies(
		UPrimitiveComponent* FirstComponent,  const FName FirstBoneName,
		UPrimitiveComponent* SecondComponent, const FName SecondBoneName);

	// Re-enables collision between two physics bodies that was previously suppressed by
	// DisableCollisionBetweenBodies. Works for two bones on the same skeletal mesh, a
	// skeletal-mesh bone vs. a separate primitive, or two unrelated primitives. Calling this for
	// a pair that is not currently disabled is a no-op.
	//
	// Notes:
	// - Both bodies must be in the same physics scene / solver.
	// - Both bodies must be dynamic or kinematic; static particles are not supported by the
	//   underlying Chaos ignore-collision manager.
	// - Body re-creation (LOD swap, SetSkeletalMesh, SetPhysicsAsset, physics-state teardown)
	//   already restores default collision, so this call is only needed to undo a still-live
	//   DisableCollisionBetweenBodies.
	//
	// Returns true on success; false on validation failure (null component, missing bone,
	// mismatched solvers, static body, etc.) - a warning is logged in that case.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	static bool EnableCollisionBetweenBodies(
		UPrimitiveComponent* FirstComponent,  const FName FirstBoneName,
		UPrimitiveComponent* SecondComponent, const FName SecondBoneName);

	// Bulk variant of DisableCollisionBetweenBodies. Disables collision for every
	// (FirstBoneNames[i], SecondBoneNames[j]) combination in a single call. An empty bone array, or
	// if the bone name is blank/None, means "use the component's single body" (e.g. for a
	// static-mesh component) and is treated as a one-element array of NAME_None. Self-pairs (the
	// same body on both sides) are skipped. See DisableCollisionBetweenBodies for the constraints
	// that apply per pair.
	//
	// Returns true only when every pair was applied successfully. Per-pair validation failures log
	// a warning and are skipped. The remaining valid pairs are still applied, and the return value
	// is false to signal partial application.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	static bool DisableCollisionBetweenBodyArrays(
		UPrimitiveComponent* FirstComponent,  const TArray<FName>& FirstBoneNames,
		UPrimitiveComponent* SecondComponent, const TArray<FName>& SecondBoneNames);

	// Bulk variant of EnableCollisionBetweenBodies. Re-enables collision for every
	// (FirstBoneNames[i], SecondBoneNames[j]) combination in a single call. An empty bone array, or
	// if the bone name is blank/None, means "use the component's single body" (e.g. for a
	// static-mesh component) and is treated as a one-element array of NAME_None. Self-pairs and
	// pairs that aren't currently disabled are no-ops. See EnableCollisionBetweenBodies for the
	// constraints that apply per pair.
	//
	// Returns true only when every pair was applied successfully. Per-pair validation failures log
	// a warning and are skipped. The remaining valid pairs are still applied, and the return value
	// is false to signal partial application.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	static bool EnableCollisionBetweenBodyArrays(
		UPrimitiveComponent* FirstComponent,  const TArray<FName>& FirstBoneNames,
		UPrimitiveComponent* SecondComponent, const TArray<FName>& SecondBoneNames);
};

#endif
