// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsControlData.h"
#include "PhysicsControlLimbData.h"
#include "PhysicsControlNameRecords.h"
#include "PhysicsControlAsset.h"
#include "PhysicsControlRecord.h"
#include "PhysicsControlPoseData.h"

#include "Components/SceneComponent.h"
#include "EngineDefines.h"
#include "PhysicsEngine/ConstraintDrives.h"
#include "Logging/LogVerbosity.h"
#include "Templates/PimplPtr.h"
#include "UObject/ObjectMacros.h"

#include "PhysicsControlComponent.generated.h"

#define UE_API PHYSICSCONTROL_API

class FPrimitiveDrawInterface;
struct FPhysicsControlComponentImpl;
struct FPhysicsControlRecord;
struct FPhysicsBodyModifierRecord;
struct FConstraintInstance;

// Specifies how any reset to cached target should work.
UENUM(BlueprintType)
enum class EResetToCachedTargetBehavior : uint8
{
	// Reset of the associated physics bodies is done immediately, to whatever transforms are in the cache. These
	// will reflect the previous or the upcoming animation targets, depending on what stage of the tick this
	// is called.
	ResetImmediately,
	// Reset of the associated physics bodies will be done during the next Tick (or UpdateControls). Note that
	// this will reset the bodies to the updated animation targets, and then there will be a subsequent physics
	// update, which will result in the final transforms.
	ResetDuringUpdateControls
};

// This is the main Physics Control Component class which manages Controls and Body Modifiers associated
// with one or more static or skeletal meshes. You can add this as a component to an actor containing a
// mesh and then use it to create, configure and destroy Controls/Body Modifiers:
//
// Controls are used to control one physics body relative to another (or the world). These controls are done
// through physical spring/damper drives.
//
// Body Modifiers are used to update the most important physical properties of physics bodies such as whether
// they are simulated vs kinematic, or whether they experience gravity.
//
// Note that Controls and Body Modifiers are given names (which are predictable). These names can then be stored
// (perhaps in arrays) to make it easy to quickly change multiple Controls/Body Modifiers.
UCLASS(MinimalAPI, meta = (BlueprintSpawnableComponent), ClassGroup = Physics)
class UPhysicsControlComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

public:

	// Allows manual ticking so that your code can run in between updating the target caches and updating
	// the controls. This allows you to read the targets coming from animation and use those values to
	// create your own controls etc.
	//
	// To use this function, you should disable ticking of the Physics Control Component, and ensure that the
	// relevant Skeletal Mesh Component (if being used) has ticked, using a tick prerequisite. Then explicitly
	// call (in order) UpdateTargetCaches and UpdateControls as you process your tick.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API void UpdateTargetCaches(float DeltaTime);

	// Allows manual ticking so that your code can run in between updating the target caches and updating
	// the controls and body modifiers. This allows you to read the targets coming from animation and use
	// those values to create your own controls etc.
	//
	// To use this function, you should disable ticking of the Physics Control Component, and ensure that the
	// relevant Skeletal Mesh Component (if being used) has ticked, using a tick prerequisite. Then explicitly
	// call (in order) UpdateTargetCaches and UpdateControls as you process your tick.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API void UpdateControls(float DeltaTime);

	// Creates a new control for mesh components
	//
	// @param ControlData   Describes the initial strength etc of the new control
	// @param ControlTarget Describes the initial target for the new control
	// @param Set Which set to include the control in (optional). Note that it automatically
	//        gets added to the set "All"
	// @param NamePrefix Optional string that is prefixed to the control that is created.
	// @return The name of the new control
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API FName CreateControl(
		UPrimitiveComponent*          ParentComponent,
		FName                         ParentBoneName,
		UPrimitiveComponent*          ChildComponent,
		const FName                   ChildBoneName,
		const FPhysicsControlData     ControlData,
		const FPhysicsControlTarget   ControlTarget, 
		FName                         Set,
		FString                       NamePrefix = TEXT("")
	);

	// Creates a new control for mesh components
	//
	// @param Name The name of the control that will be created. Creation will fail if this name is already in use.
	// @param ControlData   Describes the initial strength etc of the new control
	// @param ControlTarget Describes the initial target for the new control
	// @param Set Which set to include the control in (optional). Note that it automatically
	//        gets added to the set "All"
	// @return True if a new control was created, false if a control of the specified name already exists
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool CreateNamedControl(
		FName                         Name,
		UPrimitiveComponent*          ParentComponent,
		const FName                   ParentBoneName,
		UPrimitiveComponent*          ChildComponent,
		const FName                   ChildBoneName,
		const FPhysicsControlData     ControlData, 
		const FPhysicsControlTarget   ControlTarget, 
		const FName                   Set);

	// Creates a collection of controls controlling a skeletal mesh
	//
	// @param SkeletalMeshComponent The skeletal mesh which will have controls
	// @param BoneName The name of the bone below which controls should be created. Each bone will be the child in a control
	// @param bIncludeSelf Whether or not to include BoneName when creating controls
	// @param ControlType What type of control to create. This determines what the parent will be for each control
	// @param ControlData   Describes the initial strength etc of the new control
	// @param Set Which set to include the control in (optional). Note that it automatically
	//        gets added to the set "All"
	// @return An array of the controls that have been created
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API TArray<FName> CreateControlsFromSkeletalMeshBelow(
		USkeletalMeshComponent*       SkeletalMeshComponent,
		const FName                   BoneName,
		const bool                    bIncludeSelf,
		const EPhysicsControlType     ControlType,
		const FPhysicsControlData     ControlData,
		const FName                   Set);

	// Creates a collection of ParentSpace controls controlling a skeletal mesh, initializing
	// them with a constraint profile
	//
	// @param SkeletalMeshComponent The skeletal mesh which will have controls
	// @param BoneName The name of the bone below which controls should be created. Each bone will be the child in a control
	// @param bIncludeSelf Whether or not to include BoneName when creating controls
	// @param ConstraintProfile The constraint profile to use for initializing the control strength and
	//        damping (etc) parameters. Note that the controls will all be created in "parent space"
	//        - i.e. with each part controlled relative to its parent. The strength and damping will
	//        be taken from the values that the relevant joint in the physics asset would have given
	//        the constraint profile (or the default profile if it can't be found) - though they will
	//        not match exactly if the linear drive and different x/y/z values, or if the angular drive
	//        was using twist/swing instead of slerp. Note also that the joint constraints do not use
	//        the animation velocity as a target, so when creating controls in this way the control
	//        will set the skeletal animation velocity multiplier to zero.
	// @param Set Which set to include the control in (optional). Note that it automatically
	//        gets added to the set "All"
	// @param bEnabled If true then the control will be enabled immediately. If false you will need to call
	//        SetControlEnabled(true) in order to enable it.
	// @return An array of the controls that have been created
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API TArray<FName> CreateControlsFromSkeletalMeshAndConstraintProfileBelow(
		USkeletalMeshComponent* SkeletalMeshComponent,
		const FName             BoneName,
		const bool              bIncludeSelf,
		const FName             ConstraintProfile,
		const FName             Set,
		const bool              bEnabled = true);

	// Creates a collection of controls controlling a skeletal mesh
	//
	// @param SkeletalMeshComponent The skeletal mesh which will have controls
	// @param BoneNames The names of bones for which controls should be created. Each bone will be the child in a control
	// @param ControlType What type of control to create. This determines what the parent will be for each control
	// @param ControlData   Describes the initial strength etc of the new control
	// @param Set Which set to include the control in (optional). Note that it automatically
	//        gets added to the set "All"
	// @return An array of the controls that have been created
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API TArray<FName> CreateControlsFromSkeletalMesh(
		USkeletalMeshComponent*       SkeletalMeshComponent,
		const TArray<FName>&          BoneNames,
		const EPhysicsControlType     ControlType,
		const FPhysicsControlData     ControlData,
		const FName                   Set);

	// Creates a collection of ParentSpace controls controlling a skeletal mesh, initializing them
	// with a constraint profile
	//
	// @param SkeletalMeshComponent The skeletal mesh which will have controls
	// @param BoneNames The names of bones for which controls should be created. Each bone will be the child in a control
	// @param ConstraintProfile The constraint profile to use for initializing the control strength and
	//        damping (etc) parameters. Note that the controls will all be created in "parent space"
	//        - i.e. with each part controlled relative to its parent. The strength and damping will
	//        be taken from the values that the relevant joint in the physics asset would have given
	//        the constraint profile (or the default profile if it can't be found) - though they will
	//        not match exactly if the linear drive and different x/y/z values, or if the angular drive
	//        was using twist/swing instead of slerp. Note also that the joint constraints do not use
	//        the animation velocity as a target, so when creating controls in this way the control
	//        will set the skeletal animation velocity multiplier to zero.
	// @param Set Which set to include the control in (optional). Note that it automatically
	//        gets added to the set "All"
	// @param bEnabled If true then the control will be enabled immediately. If false you will need to call
	//        SetControlEnabled(true) in order to enable it.
	// @return An array of the controls that have been created
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API TArray<FName> CreateControlsFromSkeletalMeshAndConstraintProfile(
		USkeletalMeshComponent* SkeletalMeshComponent,
		const TArray<FName>&    BoneNames,
		const FName             ConstraintProfile,
		const FName             Set,
		const bool              bEnabled = true);

	// Calculates which bones belong to which limb in a skeletal mesh
	//
	// @param SkeletalMeshComponent The skeletal mesh which will be analyzed
	// @param LimbSetupData This needs to be filled in with the list of limbs to "discover". Note that the
	//        limbs should be listed starting at the "leaf" (i.e. outer) parts of the skeleton
	//        first, typically finishing with the spine. In addition, the spine limb is typically
	//        specified using the first spine bone, but flagging it to include its parent
	//        (normally the pelvis).
	// @return A map of limb names to bones
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API TMap<FName, FPhysicsControlLimbBones> GetLimbBonesFromSkeletalMesh(
		USkeletalMeshComponent*                     SkeletalMeshComponent,
		const TArray<FPhysicsControlLimbSetupData>& LimbSetupData) const;

	// Creates a collection of controls controlling a skeletal mesh, grouped together in limbs
	//
	// @param AllControls A single container for all the controls that have been created
	// @param LimbBones A map relating the limbs and the bones that they contain. Typically create this
	//        using GetLimbBonesFromSkeletalMesh
	// @param ControlType What type of control to create. This determines what the parent will be for each control
	// @param ControlData Describes the initial strength etc of the new control
	// @param WorldComponent Optional component to use as the parent object for any "world-space" controls
	//        that are created. Will be ignored if the controls being created are not world-space.
	// @param WorldBoneName Additional bone name to identify the world object if the WorldComponent is actually
	//        a skeletal mesh component.
	// @param NamePrefix Optional string that is prefixed to each control that is created.
	// @return A map containing the controls for each limb
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API TMap<FName, FPhysicsControlNames> CreateControlsFromLimbBones(
		FPhysicsControlNames&                        AllControls,
		const TMap<FName, FPhysicsControlLimbBones>& LimbBones,
		const EPhysicsControlType                    ControlType,
		const FPhysicsControlData                    ControlData,
		UPrimitiveComponent*                         WorldComponent = nullptr,
		FName                                        WorldBoneName = NAME_None,
		FString                                      NamePrefix = TEXT(""));

	// Creates a collection of ParentSpace controls controlling a skeletal mesh, grouped together in limbs, initializing
	// them with a constraint profile
	//
	// @param AllControls A single container for all the controls that have been created
	// @param LimbBones A map relating the limbs and the bones that they contain. Typically create this
	//        using GetLimbBonesFromSkeletalMesh
	// @param ConstraintProfile The constraint profile to use for initializing the control strength and
	//        damping (etc) parameters. Note that the controls will all be created in "parent space"
	//        - i.e. with each part controlled relative to its parent. The strength and damping will
	//        be taken from the values that the relevant joint in the physics asset would have given
	//        the constraint profile (or the default profile if it can't be found) - though they will
	//        not match exactly if the linear drive and different x/y/z values, or if the angular drive
	//        was using twist/swing instead of slerp. Note also that the joint constraints do not use
	//        the animation velocity as a target, so when creating controls in this way the control
	//        will set the skeletal animation velocity multiplier to zero.
	// @param bEnabled If true then the control will be enabled immediately. If false you will need to call
	//        SetControlEnabled(true) in order to enable it.
	// @return A map containing the controls for each limb
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API TMap<FName, FPhysicsControlNames> CreateControlsFromLimbBonesAndConstraintProfile(
		FPhysicsControlNames&                        AllControls,
		const TMap<FName, FPhysicsControlLimbBones>& LimbBones,
		const FName                                  ConstraintProfile,
		const bool                                   bEnabled = true);

	// Destroys all controls and body modifiers
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API void DestroyAllControlsAndBodyModifiers();

	// Destroys a control or controls in a set
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Name The name of the control or set of controls to destroy.
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool DestroyControl(
		const FName Name, const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Destroys all controls or sets of controls
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Names The names of the controls or sets of controls to destroy. Note that if you have these
	//        in a FPhysicsControlNameArray then it can be split.
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool DestroyControls(
		const TArray<FName>& Names, const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Destroys all controls in a set
	//
	// @param Set The set of controls to use to destroy. Standard sets will include "All", "WorldSpace",
	// "ParentSpace" and things like "WorldSpace_ArmLeft", depending on how controls have been created.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool DestroyControlsInSet(const FName Set);

	// Updates the parent object part of a control or controls in a set. Note that this won't change
	// the name of the control (which may subsequently be misleading), or any set it is included in, etc.
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlParent(
		const FName Name, UPrimitiveComponent* ParentComponent, const FName ParentBoneName,
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Updates the parent object part of controls or controls in a set. Note that this won't change
	// the name of the controls (which may subsequently be misleading), or any set they are included in, etc.
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlParents(
		const TArray<FName>& Names, UPrimitiveComponent* ParentComponent, const FName ParentBoneName,
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlParentsInSet(
		const FName          Set,
		UPrimitiveComponent* ParentComponent,
		const FName          ParentBoneName);

	// Modifies an existing control data in a control or controls in a set - i.e. the strengths
	// etc of the control driving towards the target
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Name The name of the control or set of controls to update.
	// @param ControlData The new control data
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlData(
		const FName Name, const FPhysicsControlData ControlData, 
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Modifies existing control data in controls or controls in sets - i.e. the strengths
	// etc of the controls driving towards the targets
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Names The names of the controls or sets of controls to update. Note that if you have
	//        these in a FPhysicsControlNameArray then it can be split.
	// @param ControlData The new control data
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlDatas(
		const TArray<FName>& Names, const FPhysicsControlData ControlData,
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Modifies existing control data - i.e. the strengths etc of the controls driving towards the targets
	//
	// @param Set The set of controls to update. Standard sets will include "All", "WorldSpace",
	//        "ParentSpace" and things like "WorldSpace_ArmLeft", depending on how controls have been created.
	// @param ControlData The new control data
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlDatasInSet(const FName Set, const FPhysicsControlData ControlData);

	// Modifies an existing control data in a control or controls in a set - i.e. the strengths
	// etc of the control driving towards the target
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Name The name of the control or set of controls to update.
	// @param ControlData The new control data
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlSparseData(
		const FName Name, const FPhysicsControlSparseData ControlData,
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Modifies existing control data in controls or controls in sets - i.e. the strengths
	// etc of the controls driving towards the targets
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Names The names of the controls or sets of controls to update. Note that if you have
	//        these in a FPhysicsControlNameArray then it can be split.
	// @param ControlData The new control data
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlSparseDatas(
		const TArray<FName>& Names, const FPhysicsControlSparseData ControlData,
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Modifies existing control data - i.e. the strengths etc of the controls driving towards the targets
	//
	// @param Set The set of controls to update. Standard sets will include "All", "WorldSpace",
	//        "ParentSpace" and things like "WorldSpace_ArmLeft", depending on how controls have been created.
	// @param ControlData The new control data
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlSparseDatasInSet(const FName Set, const FPhysicsControlSparseData ControlData);

	// Modifies existing control data in a control or controls in a set using the multipliers
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Name The name of the control or set of controls to update.
	// @param ControlMultipliers The new control multipliers
	// @param bEnableControl Enables the control if it is currently disabled
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlMultiplier(
		const FName Name, const FPhysicsControlMultiplier ControlMultiplier, const bool bEnableControl = true,
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Modifies existing control data in controls or controls in sets using the multipliers
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Names The names of the controls or sets of controls to update. Note that if you have
	//        these in a FPhysicsControlNameArray then it can be split.
	// @param ControlMultiplier The new control multiplier
	// @param bEnableControl Enables the controls if currently disabled
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlMultipliers(
		const TArray<FName>& Names, const FPhysicsControlMultiplier ControlMultiplier, const bool bEnableControl = true,
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Modifies existing control data using the multipliers
	//
	// @param Set The set of controls to update. Standard sets will include "All", "WorldSpace",
	//        "ParentSpace" and things like "WorldSpace_ArmLeft", depending on how controls have been created.
	// @param ControlMultiplier The new control multiplier
	// @param bEnableControl Enables the controls if currently disabled
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlMultipliersInSet(
		const FName Set, const FPhysicsControlMultiplier  ControlMultiplier, const bool bEnableControl = true);

	// Modifies existing control data in a control or controls in a set using the multipliers
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Name The name of the control or set of controls to update.
	// @param ControlMultipliers The new control multipliers
	// @param bEnableControl Enables the control if it is currently disabled
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlSparseMultiplier(
		const FName Name, const FPhysicsControlSparseMultiplier ControlMultiplier, const bool bEnableControl = true,
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Modifies existing control data in controls or controls in sets using the multipliers
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Names The names of the controls or sets of controls to update. Note that if you have these in a
	//        FPhysicsControlNameArray then it can be split.
	// @param ControlMultiplier The new control multiplier
	// @param bEnableControl Enables the controls if currently disabled
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlSparseMultipliers(
		const TArray<FName>& Names, const FPhysicsControlSparseMultiplier ControlMultiplier, const bool bEnableControl = true,
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Modifies existing control data using the multipliers
	//
	// @param Set The set of controls to update. Standard sets will include "All", "WorldSpace",
	//        "ParentSpace" and things like "WorldSpace_ArmLeft", depending on how controls have been created.
	// @param ControlMultiplier The new control multiplier
	// @param bEnableControl Enables the controls if currently disabled
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlSparseMultipliersInSet(
		const FName                           Set, 
		const FPhysicsControlSparseMultiplier ControlMultiplier, 
		const bool                            bEnableControl = true);

	// Modifies existing control's linear data in a control or controls in a set - i.e. the strengths
	// etc of the control driving towards the target
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Name The name of the control or set of controls to update.
	// @param Strength The strength used to drive linear motion
	// @param DampingRatio The amount of damping associated with the linear strength. 1 Results in critically damped motion
	// @param ExtraDamping The amount of additional linear damping
	// @param MaxForce The maximum force used to drive the linear motion. Zero indicates no limit.
	// @param bEnableControl Enables the control if it is currently disabled
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlLinearData(
		const FName Name,
		const float Strength = 1.0f, 
		const float DampingRatio = 1.0f, 
		const float ExtraDamping = 0.0f, 
		const float MaxForce = 0.0f, 
		const bool  bEnableControl = true,
		const bool  bApplyToControlsWithName = true, 
		const bool  bApplyToSetsWithName = true);

	// Modifies existing control's angular data in a control or controls in a set - i.e. the strengths
	// etc of the control driving towards the target
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Name The name of the control or set of controls to update.
	// @param Strength The strength used to drive angular motion
	// @param DampingRatio The amount of damping associated with the angular strength. 1 Results in critically damped motion
	// @param ExtraDamping The amount of additional angular damping
	// @param MaxTorque The maximum torque used to drive the angular motion. Zero indicates no limit.
	// @param bEnableControl Enables the control if it is currently disabled
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlAngularData(
		const FName Name,
		const float Strength = 1.0f, 
		const float DampingRatio = 1.0f, 
		const float ExtraDamping = 0.0f, 
		const float MaxTorque = 0.0f, 
		const bool  bEnableControl = true,
		const bool  bApplyToControlsWithName = true,
		const bool  bApplyToSetsWithName = true);

	// Sets the point at which controls will "push" the child object.
	//
	// @param Name The name of the control to update.
	// @param Position The position of the control point on the child mesh object (only relevant if that
	//        object is in use and is being simulated)
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlPoint(const FName Name, const FVector Position);

	// Resets the control point to the center of mass of the mesh
	//
	// @param Name The name of the control to update.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool ResetControlPoint(const FName Name);

	// Modifies an existing control target in a control or controls in a set - i.e. what it is driving
	// towards, relative to the parent object
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Name The name of the control or set of controls to update.
	// @param ControlTarget The new target for the control
	// @param bEnableControl Enables the control if it is currently disabled
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlTarget(
		const FName Name, const FPhysicsControlTarget ControlTarget, const bool bEnableControl = true,
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Modifies existing control targets in controls or controls in sets - i.e. what they are driving
	// towards, relative to the parent objects
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Names The names of the controls or sets of controls to update. Note that if you have these in a
	//        FPhysicsControlNameArray then it can be split.
	// @param ControlTarget The new target for the controls
	//
	// @param bEnableControl Enables the controls if currently disabled
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlTargets(
		const TArray<FName>& Names, const FPhysicsControlTarget ControlTarget, const bool bEnableControl = true,
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Modifies existing control targets - i.e. what they are driving towards, relative to the parent objects
	//
	// @param Set The set of controls to update. Standard sets will include "All", "WorldSpace",
	//        "ParentSpace" and things like "WorldSpace_ArmLeft", depending on how controls have been created.
	// @param ControlTarget The new target for the controls
	// @param bEnableControl Enables the controls if currently disabled
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlTargetsInSet(
		const FName                 Set, 
		const FPhysicsControlTarget ControlTarget, 
		const bool                  bEnableControl = true);

	// Modifies an existing control target in a control or controls in a set - i.e. what it is driving
	// towards, relative to the parent object
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Name The name of the control or set of controls to update.
	// @param Position The new position target for the control
	// @param Orientation The new orientation target for the control
	// @param VelocityDeltaTime If non-zero, the target velocity will be calculated using the current target
	//        position. If zero, the target velocity will be set to zero.
	// @param bEnableControl Enables the control if it is currently disabled
	// @param bApplyControlPointToTarget If true, then the target position/orientation is treated as
	//        a "virtual" object, where the system attempts to move the object to match the pose of
	//        this "virtual" object that has been placed at the target transform. Use this when you
	//        want to specify the target transform for the object as a whole. If false, then the
	//        target transform is used as is, and the system drives the control point towards this
	//        transform.
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	// @return Returns true if the control/set could be found
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlTargetPositionAndOrientation(
		const FName    Name,
		const FVector  Position,
		const FRotator Orientation,
		const float    VelocityDeltaTime,
		const bool     bEnableControl = true,
		const bool     bApplyControlPointToTarget = false,
		const bool     bApplyToControlsWithName = true, 
		const bool     bApplyToSetsWithName = true);

	// Modifies existing control targets in controls or controls in sets - i.e. what it is driving
	// towards, relative to the parent object
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Names The name of the controls or sets of controls to update.
	// @param Position The new position target for the control
	// @param Orientation The new orientation target for the control
	// @param VelocityDeltaTime If non-zero, the target velocity will be calculated using the current target
	//        position. If zero, the target velocity will be set to zero.
	// @param bEnableControl Enables the control if it is currently disabled
	// @param bApplyControlPointToTarget If true, then the target position/orientation is treated as
	//        a "virtual" object, where the system attempts to move the object to match the pose of
	//        this "virtual" object that has been placed at the target transform. Use this when you
	//        want to specify the target transform for the object as a whole. If false, then the
	//        target transform is used as is, and the system drives the control point towards this
	//        transform.
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	// @return Returns true if all the control names were found
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlTargetPositionsAndOrientations(
		const TArray<FName>& Names,
		const FVector        Position,
		const FRotator       Orientation,
		const float          VelocityDeltaTime,
		const bool           bEnableControl = true,
		const bool           bApplyControlPointToTarget = false,
		const bool           bApplyToControlsWithName = true, 
		const bool           bApplyToSetsWithName = true);

	// Calls SetControlTargetPositionAndOrientation for each control in the set
	// @return Returns true if the set was be found
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlTargetPositionsAndOrientationsInSet(
		const FName          SetName,
		const FVector        Position,
		const FRotator       Orientation,
		const float          VelocityDeltaTime,
		const bool           bEnableControl = true,
		const bool           bApplyControlPointToTarget = false);

	// Modifies an existing control target of a control or controls in a set - i.e. what it is driving
	// towards, relative to the parent object
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Name The name of the control or set of controls to update.
	// @param Position          The new position target for the control
	// @param VelocityDeltaTime If non-zero, the target velocity will be calculated using the current target
	//        position. If zero, the target velocity will be set to zero.
	// @param bEnableControl Enables the control if it is currently disabled
	// @param bApplyControlPointToTarget If true, then the target position/orientation is treated as
	//        a "virtual" object, where the system attempts to move the object
	//        to match the pose of this "virtual" object that has been placed at
	//        the target transform. Use this when you want to specify the target
	//        transform for the object as a whole. If false, then the target transform
	//        is used as is, and the system drives the control point towards this
	//        transform.
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	// @return Returns true if the control/set can be found
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlTargetPosition(
		const FName   Name,
		const FVector Position, 
		const float   VelocityDeltaTime, 
		const bool    bEnableControl = true,
		const bool    bApplyControlPointToTarget = false,
		const bool    bApplyToControlsWithName = true, 
		const bool    bApplyToSetsWithName = true);

	// Modifies existing control targets in controls or controls in sets - i.e. what it is driving
	// towards, relative to the parent object
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Names The name of the controls or sets of controls to update.
	// @param Position The new position target for the control
	// @param VelocityDeltaTime If non-zero, the target velocity will be calculated using the current target
	//        position. If zero, the target velocity will be set to zero.
	// @param bEnableControl Enables the control if it is currently disabled
	// @param bApplyControlPointToTarget If true, then the target position/orientation is treated as
	//        a "virtual" object, where the system attempts to move the object to match the pose of
	//        this "virtual" object that has been placed at the target transform. Use this when you
	//        want to specify the target transform for the object as a whole. If false, then the
	//        target transform is used as is, and the system drives the control point towards this
	//        transform.
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	// @return Returns true if all the controls/sets can be found
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlTargetPositions(
		const TArray<FName>& Names,
		const FVector        Position, 
		const float          VelocityDeltaTime, 
		const bool           bEnableControl = true,
		const bool           bApplyControlPointToTarget = false,
		const bool           bApplyToControlsWithName = true, 
		const bool           bApplyToSetsWithName = true);

	// Calls SetControlTargetPosition for each of the controls in the set
	// @return Returns true if the set name and its controls can be found
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlTargetPositionsInSet(
		const FName   SetName,
		const FVector Position, 
		const float   VelocityDeltaTime, 
		const bool    bEnableControl = true,
		const bool    bApplyControlPointToTarget = false);

	// Modifies an existing control target in a control or controls in a set - i.e. what it is driving
	// towards, relative to the parent object
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Name The name of the control or set of controls to update.
	// @param Orientation The new orientation target for the control
	// @param AngularVelocityDeltaTime If non-zero, the target angular velocity will be calculated using the current
	//        target position. If zero, the target velocity will be set to zero.
	// @param bEnableControl Enables the control if it is currently disabled
	// @param bApplyControlPointToTarget If true, then the target position/orientation is treated as
	//        a "virtual" object, where the system attempts to move the object
	//        to match the pose of this "virtual" object that has been placed at
	//        the target transform. Use this when you want to specify the target
	//        transform for the object as a whole. If false, then the target transform
	//        is used as is, and the system drives the control point towards this
	//        transform.
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	// @return Returns true if the control/set can be found
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlTargetOrientation(
		const FName    Name,
		const FRotator Orientation, 
		const float    AngularVelocityDeltaTime, 
		const bool     bEnableControl = true,
		const bool     bApplyControlPointToTarget = false,
		const bool     bApplyToControlsWithName = true, 
		const bool     bApplyToSetsWithName = true);

	// Modifies existing control targets in controls or controls in sets - i.e. what it is driving
	// towards, relative to the parent object
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Name The name of the control or set of controls to update.
	// @param Orientation The new orientation target for the control
	// @param AngularVelocityDeltaTime If non-zero, the target angular velocity will be calculated using the current
	//        target position. If zero, the target velocity will be set to zero.
	// @param bEnableControl Enables the control if it is currently disabled
	// @param bApplyControlPointToTarget If true, then the target position/orientation is treated as
	//        a "virtual" object, where the system attempts to move the object
	//        to match the pose of this "virtual" object that has been placed at
	//        the target transform. Use this when you want to specify the target
	//        transform for the object as a whole. If false, then the target transform
	//        is used as is, and the system drives the control point towards this
	//        transform.
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	// @return Returns true if all the controls can be found
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlTargetOrientations(
		const TArray<FName>& Names,
		const FRotator Orientation,
		const float    AngularVelocityDeltaTime, 
		const bool     bEnableControl = true,
		const bool     bApplyControlPointToTarget = false,
		const bool     bApplyToControlsWithName = true,
		const bool     bApplyToSetsWithName = true);

	// Calls SetControlTargetOrientation for each of the controls in the set
	// @return Returns true if the set and all the controls can be found
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlTargetOrientationsInSet(
		const FName    SetName,
		const FRotator Orientation, 
		const float    AngularVelocityDeltaTime, 
		const bool     bEnableControl = true,
		const bool     bApplyControlPointToTarget = false);

	// Calls SetControlTargetPosition for each element of the control names and positions. These array should match
	// in size.
	// @return true if the control/position arrays match, false if they don't.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlTargetPositionsFromArray(
		const TArray<FName>&   Names,
		const TArray<FVector>& Positions, 
		const float            VelocityDeltaTime, 
		const bool             bEnableControl = true,
		const bool             bApplyControlPointToTarget = false);

	// Calls SetControlTargetPosition for each element of the control names and positions. These array should match
	// in size.
	// @return true if the control/position arrays match, false if they don't.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlTargetOrientationsFromArray(
		const TArray<FName>&    Names,
		const TArray<FRotator>& Orientations, 
		const float             VelocityDeltaTime, 
		const bool              bEnableControl = true,
		const bool              bApplyControlPointToTarget = false);

	// Calls SetControlTargetPositionAndOrientation for each element of the control names, positions and orientations.
	// These array should match in size.
	// @return true if the control/position/orientation arrays match, false if they don't.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlTargetPositionsAndOrientationsFromArray(
		const TArray<FName>&    Names,
		const TArray<FVector>&  Positions,
		const TArray<FRotator>& Orientations,
		const float             VelocityDeltaTime, 
		const bool              bEnableControl = true,
		const bool              bApplyControlPointToTarget = false);

	// Calculates and sets an existing control target. This takes the "virtual" position/orientation of the parent
	// and child and calculates the relative control. Note that this will set bApplyControlPointToTarget to true.
	//
	// @param Name              The name of the control to update.
	// @param ParentPosition    The virtual/target parent position
	// @param ParentOrientation The virtual/target parent orientation
	// @param ChildPosition     The virtual/target child position
	// @param ChildOrientation  The virtual/target child orientation
	// @param VelocityDeltaTime If non-zero, the target velocity will be calculated using the current target
	//        position. If zero, the target velocity will be set to zero.
	// @param bEnableControl    Enables the control if it is currently disabled
	// @return                  Returns true if the control was found and modified, false if not
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlTargetPoses(
		const FName    Name,
		const FVector  ParentPosition, 
		const FRotator ParentOrientation,
		const FVector  ChildPosition, 
		const FRotator ChildOrientation,
		const float    VelocityDeltaTime, 
		const bool     bEnableControl = true);

	// Sets whether or not the control or controls in a set should use skeletal animation for the targets
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Name The name of the control or set of controls to update.
	// @param bUseSkeletalAnimation If true then the targets will be a combination of the skeletal animation (if
	//        there is any) and the control target that has been set
	// @param AngularTargetVelocityMultiplier If skeletal animation is being used, then this determines the amount of
	//        angular velocity extracted from the animation that is used as targets for the controls
	// @param LinearTargetVelocityMultiplier If skeletal animation is being used, then this determines the amount of
	//        linear velocity extracted from the animation that is used as targets for the controls
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlUseSkeletalAnimation(
		const FName Name,
		const bool  bUseSkeletalAnimation = true,
		const float AngularTargetVelocityMultiplier = 1.0f,
		const float LinearTargetVelocityMultiplier = 1.0f,
		const bool  bApplyToControlsWithName = true, 
		const bool  bApplyToSetsWithName = true);

	// Sets whether or not the controls or controls in sets should use skeletal animation for the targets
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Names The names of the controls or sets of controls to update. Note that if you have
	//        these in a FPhysicsControlNameArray then it can be split.
	// @param bUseSkeletalAnimation If true then the targets will be a combination of the skeletal animation (if
	//        there is any) and the control target that has been set
	// @param AngularTargetVelocityMultiplier If skeletal animation is being used, then this determines the amount of
	//        angular velocity extracted from the animation that is used as targets for the controls
	// @param LinearTargetVelocityMultiplier If skeletal animation is being used, then this determines the amount of
	//        linear velocity extracted from the animation that is used as targets for the controls
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlsUseSkeletalAnimation(
		const TArray<FName>& Names,
		const bool           bUseSkeletalAnimation = true,
		const float          AngularTargetVelocityMultiplier = 1.0f,
		const float          LinearTargetVelocityMultiplier = 1.0f,
		const bool           bApplyToControlsWithName = true,
		const bool           bApplyToSetsWithName = true);

	// Sets whether or not the controls should use skeletal animation for the targets
	//
	// @param Set The set of controls to update. Standard sets will include "All", "WorldSpace",
	//        "ParentSpace" and things like "WorldSpace_ArmLeft", depending on how controls have been created.
	// @param bUseSkeletalAnimation If true then the targets will be a combination of the skeletal animation (if
	//        there is any) and the control target that has been set
	// @param AngularTargetVelocityMultiplier If skeletal animation is being used, then this determines the amount of
	//        angular velocity extracted from the animation that is used as targets for the controls
	// @param LinearTargetVelocityMultiplier If skeletal animation is being used, then this determines the amount of
	//        linear velocity extracted from the animation that is used as targets for the controls
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlsInSetUseSkeletalAnimation(
		const FName Set,
		const bool  bUseSkeletalAnimation = true,
		const float AngularTargetVelocityMultiplier = 1.0f,
		const float LinearTargetVelocityMultiplier = 1.0f);

	// Activates or deactivates a control or controls in a set.
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Name     The name of the control or set of controls to update.
	// @param bEnable  Whether to enable/disable the control
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlEnabled(
		const FName Name, const bool bEnable = true, 
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Activates or deactivates controls or sets of controls
	// Standard set names will include "All", "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft",
	// depending on how controls have been created.
	//
	// @param Names The names of the controls or sets of controls to update. Note that if you have
	//        these in a FPhysicsControlNameArray then it can be split.
	// @param bEnable  Whether to enable/disable the controls
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlsEnabled(
		const TArray<FName>& Names, const bool bEnable = true, 
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// Activates or deactivates controls
	//
	// @param Set The set of controls to update. Standard sets will include "All", "WorldSpace",
	//        "ParentSpace" and things like "WorldSpace_ArmLeft", depending on how controls have been created.
	// @param bEnable  Whether to enable/disable the controls

	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlsInSetEnabled(const FName Set, const bool bEnable = true);

	// @param Name The name of the control or set of controls to update. Standard sets will include "All", "WorldSpace",
	//        "ParentSpace" and things like "WorldSpace_ArmLeft", depending on how controls have been created.
	// @param bDisableCollision If set then the control will disable collision between the bodies it connects.
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetControlDisableCollision(
		const FName Name, const bool bDisableCollision,
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// @param Names The names of the controls or sets of controls to update. Standard sets will include "All", "WorldSpace",
	//        "ParentSpace" and things like "WorldSpace_ArmLeft", depending on how controls have been created.
	// @param bDisableCollision If set then the control will disable collision between the bodies it connects.
	// @param bApplyToControlsWithName If true, then controls with the name will be modified
	// @param bApplyToSetsWithName If true, then controls in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlsDisableCollision(
		const TArray<FName>& Names, const bool bDisableCollision,
		const bool bApplyToControlsWithName = true, const bool bApplyToSetsWithName = true);

	// @param Set The set of controls to update. Standard sets will include "All", "WorldSpace",
	//        "ParentSpace" and things like "WorldSpace_ArmLeft", depending on how controls have been created.
	// @param bDisableCollision If set then the control will disable collision between the bodies it connects.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlsInSetDisableCollision(const FName Set, const bool bDisableCollision);

	// Sets the angular drive limit violation response on the live physics asset constraint for each
	// control in the set. This controls how the drive target rotation is adjusted when it would
	// violate the joint's angular limits, preventing the drive and the limit constraint from competing.
	// Only has an effect when the control is using a skeletal mesh with a physics asset constraint.
	//
	// @param Set      The set of controls to update.
	// @param Response The response to apply when the drive target violates a joint limit.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetControlsInSetDriveLimitViolationResponse(const FName Set, const EAngularDriveLimitViolationResponse Response);

	// @param Name     The name of the control to access.
	// @param Control  The control data that will be filled in if found
	// @return         Returns true if the control was found, false if not
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool GetControlData(const FName Name, FPhysicsControlData& ControlData) const;

	// @param Name     The name of the control to access.
	// @param Control  The control multipliers that will be filled in if found
	// @return         Returns true if the control was found, false if not
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool GetControlMultiplier(const FName Name, FPhysicsControlMultiplier& ControlMultiplier) const;

	// @param Name     The name of the control to access.
	// @param Control  The control target, if found
	// @return         Returns true if the control was found, false if not
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool GetControlTarget(const FName Name, FPhysicsControlTarget& ControlTarget) const;

	// @param Name        The name of the control to access.
	// @return            Returns true if the control is enabled
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool GetControlEnabled(const FName Name) const;

	// Creates a new body modifier for mesh components
	//
	// @param Component The Mesh Component used as a target for the modifier
	// @param BoneName The bone name, if a skeletal mesh is used
	// @param Set Which set to include the body modifier in (optional). Note that it automatically
	//        gets added to the set "All"
	// @param BodyModifierData The initial properties of the modifier
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API FName CreateBodyModifier(
		UPrimitiveComponent*              Component,
		const FName                       BoneName,
		const FName                       Set,
		const FPhysicsControlModifierData BodyModifierData);

	// Creates a new body modifier for mesh components
	//
	// @param The name of the body modifier that will be created. Creation will fail if this name is already in use.
	// @param Component The Mesh Component used as a target for the modifier
	// @param BoneName The bone name, if a skeletal mesh is used
	// @param Set Which set to include the body modifier in (optional). Note that it automatically
	//        gets added to the set "All"
	// @param BodyModifierData The initial properties of the modifier
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool CreateNamedBodyModifier(
		const FName                       Name,
		UPrimitiveComponent*              Component,
		const FName                       BoneName,
		const FName                       Set,
		const FPhysicsControlModifierData BodyModifierData);

	// Creates new body modifiers for skeletal mesh components
	//
	// @param SkeletalMeshComponent The skeletal mesh which will have body modifiers
	// @param BoneName The bone name below which modifiers should be created
	// @param bIncludeSelf Whether or not to include BoneName when creating modifiers
	// @param Set Which set to include the body modifier in (optional). Note that it automatically
	//        gets added to the set "All"
	// @param BodyModifierData The initial properties of the modifier
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API TArray<FName> CreateBodyModifiersFromSkeletalMeshBelow(
		USkeletalMeshComponent*           SkeletalMeshComponent,
		const FName                       BoneName,
		const bool                        bIncludeSelf,
		const FName                       Set,
		const FPhysicsControlModifierData BodyModifierData);

	// Creates a collection of controls controlling a skeletal mesh, grouped together in limbs
	//
	// @param AllControls A single container for all the controls that have been created
	// @param LimbBones A map relating the limbs and the bones that they contain. Typically create this
	//        using GetLimbBonesFromSkeletalMesh
	// @param BodyModifierData The initial properties of the modifier
	//
	// @return A map containing the modifiers for each limb
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API TMap<FName, FPhysicsControlNames> CreateBodyModifiersFromLimbBones(
		FPhysicsControlNames&                        AllBodyModifiers,
		const TMap<FName, FPhysicsControlLimbBones>& LimbBones,
		const FPhysicsControlModifierData            BodyModifierData);

	// Destroys a body modifier or body modifiers in a set
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Name The name of the body modifier or set of body modifiers to destroy.
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	// @return Returns true if the body modifier or set can found/destroyed
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool DestroyBodyModifier(
		const FName Name, const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Destroys body modifiers or body modifiers in sets
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Names The names of the body modifiers or sets of body modifiers to destroy. Note that if you
	//        have these in a FPhysicsControlNameArray then it can be split.
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	// @return Returns true if all the body modifiers can be found/destroyed
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool DestroyBodyModifiers(
		const TArray<FName>& Names, const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Destroys body modifiers
	//
	// @param Set The set of body modifiers to destroy. Standard sets will include "All" and things like
	//        "ArmLeft", depending on how body modifiers have been created.
	// @return Returns true if the set and all its modifiers can be found/destroyed
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool DestroyBodyModifiersInSet(const FName Set);

	// Modifies an existing Body Modifier Data in a body modifier or body modifiers in a set
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Name The name of the modifier or set to update.
	// @param ModifierData The new data
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetBodyModifierData(
		const FName Name, const FPhysicsControlModifierData ModifierData, 
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Modifies existing body modifier data in body modifiers or body modifiers in sets
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Names The names of the modifiers or sets to update. Note that if you have these
	//        in a FPhysicsControlNameArray then it can be split.
	// @param ModifierData The new data
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifierDatas(
		const TArray<FName>& Names, const FPhysicsControlModifierData ModifierData,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Modifies existing Body Modifier Data
	//
	// @param Set The set of modifiers to update. Standard sets will include "All" and things like "ArmLeft",
	//        depending on how body modifiers have been created.
	// @param ModifierData The new data
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifierDatasInSet(const FName Set, const FPhysicsControlModifierData ModifierData);

	// Modifies an existing Body Modifier Data in a body modifier or body modifiers in a set
	//
	// @param Name The name of the modifier or set to update.
	// @param ModifierData The new data
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetBodyModifierSparseData(
		const FName Name, const FPhysicsControlModifierSparseData ModifierData,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Modifies existing Body Modifier Data in body modifiers or body modifiers in sets
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Names The names of the modifiers or sets to update. Note that if you have these in a FPhysicsControlNameArray
	//        then it can be split.
	// @param ModifierData The new data
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifierSparseDatas(
		const TArray<FName>& Names, const FPhysicsControlModifierSparseData ModifierData,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Modifies existing Body Modifier Data
	//
	// @param Set The set of modifiers to update. Standard sets will include "All" and things like "ArmLeft",
	//        depending on how body modifiers have been created.
	// @param ModifierData The new data
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifierSparseDatasInSet(const FName Set, const FPhysicsControlModifierSparseData ModifierData);

	// Sets the kinematic target transform for a body modifier.
	//
	// @param Name The name of the body modifier or set to update.
	// @param KinematicTargetPosition The position to use as the kinematic target of the associated body,
	//        if it is kinematic
	// @param KinematicTargetOrientation The orientation to use as the kinematic target of the associated body,
	//        if it is kinematic
	// @param bMakeKinematic If set then the body will be made kinematic. If not set, then it won't be changed.
	// @return true if the body modifier was found, false if not
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetBodyModifierKinematicTarget(
		const FName    Name,
		const FVector  KinematicTargetPosition, 
		const FRotator KinematicTargetOrienation,
		const bool     bMakeKinematic);

	// Sets the movement type for a body modifier or body modifiers in a set
	//
	// @param Name The name of the body modifier or set to update.
	// @param MovementType Whether to enable/disable simulation on the body
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetBodyModifierMovementType(
		const FName Name, const EPhysicsMovementType MovementType = EPhysicsMovementType::Simulated,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Sets the movement type for body modifiers or body modifiers in sets
	//
	// @param Names The names of the body modifiers or sets to update. Note that if you have
	//        these in a FPhysicsControlNameArray then it can be split.
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	// @param MovementType Whether to enable/disable simulation on the bodies
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifiersMovementType(
		const TArray<FName>& Names, const EPhysicsMovementType MovementType = EPhysicsMovementType::Simulated,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Sets the movement type for body modifiers
	//
	// @param Set The set of body modifiers to update. Standard sets will include "All" and things like
	//        "ArmLeft", depending on how body modifiers have been created.
	// @param MovementType Whether to enable/disable simulation on the bodies
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifiersInSetMovementType(
		const FName                Set,
		const EPhysicsMovementType MovementType = EPhysicsMovementType::Simulated);

	// Sets the collision type for a body modifier or body modifiers in a set
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Name The name of the body modifier or set to update.
	// @param CollisionType Collision type to set on the body
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetBodyModifierCollisionType(
		const FName Name, const ECollisionEnabled::Type CollisionType = ECollisionEnabled::QueryAndPhysics,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Sets the collision type for body modifiers or body modifiers in sets
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Names The names of the body modifiers or sets to update. Note that if you have
	//        these in a FPhysicsControlNameArray then it can be split.
	// @param CollisionType Collision type to set on the bodies
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifiersCollisionType(
		const TArray<FName>& Names, const ECollisionEnabled::Type CollisionType = ECollisionEnabled::QueryAndPhysics,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Sets the collision type for body modifiers
	//
	// @param Set The set of body modifiers to update. Standard sets will include "All" and things like
	//        "ArmLeft", depending on how body modifiers have been created.
	// @param CollisionType Collision type to set on the bodies
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifiersInSetCollisionType(
		const FName                   Set,
		const ECollisionEnabled::Type CollisionType = ECollisionEnabled::QueryAndPhysics);

	// Sets the gravity multiplier for a body modifier or body modifiers in a set
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Name The name of the body modifier or set to update.
	// @param GravityMultiplier The amount of gravity to apply when simulating
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetBodyModifierGravityMultiplier(
		const FName Name, const float GravityMultiplier = 1.0f,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Sets the gravity multiplier for body modifiers or body modifiers in sets
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Names The names of the body modifiers or sets to update. Note that if you have
	//        these in a FPhysicsControlNameArray then it can be split.
	// @param GravityMultiplier The amount of gravity to apply when simulating
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifiersGravityMultiplier(
		const TArray<FName>& Names, const float GravityMultiplier = 1.0f,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Sets the gravity multiplier for body modifiers
	//
	// @param Set The set of body modifiers to update. Standard sets will include "All" and things like
	//        "ArmLeft", depending on how body modifiers have been created.
	// @param GravityMultiplier The amount of gravity to apply when simulating
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifiersInSetGravityMultiplier(
		const FName Set,
		const float GravityMultiplier = 1.0f);

	// Sets the physics blend weight for a body modifier or body modifiers in a set
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Name The name of the body modifier or set to update.
	// @param PhysicsBlendWeight The blend weight between the body transform coming from
	//        animation and that coming from simulation.
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetBodyModifierPhysicsBlendWeight(
		const FName Name, const float PhysicsBlendWeight = 1.0f,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Sets the physics blend weight for body modifiers or body modifiers in sets
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Names The names of the body modifiers or sets to update. Note that if you have
	//        these in a FPhysicsControlNameArray then it can be split.
	// @param PhysicsBlendWeight The blend weight between the body transform coming from
	//        animation and that coming from simulation.
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifiersPhysicsBlendWeight(
		const TArray<FName>& Names, const float PhysicsBlendWeight = 1.0f,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Sets the physics blend weight for body modifiers
	//
	// @param Set The set of body modifiers to update. Standard sets will include "All" and things like
	//        "ArmLeft", depending on how body modifiers have been created.
	// @param PhysicsBlendWeight The blend weight between the body transform coming from
	//        animation and that coming from simulation.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifiersInSetPhysicsBlendWeight(
		const FName Set,
		const float PhysicsBlendWeight = 1.0f);

	// Sets what space kinematic targets should be applied in for a body modifier or body modifiers in a set
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Name The name of the body modifier or set to update.
	// @param KinematicTargetSpace The space in which the kinematic target should be applied.
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetBodyModifierKinematicTargetSpace(
		const FName Name, const EPhysicsControlKinematicTargetSpace KinematicTargetSpace,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Sets what space kinematic targets should be applied in for body modifiers or body modifiers in sets
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Names The names of the body modifiers or sets to update. Note that if you have
	//        these in a FPhysicsControlNameArray then it can be split.
	// @param KinematicTargetSpace The space in which the kinematic target should be applied.
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifiersKinematicTargetSpace(
		const TArray<FName>& Names, const EPhysicsControlKinematicTargetSpace KinematicTargetSpace,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Sets what space kinematic targets should be applied in
	//
	// @param Set The set of body modifiers to update. Standard sets will include "All" and things like
	//        "ArmLeft", depending on how body modifiers have been created.
	// @param KinematicTargetSpace The space in which the kinematic target should be applied.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifiersInSetKinematicTargetSpace(
		const FName                               Set,
		const EPhysicsControlKinematicTargetSpace KinematicTargetSpace);

	// Sets whether a body modifier should update kinematics from the simulation results, for a
	// body modifier or body modifiers in a set
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Name The name of the body modifier or set to update.
	// @param bUpdateKinematicFromSimulation Whether the body should be updated from the simulation when
	//        it is kinematic, or whether it should track the kinematic target directly. This will be most
	//        likely useful when using async physics, in order to make kinematic parts behave the same as
	//        dynamic ones.
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetBodyModifierUpdateKinematicFromSimulation(
		const FName Name, const bool  bUpdateKinematicFromSimulation,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Sets whether body modifiers should update kinematics from the simulation results for body
	// modifiers or body modifiers in sets
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Names The names of the body modifiers or sets to update. Note that if you have
	//        these in a FPhysicsControlNameArray then it can be split.
	// @param bUpdateKinematicFromSimulation Whether the body should be updated from the simulation when
	//        it is kinematic, or whether it should track the kinematic target directly. This will be most
	//        likely useful when using async physics, in order to make kinematic parts behave the same as
	//        dynamic ones.
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifiersUpdateKinematicFromSimulation(
		const TArray<FName>& Names, const bool bUpdateKinematicFromSimulation,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Sets whether body modifiers should update kinematics from the simulation results
	//
	// @param Set The set of body modifiers to update. Standard sets will include "All" and things like
	//        "ArmLeft", depending on how body modifiers have been created.
	// @param bUpdateKinematicFromSimulation Whether the body should be updated from the simulation when
	//        it is kinematic, or whether it should track the kinematic target directly. This will be most
	//        likely useful when using async physics, in order to make kinematic parts behave the same as
	//        dynamic ones.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API bool SetBodyModifiersInSetUpdateKinematicFromSimulation(
		const FName Set,
		const bool  bUpdateKinematicFromSimulation);

	// Creates a collections of controls and body modifiers for a character, based on the description passed in.
	// This makes:
	// - World-space controls
	// - Parent-space controls
	// - Body modifiers
	// for all the body parts. In addition, they get added to sets, so they can be referenced later. Each control
	// is added to three sets:
	// - "All"
	// - "ControlType - i.e. "WorldSpace" or "ParentSpace", each of which will end up containing all controls of that type
	// - "ControlType_LimbName" - e.g. "WorldSpace_ArmLeft" or "ParentSpace_Head"
	// Each body modifier is added to "All" and a set named after the limb - e.g. "Spine" or "LegRight".
	// It is also possible to specify a mesh component to use for the "world" object - so that the world controls can
	// be made to work in the space of another object (or a bone if that is a skeletal mesh component)
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool CreateControlsAndBodyModifiersFromLimbBones(
		FPhysicsControlNames&                       AllWorldSpaceControls,
		TMap<FName, FPhysicsControlNames>&          LimbWorldSpaceControls,
		FPhysicsControlNames&                       AllParentSpaceControls,
		TMap<FName, FPhysicsControlNames>&          LimbParentSpaceControls,
		FPhysicsControlNames&                       AllBodyModifiers,
		TMap<FName, FPhysicsControlNames>&          LimbBodyModifiers,
		USkeletalMeshComponent*                     SkeletalMeshComponent,
		const TArray<FPhysicsControlLimbSetupData>& LimbSetupData,
		const FPhysicsControlData                   WorldSpaceControlData,
		const FPhysicsControlData                   ParentSpaceControlData,
		const FPhysicsControlModifierData           BodyModifierData,
		UPrimitiveComponent*                        WorldComponent = nullptr,
		FName                                       WorldBoneName = NAME_None);

	// This uses the control profile asset (that should have already been assigned in our data) to create
	// controls and body modifiers.
	// It is also possible to specify a mesh component to use for the "world" object - so that the world controls can
	// be made to work in the space of another object (or a bone if that is a skeletal mesh component)
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool CreateControlsAndBodyModifiersFromPhysicsControlAsset(
		USkeletalMeshComponent* SkeletalMeshComponent,
		UPrimitiveComponent*    WorldComponent,
		FName                   WorldBoneName);

	// Looks up the profile which should exist in the registered control profile asset, and invokes it.
	//
	// @param ProfileName The name of the control profile to apply
	// @param ControlSetMask The optional name of a control set to use as a mask, so only controls
	//        within that set will be modified. Standard control set names will include "All",
	//        "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft", depending on how
	//        controls have been created. If unset or invalid, no mask will be applied.
	// @param BodyModifierSetMask The optional name of a body modifier set to use as a mask, so only
	//        body modifiers within that set will be modified. Standard body modifier sets will
	//        include "All" and things like "ArmLeft", depending on how body modifiers have been
	//        created. If unset or invalid, no mask will be applied.
	// @return true if successful, and false if the profile cannot be found.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool InvokeControlProfile(
		const FName ProfileName, 
		const FName ControlSetMask = TEXT(""),
		const FName BodyModifierSetMask = TEXT(""));

	// Adds a Control to a Set. This will add a new set if necessary. For example, you might
	// make a set of Controls called "ParentSpace_Feet" by calling this twice, passing in the left and right
	// foot ParentSpace controls.
	//
	// @return The new/updated set of controls, in case you want to store it
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API void AddControlToSet(FPhysicsControlNames& NewSet, const FName Control, const FName Set);

	// Adds Controls to a Set. This will add a new set if necessary. For example, you might
	// make a set of ParentSpace Arm controls by calling this twice, passing in the left and right
	// arm ParentSpace controls.
	//
	// @return The new/updated set of controls, in case you want to store it
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API void AddControlsToSet(FPhysicsControlNames& NewSet, const TArray<FName>& Controls, const FName Set);

	// Returns a reference to all the control names that have been created.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API const TArray<FName>& GetAllControlNames() const;

	// Returns a reference to all the control names that have been created and are in the specified
	// set, which could be a limb, or a subsequently created set. Standard sets will include "All", "WorldSpace",
	// "ParentSpace" and things like "WorldSpace_ArmLeft", depending on how controls have been created.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API const TArray<FName>& GetControlNamesInSet(const FName Set) const;

	// Returns a reference to all the body modifier names that have been created.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API const TArray<FName>& GetAllBodyModifierNames() const;

	// Returns a reference to all the body modifier names that have been created and are in the specified
	// set, which could be a limb, or a subsequently created set. Standard sets will include "All" and things like
	// "ArmLeft", depending on how body modifiers have been created.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API const TArray<FName>& GetBodyModifierNamesInSet(const FName Set) const;

	// Adds a body modifier to a Set. This will add a new set if necessary. For example, you might
	// make a set of body modifiers called "Feet" by calling this twice, passing in the left and right
	// foot body modifiers.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API void AddBodyModifierToSet(FPhysicsControlNames& NewSet, const FName BodyModifier, const FName Set);

	// Adds body modifiers to a Set. This will add a new set if necessary. For example, you might
	// make a set of Arm body modifiers by calling this twice, passing in the left and right
	// arm body modifiers.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API void AddBodyModifiersToSet(FPhysicsControlNames& NewSet, const TArray<FName>& BodyModifiers, const FName Set);

	// Returns the names of all sets containing the control (may be empty - e.g. if it doesn't exist)
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API TArray<FName> GetSetsContainingControl(const FName Control) const;

	// Returns the names of all sets containing the body modifier (may be empty - e.g. if it doesn't exist)
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API TArray<FName> GetSetsContainingBodyModifier(const FName Control) const;

	// Gets the transforms of the requested bones that will be used as targets (in world space). Targets for bones
	// that are not found will be set to Identity. Note that these targets will have been calculated and cached
	// at the start of the Physics Control Component, so if using the built in tick, may be too old to be useful.
	// If you manually update the component then you can access these target transforms prior to applying your
	// own targets.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API TArray<FTransform> GetCachedBoneTransforms(
		const USkeletalMeshComponent* SkeletalMeshComponent,
		const TArray<FName>&          BoneNames);

	// Gets the positions of the requested bones that will be used as targets (in world space). Targets for bones
	// that are not found will be set to zero. Note that these targets will have been calculated and cached
	// at the start of the Physics Control Component, so if using the built in tick, may be too old to be useful.
	// If you manually update the component then you can access these target transforms prior to applying your
	// own targets.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API TArray<FVector> GetCachedBonePositions(
		const USkeletalMeshComponent* SkeletalMeshComponent,
		const TArray<FName>&          BoneNames);

	// Gets the orientations of the requested bones that will be used as targets (in world space). Targets for bones
	// that are not found will be set to identity. Note that these targets will have been calculated and cached
	// at the start of the Physics Control Component, so if using the built in tick, may be too old to be useful.
	// If you manually update the component then you can access these target transforms prior to applying your
	// own targets.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API TArray<FRotator> GetCachedBoneOrientations(
		const USkeletalMeshComponent* SkeletalMeshComponent,
		const TArray<FName>&          BoneNames);

	// Gets the transforms of the requested bone that will be used as a target (in world space). Targets for bones
	// that are not found will be set to Identity. Note that these targets will have been calculated and cached
	// at the start of the Physics Control Component, so if using the built in tick, may be too old to be useful.
	// If you manually update the component then you can access these target transforms prior to applying your
	// own targets.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API FTransform GetCachedBoneTransform(
		const USkeletalMeshComponent* SkeletalMeshComponent,
		const FName                   BoneName);

	// Gets the position of the requested bone that will be used as a target (in world space). Targets for bones
	// that are not found will be set to zero. Note that these targets will have been calculated and cached
	// at the start of the Physics Control Component, so if using the built in tick, may be too old to be useful.
	// If you manually update the component then you can access these target transforms prior to applying your
	// own targets.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API FVector GetCachedBonePosition(
		const USkeletalMeshComponent* SkeletalMeshComponent,
		const FName                   BoneName);

	// Gets the orientation of the requested bone that will be used as a target (in world space). Targets for bones
	// that are not found will be set to Identity. Note that these targets will have been calculated and cached
	// at the start of the Physics Control Component, so if using the built in tick, may be too old to be useful.
	// If you manually update the component then you can access these target transforms prior to applying your
	// own targets.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API FRotator GetCachedBoneOrientation(
		const USkeletalMeshComponent* SkeletalMeshComponent,
		const FName                   BoneName);

	// This allows the caller to override the target that will have been calculated and cached at the start of
	// the Physics Control Component update. This is unlikely to be useful when using the built in tick,
	// but if you are manually updating the component then you may wish to call this after UpdateTargetCaches
	// but before UpdateControls.
	//
	// @return true if successful, and false if no cached target can be found for the bone.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool SetCachedBoneData(
		const USkeletalMeshComponent* SkeletalMeshComponent,
		const FName                   BoneName, 
		const FTransform&             TM);

	// This sets cached bone velocities to zero, by forgetting any previously used transform. This may be useful if
	// the pose is changed suddenly and you don't want this change to show up in the simulated velocities.
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API void SetCachedBoneVelocitiesToZero();

	// This flags the body associated with the modifier, or modifiers in a set, to set (using
	// teleport) its position and velocity to the cached animation target. This will only affect
	// skeletal mesh component bodies.
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	//
	// @param Name The name of the body modifier or set of body modifiers to use to identify the body/bodies to reset.
	// @param Behavior When the reset should happen.
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API void ResetBodyModifierToCachedBoneTransform(
		const FName Name, const EResetToCachedTargetBehavior Behavior = EResetToCachedTargetBehavior::ResetImmediately,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Calls ResetBodyModifierToCachedTarget for each of the body modifiers or body modifiers in the sets
	// Standard sets will include "All" and things like "ArmLeft", depending on how body modifiers have been created.
	// @param bApplyToModifiersWithName If true, then body modifiers with the name will be modified
	// @param bApplyToSetsWithName If true, then body modifiers in sets with the name will be modified
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API void ResetBodyModifiersToCachedBoneTransforms(
		const TArray<FName>& Names, const EResetToCachedTargetBehavior Behavior = EResetToCachedTargetBehavior::ResetImmediately,
		const bool bApplyToModifiersWithName = true, const bool bApplyToSetsWithName = true);

	// Calls ResetBodyModifierToCachedTarget for each of the body modifiers in the set
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API void ResetBodyModifiersInSetToCachedBoneTransforms(
		const FName                        SetName,
		const EResetToCachedTargetBehavior Behavior = EResetToCachedTargetBehavior::ResetImmediately);

	// Indicates if a control with the name exists (doesn't produce a warning if it doesn't)
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool GetControlExists(const FName Name) const;

	// Indicates if a body modifier with the name exists (doesn't produce a warning if it doesn't)
	UFUNCTION(BlueprintCallable, Category = PhysicsControl, Meta = (ReturnDisplayName = "Success"))
	UE_API bool GetBodyModifierExists(const FName Name) const;

	// Logs all controls and control sets to console and log file
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API void LogControls() const;

	// Logs all body modifiers and body modifier sets to console and log file
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API void LogBodyModifiers() const;

	// Logs all controls and body modifiers and their sets to console and log file
	UFUNCTION(BlueprintCallable, Category = PhysicsControl)
	UE_API void LogControlsAndBodyModifiers() const;

public:
	// Public property data

	// Optional PhysicsControlAsset to use. If this is set, then controls can be instantiated from the asset.
	// Individual profiles can subsequently be "invoked" referencing the named profiles in the asset.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControlAsset)
	TSoftObjectPtr<UPhysicsControlAsset> PhysicsControlAsset;

	// If the component moves by more than this distance then it is treated as a teleport,
	// which prevents velocities being used for a frame. It is also used as the threshold for
	// teleporting when moving kinematic objects. Zero or negative disables.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Teleport)
	float TeleportDistanceThreshold = 300.0f;

	// If the component rotates by more than this angle (in degrees) then it is treated as a teleport,
	// which prevents velocities being used for a frame. It is also used as the threshold for
	// teleporting when moving kinematic objects. Zero or negative disables.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Teleport)
	float TeleportRotationThreshold = 0.0f;

	// Visualize the controls when this component is selected
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Debug)
	bool bShowDebugVisualization = true;

	// Size of the gizmos etc used during visualization
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Debug)
	float VisualizationSizeScale = 5.0f;

	// Display all the controls and their basic properties when this component is selected
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Debug)
	bool bShowDebugControlList = false;

	// Display detailed info for controls containing this string (if non-empty) when this component is selected
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Debug)
	FString DebugControlDetailFilter;

	// Display all the body modifiers and their basic properties when this component is selected
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Debug)
	bool bShowDebugBodyModifierList = false;

	// Display detailed info for body modifiers containing this string (if non-empty) when this component is selected
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Debug)
	FString DebugBodyModifierDetailFilter;

	// The time used when "predicting" the target position/orientation. Zero will disable the visualization
	// of this.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Debug)
	float VelocityPredictionTime = 0.2f;

	// Warn if an an invalid control or body modifier name is used. This can happen quite easily since
	// they're only referenced through names, which are likely auto-generated. However, it may happen for
	// valid reasons, in which case you'll want to disable this.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Debug)
	bool bWarnAboutInvalidNames = true;

	// Upper limit on the number of controls or modifiers that will be created using the same name (which
	// will get a numerical postfix). When this limit is reached a warning will be issued  and the control 
	// or modifier won't be created. This is to avoid problems if controls or modifiers are being created 
	// dynamically, and can generally be a "moderately large" number, depending on how many controls or 
	// modifiers you expect to create.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	int32 MaxNumControlsOrModifiersPerName = 256;

	// If set then controls that have been disabled due to the bodies being set to NoCollision (for
	// example) will be reinitialized, if possible, when those bodies are returned to the simulation.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	bool bAttemptToRecreateDisabledControls = false;

public:
	//Begin ActorComponent interface
	UE_API virtual void OnRegister() override;
	UE_API virtual bool ShouldCreatePhysicsState() const override;
	UE_API virtual void OnCreatePhysicsState() override;
	UE_API virtual void OnDestroyPhysicsState() override;
	//End ActorComponent interface

#if WITH_EDITOR
	// Used by the component visualizer
	UE_API virtual void DebugDraw(FPrimitiveDrawInterface* PDI) const;
	UE_API virtual void DebugDrawControl(FPrimitiveDrawInterface* PDI, const FPhysicsControlRecord& Record, const FName ControlName) const;
#endif

protected:

	// Begin UActorComponent Interface
	UE_API virtual void InitializeComponent() override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	// End UActorComponent Interface

	// Terminates everything we have made in physics
	UE_API void DestroyPhysicsState();

	// Retrieves the bone data for the specified bone given the skeletal mesh component.
	//
	// @param OutBoneData If successful, this will contain the output bone data.
	// @param OutPoseData If successful, this will contain the pose data structure, in case more info is required
	// @param InSkeletalMeshComponent Required to be a valid pointer to a skeletal mesh component
	// @param InBoneName The name of the bone to retrieve data for
	// @return true if the bone and data were found, false if not (in which case warnings will be logged)
	UE_API bool GetBoneData(
		UE::PhysicsControl::FBoneData&                      OutBoneData,
		const UE::PhysicsControl::FPhysicsControlPoseData*& OutPoseData,
		const USkeletalMeshComponent*                       InSkeletalMeshComponent,
		const FName                                         InBoneName) const;

	// Retrieves the bone data for the specified bone given the skeletal mesh component, for modification
	//
	// @param OutBoneData If successful, this will point to the bone data.
	// @param InSkeletalMeshComponent Required to be a valid pointer to a skeletal mesh component
	// @param InBoneName The name of the bone to retrieve data for
	// @return true if the bone and data were found, false if not (in which case warnings will be logged)
	UE_API bool GetModifiableBoneData(
		UE::PhysicsControl::FBoneData*&      OutBoneData,
		const USkeletalMeshComponent*        InSkeletalMeshComponent,
		const FName                          InBoneName);

	// Retrieves the control for the name. Returns a null pointer if the name cannot be found
	UE_API FPhysicsControl* FindControl(const FName Name);
	UE_API const FPhysicsControl* FindControl(const FName Name) const;

	// Retrieves the control record for the name. Returns a null pointer if the name cannot be found
	UE_API FPhysicsControlRecord* FindControlRecord(const FName Name);
	UE_API const FPhysicsControlRecord* FindControlRecord(const FName Name) const;

	// Updates the world-space bone positions etc for each skeleton we're tracking
	UE_API void UpdateCachedSkeletalBoneData(float DeltaTime);

	// Terminates the underlying physical constraints, resets our internal stored state for each control,
	// and optionally deletes all record of the controls.
	UE_API void ResetControls(bool bKeepControlRecords);

	// Starts caching skeletal mesh poses, and registers for a tick pre-requisite
	UE_API void AddSkeletalMeshReferenceForCaching(USkeletalMeshComponent* SkeletalMeshComponent);

	// Stops caching skeletal mesh poses (if this is the last one), and deregisters for a tick pre-requisite.
	// Returns true if this was the last reference, false otherwise.
	UE_API bool RemoveSkeletalMeshReferenceForCaching(USkeletalMeshComponent* SkeletalMeshComponent);

	// Records that a modifier is working with the skeletal mesh and stores original data if necessary
	UE_API void AddSkeletalMeshReferenceForModifier(USkeletalMeshComponent* SkeletalMeshComponent);

	// Records that a modifier has stopped working with the skeletal mesh and restores original data if necessary.
	// Returns true if this was the last reference, false otherwise.
	UE_API bool RemoveSkeletalMeshReferenceForModifier(USkeletalMeshComponent* SkeletalMeshComponent);

	// Update the constraint based on the record. PhysicsAssetConstraintInstance and
	// DefaultConstraintInstance are the live and authored constraint instances for this bone pair;
	// both are null when the control does not use skeletal animation or no matching constraint was
	// found in the physics asset.
	UE_API void ApplyControl(
		FPhysicsControlRecord&     Record,
		FConstraintInstance*       PhysicsAssetConstraintInstance,
		const FConstraintInstance* DefaultConstraintInstance);

	// Updates the constraint strengths. Returns true if there is some strength,
	// false otherwise (e.g. to skip setting targets)
	UE_API bool ApplyControlStrengths(FPhysicsControlRecord& Record, FConstraintInstance* ConstraintInstance);

	// Calculates the Target TM and velocities that will get passed to the constraint - so this
	// is a target that is defined in the space of the parent body (or in world space, if it doesn't exist).
	UE_API void CalculateControlTargetData(
		FTransform&                  OutTargetTM, 
		FTransform&                  OutSkeletalTargetTM,
		FVector&                     OutTargetVelocity,
		FVector&                     OutTargetAngularVelocity,
		const FPhysicsControlRecord& Record,
		bool                         bUsePreviousSkeletalTargetTM) const;

	// Updates the body based on the modifier
	UE_API void ApplyBodyModifier(FPhysicsBodyModifierRecord& BodyModifier);

	// This will set the kinematic target for the appropriate body based on the weighted target position and
	// orientation (and whether any were found) for any controls that are related to the body modifier.
	UE_API void ApplyKinematicTarget(const FPhysicsBodyModifierRecord& BodyModifier) const;

	// Sets the body (simulated or kinematic) to have the position/velocity etc state that has been cached.
	// Has no effect if there is no cached target data.
	UE_API void ResetToCachedTarget(const FPhysicsBodyModifierRecord& BodyModifier) const;

	// Retrieves the body modifier for the name. Note that if Name is blank then the first modifier will be returned,
	// assuming there is one.
	UE_API FPhysicsBodyModifierRecord* FindBodyModifierRecord(const FName Name);
	UE_API const FPhysicsBodyModifierRecord* FindBodyModifierRecord(const FName Name) const;

	// When destroying a control or modifier, the record will normally be removed, but it can be retained if you
	// will subsequently update the records
	enum class EDestroyBehavior : uint8
	{
		KeepRecord,
		RemoveRecord
	};

	// Destroys the control. It will optionally be removed from the array of records too
	UE_API bool DestroyControl(const FName Name, const EDestroyBehavior DestroyBehavior);

	// Destroys the modifier. It will optionally be removed from the array of records too
	UE_API bool DestroyBodyModifier(const FName Name, const EDestroyBehavior DestroyBehavior);

	// Applies the updates to controls and body modifiers
	//
	// @param ControlAndModifierUpdates The updates to apply
	// @param ControlSetMask The optional name of a control set to use as a mask, so only controls
	//        within that set will be modified. Standard control set names will include "All",
	//        "WorldSpace", "ParentSpace" and things like "WorldSpace_ArmLeft", depending on how
	//        controls have been created. If unset or invalid, no mask will be applied.
	// @param BodyModifierSetMask The optional name of a body modifier set to use as a mask, so only
	//        body modifiers within that set will be modified. Standard body modifier sets will
	//        include "All" and things like "ArmLeft", depending on how body modifiers have been
	//        created. If unset or invalid, no mask will be applied.
	UE_API void ApplyControlAndModifierUpdates(
		const FPhysicsControlControlAndModifierUpdates& ControlAndModifierUpdates, 
		const FName                                     ControlSetMask = TEXT(""),
		const FName                                     BodyModifierSetMask = TEXT(""));

protected:

	// Cached transforms from each skeletal mesh we're working with. Will be updated at the
	// beginning of each tick.
	TMap<TWeakObjectPtr<USkeletalMeshComponent>, UE::PhysicsControl::FPhysicsControlPoseData> CachedPoseDatas;

	// Track which skeletons have been affected by a body modifier - some settings get overridden
	// and then need to be restored when the last body modifier is destroyed.
	TMap<TWeakObjectPtr<USkeletalMeshComponent>, FModifiedSkeletalMeshData> ModifiedSkeletalMeshDatas;

	TMap<FName, FPhysicsControlRecord>      ControlRecords;
	TMap<FName, FPhysicsBodyModifierRecord> BodyModifierRecords;

	// Keep track of the names of everything we have created
	FPhysicsControlNameRecords NameRecords;

	// Update counter - incremented every tick, and used to check whether previous targets etc are
	// valid when calculating velocities.
	FGraphTraversalCounter CurrentUpdateCounter;
};

#undef UE_API
