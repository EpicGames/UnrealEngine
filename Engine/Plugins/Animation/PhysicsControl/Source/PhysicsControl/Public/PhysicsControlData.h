// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Components/ActorComponent.h"
#include "Components/MeshComponent.h"
#include "EngineDefines.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsDeclares.h"
#include "PhysicsControlData.generated.h"

#define UE_API PHYSICSCONTROL_API

// Note that this file defines structures that mostly are, or could be, shared between the PhysicsControlComponent,
// the RigidBodyWithControl node, and Control Rig physics.

// Used by Body Modifiers to specify how the physical bodies should move.
UENUM(BlueprintType)
enum class EPhysicsMovementType : uint8
{
	// Static means that the object won't be simulated, and it won't be moved according to the
	// kinematic target set in the Body Modifier (though something else might move it)
	Static,
	// Kinematic means that the object won't be simulated, but will be moved according to the
	// kinematic target set in the Body Modifier.
	Kinematic,
	// Simulated means that the object will be controlled by the physics solver
	Simulated,
	// Default means that the movement type shouldn't be changed - for example, it will use the
	// value in the physics asset
	Default
};

inline FName GetPhysicsMovementTypeName(const EPhysicsMovementType MovementType)
{
	switch (MovementType)
	{
	case EPhysicsMovementType::Static:
		return "Static";
	case EPhysicsMovementType::Kinematic:
		return "Kinematic";
	case EPhysicsMovementType::Simulated:
		return "Simulated";
	case EPhysicsMovementType::Default:
		return "Default";
	}
	return "None";
}

// Specifies the type of control that is created when making controls from a skeleton or a set of limbs.
// Note that if controls are made individually then other options are available - i.e. in a character,
// any body part can be controlled relative to any other part, or indeed any other object.
UENUM(BlueprintType)
enum class EPhysicsControlType : uint8
{
	// Control is done in world space, so each object/part is driven independently
	WorldSpace,
	// Control is done in the space of the parent of each object
	ParentSpace,
};

inline FName GetPhysicsControlTypeName(const EPhysicsControlType ControlType)
{
	switch (ControlType)
	{
	case EPhysicsControlType::WorldSpace:
		return "WorldSpace";
	case EPhysicsControlType::ParentSpace:
		return "ParentSpace";
	}
	return "None";
}

// Update an existing set, or add to it
USTRUCT(BlueprintType)
struct FPhysicsControlSetUpdate
{
	GENERATED_BODY();

	bool operator==(const FPhysicsControlSetUpdate& other) const = default;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName SetName;

	// The names of either controls or body modifiers (depending on context), or sets of controls/body modifiers
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TArray<FName> Names;
};

// Combines updates for control and modifier sets
USTRUCT(BlueprintType)
struct FPhysicsControlSetUpdates
{
	GENERATED_BODY();

	UE_API FPhysicsControlSetUpdates& operator+=(const FPhysicsControlSetUpdates& Other);

	bool operator==(const FPhysicsControlSetUpdates& other) const = default;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TArray<FPhysicsControlSetUpdate> ControlSetUpdates;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TArray<FPhysicsControlSetUpdate> ModifierSetUpdates;
};

// Specifies what space a kinematic target is defined in.
UENUM(BlueprintType)
enum class EPhysicsControlKinematicTargetSpace : uint8
{
	// Target is in world space. For a control rig, this is the space of the control rig, rather
	// than the "real" world.
	World,
	// If a skeleton/bone is being used, the target is in the space of that component. If not then
	// it is in the space of the owning component (e.g. PhysicsControl, or ControlRig etc).
	Component,
	// If a skeleton/bone is being used, the target is specified relative to the bone that controls
	// the (kinematic) body, with the offset applied in the space of that body. If not, then this is
	// the same as World.
	OffsetInBoneSpace,
	// If a skeleton/bone is being used, the target is specified relative to the bone that controls
	// the (kinematic) body, with the offset applied in world space. If not, then this is
	// the same as World.
	OffsetInWorldSpace,
	// If a skeleton/bone is being used, the target is specified relative to the bone that controls
	// the (kinematic) body, with the offset applied in component space. If not, then this is
	// the same as Component.
	OffsetInComponentSpace,
	// Ignore the specified target, and just track the bone/object
	IgnoreTarget
};

// Analogous to the ControlData, this indicates how an individual controlled body should move, with flags indicating
// whether each element should get used.
USTRUCT(BlueprintType)
struct FPhysicsControlModifierSparseData
{
	GENERATED_BODY();

	FPhysicsControlModifierSparseData(
		const EPhysicsMovementType                InMovementType = EPhysicsMovementType::Simulated,
		const ECollisionEnabled::Type             InCollisionType = ECollisionEnabled::QueryAndPhysics,
		const float                               InGravityMultiplier = 1.0f,
		const float                               InPhysicsBlendWeight = 1.0f,
		const EPhysicsControlKinematicTargetSpace InKinematicTargetSpace = EPhysicsControlKinematicTargetSpace::OffsetInBoneSpace,
		const bool                                InUpdateKinematicFromSimulation = true,
		const bool                                InEnableCCD = false)
		: MovementType(InMovementType)
		, CollisionType(InCollisionType)
		, GravityMultiplier(InGravityMultiplier)
		, PhysicsBlendWeight(InPhysicsBlendWeight)
		, KinematicTargetSpace(InKinematicTargetSpace)
		, bUpdateKinematicFromSimulation(InUpdateKinematicFromSimulation)
		, bEnableCCD(InEnableCCD)
		// The flags
		, bEnableMovementType(true)
		, bEnableCollisionType(true)
		, bEnableGravityMultiplier(true)
		, bEnablePhysicsBlendWeight(true)
		, bEnableKinematicTargetSpace(true)
		, bEnablebUpdateKinematicFromSimulation(true)
		, bEnablebEnableCCD(true)
	{
	}

	bool operator==(const FPhysicsControlModifierSparseData& other) const = default;

	// How the body should move.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (editcondition = "bEnableMovementType"))
	EPhysicsMovementType MovementType;

	// The collision type. Note that PhysicsControlComponent uses the full filtering, but
	// RigidBodyWithControl just treats this as enable/disable collision
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (editcondition = "bEnableCollisionType"))
	TEnumAsByte<ECollisionEnabled::Type> CollisionType;

	// How much gravity to use when simulating - typically between 0 and 1
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (editcondition = "bEnableGravityMultiplier"))
	float GravityMultiplier;

	// When converting back from simulation to animation, how much to use the simulation output
	// versus the original animation input.
	// Note - this is not used in RBWC
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (editcondition = "bEnablePhysicsBlendWeight"))
	float PhysicsBlendWeight;

	// The space in which kinematic targets should operate
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (editcondition = "bEnableKinematicTargetSpace"))
	EPhysicsControlKinematicTargetSpace KinematicTargetSpace = EPhysicsControlKinematicTargetSpace::OffsetInBoneSpace;

	// If true then the associated actor's transform will be updated from the simulation when it is
	// kinematic. This is most likely useful when using async physics in order to prevent different
	// parts of the skeleton from being torn apart. Note - this is not used in RBWC
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (editcondition = "bEnablebUpdateKinematicFromSimulation"))
	uint8 bUpdateKinematicFromSimulation : 1;

	// Enable/disable CCD. Note that this can be expensive, so disable it when not needed!
	// Requires CCD to be allowed in the solver.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (editcondition = "bEnablebEnableCCD"))
	uint8 bEnableCCD : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableMovementType : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableCollisionType : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableGravityMultiplier : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnablePhysicsBlendWeight : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableKinematicTargetSpace : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnablebUpdateKinematicFromSimulation : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnablebEnableCCD : 1;
};

// Specifies a space for queries, forces etc
UENUM(BlueprintType)
enum class EPhysicsControlSpace : uint8
{
	// World space. For a control rig, this is the space of the control rig, rather than the "real"
	// world, so is the same as Component.
	World,
	// If a skeleton/bone is being used, the target is in the space of that component. If not then
	// it is in the space of the owning component (e.g. PhysicsControl, or ControlRig etc).
	Component,
	// In the space of the body under consideration
	Body
};

// Analogous to the ControlData, this indicates how an individual controlled body should move
USTRUCT(BlueprintType)
struct FPhysicsControlModifierData
{
	GENERATED_BODY();

	FPhysicsControlModifierData(
		const EPhysicsMovementType                InMovementType = EPhysicsMovementType::Simulated,
		const ECollisionEnabled::Type             InCollisionType = ECollisionEnabled::QueryAndPhysics,
		const float                               InGravityMultiplier = 1.0f,
		const float                               InPhysicsBlendWeight = 1.0f,
		const EPhysicsControlKinematicTargetSpace InKinematicTargetSpace = EPhysicsControlKinematicTargetSpace::OffsetInBoneSpace,
		const bool                                InUpdateKinematicFromSimulation = true,
		const bool                                InEnableCCD = false)
		: MovementType(InMovementType)
		, CollisionType(InCollisionType)
		, KinematicTargetSpace(InKinematicTargetSpace)
		, GravityMultiplier(InGravityMultiplier)
		, PhysicsBlendWeight(InPhysicsBlendWeight)
		, bUpdateKinematicFromSimulation(InUpdateKinematicFromSimulation)
		, bEnableCCD(InEnableCCD)
	{
	}

	bool operator==(const FPhysicsControlModifierData& other) const = default;

	friend UE_API FArchive& operator <<(FArchive& Ar, FPhysicsControlModifierData& Data);

	UE_API void UpdateFromSparseData(const FPhysicsControlModifierSparseData& SparseData);

	// How the body should move.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	EPhysicsMovementType MovementType = EPhysicsMovementType::Simulated;

	// The collision type. Note that PhysicsControlComponent uses the full filtering, but
	// RigidBodyWithControl just treats this as enable/disable collision
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TEnumAsByte<ECollisionEnabled::Type> CollisionType = ECollisionEnabled::QueryAndPhysics;

	// The space in which kinematic targets should operate. Note that some target spaces are more
	// expensive than others due to the calculations: IgnoreTarget is the cheapest,
	// OffsetInBoneSpace and OffsetInComponentSpace are cheap, but OffsetInWorldSpace is relatively expensive.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Physics)
	EPhysicsControlKinematicTargetSpace KinematicTargetSpace = EPhysicsControlKinematicTargetSpace::OffsetInBoneSpace;

	// How much gravity to use when simulating - typically between 0 and 1
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	float GravityMultiplier = 1.0f;

	// When converting back from simulation to animation, how much to use the simulation output
	// versus the original animation input.
	// Note - this is not used in RBWC
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	float PhysicsBlendWeight = 1.0f;

	// If true then the associated actor's transform will be updated from the simulation when it is
	// kinematic. This is most likely useful when using async physics in order to prevent different
	// parts of the skeleton from being torn apart.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	bool bUpdateKinematicFromSimulation;

	// Enable/disable CCD. Note that this can be expensive, so disable it when not needed!
	// Requires CCD to be allowed in the solver.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	bool bEnableCCD;
};

// Represents a single/individual body modifier, with the modifier data
USTRUCT(BlueprintType)
struct FPhysicsBodyModifier
{
	GENERATED_BODY();

	FPhysicsBodyModifier() {}

	FPhysicsBodyModifier(FName InBoneName, const FPhysicsControlModifierData& InModifierData)
		: BoneName(InBoneName), ModifierData(InModifierData) {}

	bool operator==(const FPhysicsBodyModifier& other) const = default;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName BoneName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsControlModifierData ModifierData;
};

// Used on creation, to allow requesting the modifier to be in certain sets
USTRUCT(BlueprintType)
struct FPhysicsBodyModifierCreationData
{
	GENERATED_BODY();

	bool operator==(const FPhysicsBodyModifierCreationData& other) const = default;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsBodyModifier Modifier;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TArray<FName> Sets;
};

// Strength and damping etc parameters that will affect a control, with flags indicating
// whether each element should get used.
USTRUCT(BlueprintType)
struct FPhysicsControlSparseData
{
	GENERATED_BODY();

	FPhysicsControlSparseData()
		: LinearStrength(0)
		, LinearDampingRatio(1)
		, LinearExtraDamping(0)
		, MaxForce(0)
		, AngularStrength(0)
		, AngularDampingRatio(1)
		, AngularExtraDamping(0)
		, MaxTorque(0)
		, LinearTargetVelocityMultiplier(1)
		, AngularTargetVelocityMultiplier(1)
		, CustomControlPoint(FVector::ZeroVector)
		, bEnabled(true)
		, bUseCustomControlPoint(false)
		, bUseSkeletalAnimation(true)
		, bDisableCollision(false)
		, bOnlyControlChildObject(false)
		, bUseAccelerationDriveMode(true)
		// the flags
		, bEnableLinearStrength(true)
		, bEnableLinearDampingRatio(true)
		, bEnableLinearExtraDamping(true)
		, bEnableMaxForce(true)
		, bEnableAngularStrength(true)
		, bEnableAngularDampingRatio(true)
		, bEnableAngularExtraDamping(true)
		, bEnableMaxTorque(true)
		, bEnableLinearTargetVelocityMultiplier(true)
		, bEnableAngularTargetVelocityMultiplier(true)
		, bEnableCustomControlPoint(true)
		, bEnablebEnabled(true)
		, bEnablebUseCustomControlPoint(true)
		, bEnablebUseSkeletalAnimation(true)
		, bEnablebDisableCollision(true)
		, bEnablebOnlyControlChildObject(true)
		, bEnablebUseAccelerationDriveMode(true)
	{
	}

	bool operator==(const FPhysicsControlSparseData& other) const = default;

	// The strength used to drive linear motion
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableLinearStrength"))
	float LinearStrength;

	// The amount of damping associated with the linear strength. A value of 1 Results in critically
	// damped motion where the control drives as quickly as possible to the target without overshooting.
	// Values > 1 result in more damped motion, and values below 1 result in faster, but more "wobbly" motion.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableLinearDampingRatio"))
	float LinearDampingRatio;

	// The amount of additional linear damping. This is added to the damping that comes from LinearDampingRatio
	// and can be useful when you want damping even when LinearStrength is zero.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableLinearExtraDamping"))
	float LinearExtraDamping;

	// The maximum force used to drive the linear motion. Zero indicates no limit.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableMaxForce"))
	float MaxForce;

	// The strength used to drive angular motion
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableAngularStrength"))
	float AngularStrength;

	// The amount of damping associated with the angular strength. A value of 1 Results in critically
	// damped motion where the control drives as quickly as possible to the target without overshooting.
	// Values > 1 result in more damped motion, and values below 1 result in faster, but more "wobbly" motion.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableAngularDampingRatio"))
	float AngularDampingRatio;

	// The amount of additional angular damping. This is added to the damping that comes from AngularDampingRatio
	// and can be useful when you want damping even when AngularStrength is zero.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableAngularExtraDamping"))
	float AngularExtraDamping;

	// The maximum torque used to drive the angular motion. Zero indicates no limit.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableMaxTorque"))
	float MaxTorque;

	// Multiplier on the linear velocity, which gets applied to the damping. A value of 1 means the animation target
	// velocity is used, which helps it track the animation. A value of 0 means damping happens in "world space"
	// - so damping acts like drag on the movement.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableLinearTargetVelocityMultiplier"))
	float LinearTargetVelocityMultiplier;

	// Multiplier on the angular velocity, which gets applied to the damping. A value of 1 means the animation target
	// velocity is used, which helps it track the animation. A value of 0 means damping happens in "world space"
	// - so damping acts like drag on the movement.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableAngularTargetVelocityMultiplier"))
	float AngularTargetVelocityMultiplier;

	// The position of the control point relative to the child mesh, when using a custom control point.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableCustomControlPoint"))
	FVector CustomControlPoint;

	// Whether this control should be enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnablebEnabled"))
	uint8 bEnabled : 1;

	// Whether or not to use the custom control point position
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnablebUseCustomControlPoint"))
	uint8 bUseCustomControlPoint : 1;

	// If true then the target will be applied on top of the skeletal animation (if there is any)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnablebUseSkeletalAnimation"))
	uint8 bUseSkeletalAnimation : 1;

	// Whether or not this control should disable collision between the parent and child bodies (only
	// has an effect if there is a parent body)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnablebDisableCollision"))
	uint8 bDisableCollision : 1;

	// If true then the control will only affect the child object, not the parent
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnablebOnlyControlChildObject"))
	uint8 bOnlyControlChildObject : 1;

	/**
	 * When true (the default), strength values are interpreted as accelerations (mass-independent).
	 * When false they are interpreted as forces (heavier bodies respond more slowly).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (editcondition = "bEnablebUseAccelerationDriveMode"))
	uint8 bUseAccelerationDriveMode : 1;

	//==================================================================================================================
	// Flags for sparse access
	//==================================================================================================================

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableLinearStrength : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableLinearDampingRatio : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableLinearExtraDamping : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableMaxForce : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableAngularStrength : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableAngularDampingRatio : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableAngularExtraDamping : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableMaxTorque : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableLinearTargetVelocityMultiplier : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableAngularTargetVelocityMultiplier : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableCustomControlPoint : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnablebEnabled : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnablebUseCustomControlPoint: 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnablebUseSkeletalAnimation : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnablebDisableCollision : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnablebOnlyControlChildObject : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnablebUseAccelerationDriveMode : 1;
};

// Contains data associated with how physical bodies should be controlled/directed towards their targets.
// The underlying control is done through damped springs, so the parameters here relate to that.
USTRUCT(BlueprintType)
struct FPhysicsControlData
{
	GENERATED_BODY()

	FPhysicsControlData()
		: bEnabled(true)
		, LinearStrength(0.0f)
		, LinearDampingRatio(1.0f)
		, LinearExtraDamping(0.0f)
		, MaxForce(0.0f)
		, AngularStrength(0.0f)
		, AngularDampingRatio(1.0f)
		, AngularExtraDamping(0.0f)
		, MaxTorque(0.0f)
		, LinearTargetVelocityMultiplier(1.0f)
		, AngularTargetVelocityMultiplier(1.0f)
		, CustomControlPoint(FVector::ZeroVector)
		, bUseCustomControlPoint(false)
		, bUseSkeletalAnimation(true)
		, bDisableCollision(false)
		, bOnlyControlChildObject(false)
		, bUseAccelerationDriveMode(true)
	{
	}

	bool operator==(const FPhysicsControlData& other) const = default;

	friend UE_API FArchive& operator<<(FArchive& Ar, FPhysicsControlData& Data);

	// Applies the values that have been flagged as enabled from the sparse data
	UE_API void UpdateFromSparseData(const FPhysicsControlSparseData& SparseData);

	// Whether this control should be enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	bool bEnabled;

	// The strength used to drive linear motion
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	float LinearStrength;

	// The amount of damping associated with the linear strength. A value of 1 Results in critically
	// damped motion where the control drives as quickly as possible to the target without overshooting.
	// Values > 1 result in more damped motion, and values below 1 result in faster, but more "wobbly" motion.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	float LinearDampingRatio;

	// The amount of additional linear damping. This is added to the damping that comes from LinearDampingRatio
	// and can be useful when you want damping even when LinearStrength is zero.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	float LinearExtraDamping;

	// The maximum force used to drive the linear motion. Zero indicates no limit.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	float MaxForce;

	// The strength used to drive angular motion
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	float AngularStrength;

	// The amount of damping associated with the angular strength. A value of 1 Results in critically
	// damped motion where the control drives as quickly as possible to the target without overshooting.
	// Values > 1 result in more damped motion, and values below 1 result in faster, but more "wobbly" motion.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	float AngularDampingRatio;

	// The amount of additional angular damping. This is added to the damping that comes from AngularDampingRatio
	// and can be useful when you want damping even when AngularStrength is zero.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	float AngularExtraDamping;

	// The maximum torque used to drive the angular motion. Zero indicates no limit.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	float MaxTorque;

	// Multiplier on the linear velocity, which gets applied to the damping. A value of 1 means the animation target
	// velocity is used, which helps it track the animation. A value of 0 means damping happens in "world space"
	// - so damping acts like drag on the movement.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	float LinearTargetVelocityMultiplier;

	// Multiplier on the angular velocity, which gets applied to the damping. A value of 1 means the animation target
	// velocity is used, which helps it track the animation. A value of 0 means damping happens in "world space"
	// - so damping acts like drag on the movement.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	float AngularTargetVelocityMultiplier;

	// The position of the control point relative to the child mesh, when using a custom control point.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FVector CustomControlPoint;

	// Whether or not to use the custom control point position
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	bool bUseCustomControlPoint;

	// If true then the target will be applied on top of the skeletal animation (if there is any)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	bool bUseSkeletalAnimation;

	// Whether or not this control should disable collision between the parent and child bodies (only
	// has an effect if there is a parent body)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	bool bDisableCollision;

	// If true then the control will only affect the child object, not the parent
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	bool bOnlyControlChildObject;

	 // When true (the default), the strength values are interpreted as accelerations - so the same setting
	 // produces a comparable response regardless of body mass. When false, the strength values are
	 // interpreted as forces - so heavier bodies respond more slowly to the same setting. Switching this
	 // value changes the effective strength dramatically, so retune after toggling.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	bool bUseAccelerationDriveMode;

	// Returns the control point, which may be custom or automatic (centre of mass)
	UE_API FVector GetControlPoint(const ImmediatePhysics::FActorHandle* ChildActorHandle) const;
};

// These parameters allow modification of the parameters in FPhysicsControlData for two reasons:
// 1. They allow per-axis settings for the linear components (e.g. so you can drive an object
//    horizontally but still let it fall under gravity)
// 2. They make it easy to create the controls with "default" strength/damping (e.g. taken from the
//    physics asset) in FPhysicsControlData, and then the strength/damping etc can be scaled every
//    tick (typically between 0 and 1, though that is up to the user).
USTRUCT(BlueprintType)
struct FPhysicsControlSparseMultiplier
{
	GENERATED_BODY()

	FPhysicsControlSparseMultiplier()
		: LinearStrengthMultiplier(1.0)
		, LinearDampingRatioMultiplier(1.0)
		, LinearExtraDampingMultiplier(1.0)
		, MaxForceMultiplier(1.0)
		, AngularStrengthMultiplier(1.0)
		, AngularDampingRatioMultiplier(1.0)
		, AngularExtraDampingMultiplier(1.0)
		, MaxTorqueMultiplier(1.0)
		// the flags
		, bEnableLinearStrengthMultiplier(true)
		, bEnableLinearDampingRatioMultiplier(true)
		, bEnableLinearExtraDampingMultiplier(true)
		, bEnableMaxForceMultiplier(true)
		, bEnableAngularStrengthMultiplier(true)
		, bEnableAngularDampingRatioMultiplier(true)
		, bEnableAngularExtraDampingMultiplier(true)
		, bEnableMaxTorqueMultiplier(true)
	{
	}

	bool operator==(const FPhysicsControlSparseMultiplier& other) const = default;

	// Per-direction multiplier on the linear strength.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableLinearStrengthMultiplier"))
	FVector LinearStrengthMultiplier;

	// Per-direction multiplier on the linear damping ratio.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableLinearDampingRatioMultiplier"))
	FVector LinearDampingRatioMultiplier;

	// Per-direction multiplier on the linear extra damping.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableLinearExtraDampingMultiplier"))
	FVector LinearExtraDampingMultiplier;

	// Per-direction multiplier on the maximum force that can be applied. Note that zero means zero force.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableMaxForceMultiplier"))
	FVector MaxForceMultiplier;

	// Multiplier on the angular strength.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableAngularStrengthMultiplier"))
	float AngularStrengthMultiplier;

	// Multiplier on the angular damping ratio.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableAngularDampingRatioMultiplier"))
	float AngularDampingRatioMultiplier;

	// Multiplier on the angular extra damping.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableAngularExtraDampingMultiplier"))
	float AngularExtraDampingMultiplier;

	// Per-direction multiplier on the maximum torque that can be applied. Note that zero means zero torque.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0", editcondition = "bEnableMaxTorqueMultiplier"))
	float MaxTorqueMultiplier;

	//==================================================================================================================
	// Flags for sparse access
	//==================================================================================================================

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableLinearStrengthMultiplier : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableLinearDampingRatioMultiplier : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableLinearExtraDampingMultiplier : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableMaxForceMultiplier : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableAngularStrengthMultiplier : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableAngularDampingRatioMultiplier : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableAngularExtraDampingMultiplier : 1;

	UPROPERTY(EditAnywhere, Category = PhysicsControl, meta = (InlineEditConditionToggle))
	uint8 bEnableMaxTorqueMultiplier : 1;
};

// These parameters allow modification of the parameters in FPhysicsControlData for two reasons:
// 1. They allow per-axis settings for the linear components (e.g. so you can drive an object
//    horizontally but still let it fall under gravity)
// 2. They make it easy to create the controls with "default" strength/damping (e.g. taken from the
//    physics asset) in FPhysicsControlData, and then the strength/damping etc can be scaled every
//    tick (typically between 0 and 1, though that is up to the user).
USTRUCT(BlueprintType)
struct FPhysicsControlMultiplier
{
	GENERATED_BODY()

	FPhysicsControlMultiplier()
		: LinearStrengthMultiplier(1.0)
		, LinearDampingRatioMultiplier(1.0)
		, LinearExtraDampingMultiplier(1.0)
		, MaxForceMultiplier(1.0)
		, AngularStrengthMultiplier(1.0)
		, AngularDampingRatioMultiplier(1.0)
		, AngularExtraDampingMultiplier(1.0)
		, MaxTorqueMultiplier(1.0)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FPhysicsControlMultiplier& Data)
	{
		Ar << Data.LinearStrengthMultiplier;
		Ar << Data.LinearDampingRatioMultiplier;
		Ar << Data.LinearExtraDampingMultiplier;
		Ar << Data.MaxForceMultiplier;
		Ar << Data.AngularStrengthMultiplier;
		Ar << Data.AngularDampingRatioMultiplier;
		Ar << Data.AngularExtraDampingMultiplier;
		Ar << Data.MaxTorqueMultiplier;
		return Ar;
	}

	bool operator==(const FPhysicsControlMultiplier& other) const = default;

	// Applies the values that have been flagged as enabled from the sparse data
	UE_API void UpdateFromSparseData(const FPhysicsControlSparseMultiplier& SparseData);

	// Per-direction multiplier on the linear strength.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	FVector LinearStrengthMultiplier;

	// Per-direction multiplier on the linear damping ratio.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	FVector LinearDampingRatioMultiplier;

	// Per-direction multiplier on the linear extra damping.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	FVector LinearExtraDampingMultiplier;

	// Per-direction multiplier on the maximum force that can be applied. Note that zero means zero force.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	FVector MaxForceMultiplier;

	// Multiplier on the angular strength.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	float AngularStrengthMultiplier;

	// Multiplier on the angular damping ratio.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	float AngularDampingRatioMultiplier;

	// Multiplier on the angular extra damping.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	float AngularExtraDampingMultiplier;

	// Per-direction multiplier on the maximum torque that can be applied. Note that zero means zero torque.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl, meta = (ClampMin = "0.0"))
	float MaxTorqueMultiplier;
};

// Defines a target position and orientation, and also the target velocity and angular velocity.
// In many cases the velocities will be calculated automatically (e.g. when setting the target position,
// the component will optionally calculate an implied velocity. However, the user can also specify a
// target velocity directly. Note that the velocity influences the control through the damping parameters
// in FPhysicsControlData
USTRUCT(BlueprintType)
struct FPhysicsControlTarget
{
	GENERATED_BODY()

	FPhysicsControlTarget()
		: TargetPosition(ForceInitToZero)
		, TargetVelocity(ForceInitToZero)
		, TargetOrientation(ForceInitToZero)
		, TargetAngularVelocity(ForceInitToZero)
		, bApplyControlPointToTarget(false)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FPhysicsControlTarget& Data)
	{
		Ar << Data.TargetPosition;
		Ar << Data.TargetVelocity;
		Ar << Data.TargetOrientation;
		Ar << Data.TargetAngularVelocity;
		Ar << Data.bApplyControlPointToTarget;
		return Ar;
	}

	// The target position of the child body, relative to the parent body
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FVector TargetPosition;

	// The target velocity of the child body, relative to the parent body
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FVector TargetVelocity;

	// The target orientation of the child body, relative to the parent body
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FRotator TargetOrientation;

	// The target angular velocity (revolutions per second) of the child body, relative to the parent body
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FVector TargetAngularVelocity;

	// Whether to use the ControlPoint as an offset for the target transform, as well as the
	// physical body. If true then the target TM is treated as a target transform for the actual
	// object, though the control is still applied through the control point (which is at the
	// center of mass by default). If false then it is treated as a target transform for the
	// control point on the object.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	bool bApplyControlPointToTarget;
};

// Structure that determines a "control" - this contains all the information needed to drive (with spring-dampers)
// a child body relative to a parent body. These bodies will be associated with either a static or skeletal mesh.
USTRUCT(BlueprintType)
struct FPhysicsControl
{
	GENERATED_BODY()

	FPhysicsControl() {}

	FPhysicsControl(
		const FName&                 InParentBoneName,
		const FName&                 InChildBoneName,
		const FPhysicsControlData&   InControlData)
		: ParentBoneName(InParentBoneName)
		, ChildBoneName(InChildBoneName)
		, ControlData(InControlData)
	{
	}

	bool operator==(const FPhysicsControl& other) const = default;

	// Indicates if the control is enabled
	bool IsEnabled() const { return ControlData.bEnabled; }

	// The name of the skeletal mesh bone or the name of the static mesh body that will be doing the driving.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName ParentBoneName;

	// The name of the skeletal mesh bone or the name of the static mesh body that the control
	// will be driving.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName ChildBoneName;

	// Strength and damping parameters. Can be modified at any time, but will sometimes have
	// been set once during initialization
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsControlData ControlData;

	// Multiplier for the ControlData. This will typically be modified dynamically, and also expose the ability
	// to set directional strengths
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsControlMultiplier ControlMultiplier;
};

// Used on creation, to allow requesting the control to be in certain sets
USTRUCT(BlueprintType)
struct FPhysicsControlCreationData
{
	GENERATED_BODY();

	bool operator==(const FPhysicsControlCreationData& other) const = default;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsControl Control;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TArray<FName> Sets;
};

// Data that can be used to parameterize (modify/update) a control
USTRUCT(BlueprintType)
struct FPhysicsControlNamedControlParameters
{
	GENERATED_BODY();

	FPhysicsControlNamedControlParameters() {}

	FPhysicsControlNamedControlParameters(FName InName, const FPhysicsControlSparseData& InData)
		: Name(InName), Data(InData) {}

	bool operator==(const FPhysicsControlNamedControlParameters& other) const = default;

	// The name of the control (or set of controls) to update
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsControlSparseData Data;
};

// Data that can be used to parameterize (modify/update) a control multiplier
USTRUCT(BlueprintType)
struct FPhysicsControlNamedControlMultiplierParameters
{
	GENERATED_BODY();

	FPhysicsControlNamedControlMultiplierParameters() {}

	FPhysicsControlNamedControlMultiplierParameters(FName InName, const FPhysicsControlSparseMultiplier& InData)
		: Name(InName), Data(InData) {}

	bool operator==(const FPhysicsControlNamedControlMultiplierParameters& other) const = default;

	// The name of the control (or set of controls) to update
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsControlSparseMultiplier Data;
};


// Data that can be used to parameterize(modify / update) a modifier
USTRUCT(BlueprintType)
struct FPhysicsControlNamedModifierParameters
{
	GENERATED_BODY();

	FPhysicsControlNamedModifierParameters() {}

	FPhysicsControlNamedModifierParameters(FName InName, const FPhysicsControlModifierSparseData& InData)
		: Name(InName), Data(InData) {}

	bool operator==(const FPhysicsControlNamedModifierParameters& other) const = default;

	// The name of the modifier (or set of modifiers) to update
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FPhysicsControlModifierSparseData Data;
};

// These apply temporary/ephemeral changes to the controls that only persist for one tick.
USTRUCT(BlueprintType)
struct FPhysicsControlControlAndModifierParameters
{
	GENERATED_BODY();

	// Parameters for existing controls. Each name can be the name of a control, or the name of a
	// set of controls. They will only apply for one tick/update. They will be applied in order (so
	// subsequent entries will override earlier ones if they apply to the same control).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	TArray<FPhysicsControlNamedControlParameters> ControlParameters;

	// Multipliers for existing controls. Each name can be the name of a control, or the name of a
	// set of controls. They will only apply for one tick/update. They will be applied in order (so
	// subsequent entries will override earlier ones if they apply to the same control).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	TArray<FPhysicsControlNamedControlMultiplierParameters> ControlMultiplierParameters;

	//  Parameters for existing modifiers. Each name can be the name of a modifier, or the name of a
	// set of modifiers. They will only apply for one tick/update.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	TArray<FPhysicsControlNamedModifierParameters> ModifierParameters;

	void Add(const FPhysicsControlNamedControlParameters& InParameters) { ControlParameters.Add(InParameters); }
	void Add(const FPhysicsControlNamedControlMultiplierParameters& InParameters) { ControlMultiplierParameters.Add(InParameters); }
	void Add(const FPhysicsControlNamedModifierParameters& InParameters) { ModifierParameters.Add(InParameters); }
};

// These apply permanent changes to the controls and modifiers, allowing all the settings to be changed
// (apart from the actual bodies that are being controlled/affected)
USTRUCT(BlueprintType)
struct FPhysicsControlControlAndModifierUpdates
{
	GENERATED_BODY();

	bool operator==(const FPhysicsControlControlAndModifierUpdates& other) const = default;

	// Modifications to the underlying controls
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TArray<FPhysicsControlNamedControlParameters> ControlUpdates;

	// Modifications to the underlying control multipliers
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TArray<FPhysicsControlNamedControlMultiplierParameters> ControlMultiplierUpdates;

	// Modifications to the underlying modifiers
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TArray<FPhysicsControlNamedModifierParameters> ModifierUpdates;
};

// Collection of controls and body modifiers, used for creation
USTRUCT(BlueprintType)
struct FPhysicsControlAndBodyModifierCreationDatas
{
	GENERATED_BODY();

	UE_API FPhysicsControlAndBodyModifierCreationDatas& operator+=(const FPhysicsControlAndBodyModifierCreationDatas& other);

	UE_API bool operator==(const FPhysicsControlAndBodyModifierCreationDatas& other) const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TMap<FName, FPhysicsControlCreationData> Controls;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TMap<FName, FPhysicsBodyModifierCreationData> Modifiers;
};

UENUM(BlueprintType)
enum class EPhysicsControlForceType : uint8
{
	Force,
	Acceleration,
	Impulse,
	VelocityChange
};

UE_API EPhysicsControlForceType ConvertForceType(ImmediatePhysics_Shared::EForceType InForceType);
UE_API ImmediatePhysics_Shared::EForceType ConvertForceType(EPhysicsControlForceType InForceType);

USTRUCT(BlueprintType)
struct FPhysicsControlForceAndTorqueData
{
	GENERATED_BODY();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Force)
	EPhysicsControlSpace LocationSpace = EPhysicsControlSpace::Body;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Force)
	EPhysicsControlSpace ForceAndTorqueSpace = EPhysicsControlSpace::World;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Force)
	EPhysicsControlForceType Type = EPhysicsControlForceType::Force;

	/// The linear force/acceleration/impulse/velocity change
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Force)
	FVector Linear = FVector::ZeroVector;

	/// The angular torque/acceleration/impulse/velocity change
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Force)
	FVector Angular = FVector::ZeroVector;

	// The position at which the linear force will be applied
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Force)
	FVector Location = FVector::ZeroVector;
};

USTRUCT(BlueprintType)
struct FPhysicsControlNamedForceAndTorqueData
{
	GENERATED_BODY();

	FPhysicsControlNamedForceAndTorqueData() {}

	FPhysicsControlNamedForceAndTorqueData(const FName InName, const FPhysicsControlForceAndTorqueData& InData)
		: Name(InName), ForceAndTorqueData(InData) {}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Force)
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Force)
	FPhysicsControlForceAndTorqueData ForceAndTorqueData;
};

// Functions to interpolate between various control/modifier datas
PHYSICSCONTROL_API FPhysicsControlData Interpolate(
	const FPhysicsControlData& A, const FPhysicsControlData& B, const float Weight);
PHYSICSCONTROL_API FPhysicsControlSparseData Interpolate(
	const FPhysicsControlSparseData& A, const FPhysicsControlSparseData& B, const float Weight);

PHYSICSCONTROL_API FPhysicsControlModifierData Interpolate(
	const FPhysicsControlModifierData& A, const FPhysicsControlModifierData& B, const float Weight);
PHYSICSCONTROL_API FPhysicsControlModifierSparseData Interpolate(
	const FPhysicsControlModifierSparseData& A, const FPhysicsControlModifierSparseData& B, const float Weight);




#undef UE_API
