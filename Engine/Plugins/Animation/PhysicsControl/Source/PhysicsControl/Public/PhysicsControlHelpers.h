// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Math/Vector.h"
#include "Math/Quat.h"

struct FBodyInstance;
struct FPhysicsControlData;
struct FConstraintProfileProperties;

class UPrimitiveComponent;
class USkeletalMeshComponent;

namespace UE
{
namespace PhysicsControl
{

// Converts strength/damping ratio/extra damping into spring stiffness/damping
PHYSICSCONTROL_API void ConvertStrengthToSpringParams(
	FVector& OutSpring, FVector& OutDamping, 
	const FVector& InStrength, const FVector& InDampingRatio, const FVector& InExtraDamping);

// Converts the drive settings from the constraint profile into the control data strength/damping etc.
// Approximations will be made if (a) the linear drive has different values for the x/y/z axes, or (b)
// the constraint profile is set to use twist/swing instead of slerp for the angular drive.
PHYSICSCONTROL_API void ConvertConstraintProfileToControlData(
	FPhysicsControlData& OutControlData, const FConstraintProfileProperties& InProfileProperties);

// Attempts to find a BodyInstance from the component. If it is a primitive or static mesh component the
// single body instance will be returned. If it is a skeletal mesh then if BoneName can be found, the
// body instance corresponding to that bone will be returned. Otherwise it will return nullptr if
// the bone can't be found.
PHYSICSCONTROL_API FBodyInstance* GetBodyInstance(UPrimitiveComponent* Component, const FName BoneName);

// Attempts to find the parent physical bone given a skeletal mesh and starting bone. This walks up
// the hierarchy, ignoring non-physical bones, until either a physical bone is found, or it has reached
// the root without finding a physical bone.
PHYSICSCONTROL_API FName GetPhysicalParentBone(USkeletalMeshComponent* SkeletalComponent, FName BoneName);

// Calculates the angular velocity that would go from PrevQ to CurrentQ in Dt. Returns zero if Dt is invalid.
FORCEINLINE FVector CalculateAngularVelocity(const FQuat& PrevQ, const FQuat& CurrentQ, float Dt)
{
	const FQuat DeltaQ = (CurrentQ * PrevQ.Inverse()).GetShortestArcWith(FQuat::Identity);
	return (Dt > UE_SMALL_NUMBER) ? DeltaQ.ToRotationVector() / Dt : FVector::ZeroVector;
}

// Calculates the linear velocity that would go from PrevP to CurrentP in Dt. Returns zero if Dt is invalid.
FORCEINLINE FVector CalculateLinearVelocity(const FVector& PrevP, const FVector& CurrentP, float Dt)
{
	return (Dt > UE_SMALL_NUMBER) ? (CurrentP - PrevP) / Dt : FVector::ZeroVector;
}

// Converts strength/damping ratio/extra damping into spring stiffness/damping. Clamps strength to >= 0.
template<typename TOut>
void ConvertStrengthToSpringParams(
	TOut& OutSpring, TOut& OutDamping,
	double InStrength, double InDampingRatio, double InExtraDamping)
{
	const double ClampedStrength = FMath::Max(InStrength, 0.0);
	TOut AngularFrequency = TOut(ClampedStrength * UE_DOUBLE_TWO_PI);
	TOut Stiffness = AngularFrequency * AngularFrequency;

	OutSpring = Stiffness;
	OutDamping = TOut(InExtraDamping + 2 * InDampingRatio * AngularFrequency);
}

// Converts spring/damping values into strength/damping ratio/extra damping. This tries to get
// as much damping into the damping ratio term as possible, without letting it go above 1.
template<typename TOut>
void ConvertSpringToStrengthParams(
	TOut& OutStrength, TOut& OutDampingRatio, TOut& OutExtraDamping,
	const double InSpring, const double InDamping)
{
	// Simple calculation to get the strength
	TOut AngularFrequency = TOut(FMath::Sqrt(InSpring));
	OutStrength = AngularFrequency / UE_TWO_PI;

	// For damping, try to put as much into the damping ratio as possible, up to a max DR of 1. Then
	// the rest goes into extra damping.
	OutDampingRatio = 1;
	TOut ImpliedDamping = TOut(2.0 * OutDampingRatio * AngularFrequency);

	if (ImpliedDamping < InDamping)
	{
		OutExtraDamping = TOut(InDamping) - ImpliedDamping;
	}
	else if (AngularFrequency > 0)
	{
		OutExtraDamping = 0;
		OutDampingRatio = TOut(InDamping) / (2 * AngularFrequency);
	}
	else
	{
		OutDampingRatio = 1;
		OutExtraDamping = TOut(InDamping);
	}
}

} // namespace PhysicsControl
} // namespace UE
