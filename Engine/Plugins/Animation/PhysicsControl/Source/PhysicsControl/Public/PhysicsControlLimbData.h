// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "EngineDefines.h"
#include "PhysicsControlData.h"
#include "PhysicsControlLimbData.generated.h"

#define UE_API PHYSICSCONTROL_API

// Setup data that are used to create the representation of a single limb. A limb is an array of
// contiguous bones (e.g. left arm, or the spine etc). We can define it has the set of bones that
// are children of a start bone (plus the start bone itself), plus optionally the parent of that start
// bone (this is useful when defining the spine, since you will want to include the pelvis, but you
// don't to include all the children of the pelvis since that would include the legs), but excluding
// any bones that are already part of another limb. This implies limbs should be constructed in order
// from leaf to root (in practice, that normally means defining the spine after other limbs such as
// the arms and legs).
USTRUCT(BlueprintType)
struct FPhysicsControlLimbSetupData
{
	GENERATED_BODY()

	FPhysicsControlLimbSetupData()
		: bIncludeParentBone(false), bCreateWorldSpaceControls(true)
		, bCreateParentSpaceControls(true), bCreateBodyModifiers(true)
	{}

	bool operator==(const FPhysicsControlLimbSetupData& other) const = default;

	// The name of the limb that this will be used to create
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName LimbName;

	// Normally the root-most bone of the limb (e.g. left clavicle when defining the left arm) - so the
	// limb will contain children of this bone (plus this bone itself).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FName StartBone;

	// Whether or not to include the parent of the start bone. This is intended to be used for limbs like
	// the spine, where you would set StartBone = spine_01 but also expect to include the pelvis (parent
	// of spine_01) in the spine limb.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	uint8 bIncludeParentBone : 1;

	// Whether to create-world space controls for this limb
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	uint8 bCreateWorldSpaceControls : 1;

	// Whether to create parent-space controls for this limb
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	uint8 bCreateParentSpaceControls : 1;

	// Whether to create body modifiers for this limb
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	uint8 bCreateBodyModifiers : 1;
};

// Wrapper for the array of bone names, so that the limbs can we can work with a map of arrays of names.
USTRUCT(BlueprintType)
struct FPhysicsControlLimbBones
{
	GENERATED_BODY()

	FPhysicsControlLimbBones()
		: bFirstBoneIsAdditional(false), bCreateWorldSpaceControls(true)
		, bCreateParentSpaceControls(true), bCreateBodyModifiers(true)
	{}

	// The Skeletal mesh that this limb is associated with
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

	// The names of the bones in the limb
	TArray<FName> BoneNames;

	// Indicates if the first bone in this limb was included due to "IncludeParentBone".
	uint8 bFirstBoneIsAdditional : 1;

	uint8 bCreateWorldSpaceControls : 1;
	uint8 bCreateParentSpaceControls : 1;
	uint8 bCreateBodyModifiers : 1;
};

// Wrapper for the array of names, so that the control/modifier sets can we work with a map
// of arrays of names. Note that in practice, the names will be of either controls or body modifiers.
USTRUCT(BlueprintType)
struct FPhysicsControlNames
{
	GENERATED_BODY()

	// The names of either controls of body modifiers (depending on context)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TArray<FName> Names;
};

USTRUCT(BlueprintType)
struct FPhysicsControlCharacterSetupData
{
	GENERATED_BODY();

	UE_API FPhysicsControlCharacterSetupData& operator+=(const FPhysicsControlCharacterSetupData& other);

	bool operator==(const FPhysicsControlCharacterSetupData& other) const = default;

	UPROPERTY(EditAnywhere, Category = ControlSetup)
	TArray<FPhysicsControlLimbSetupData> LimbSetupData;

	UPROPERTY(EditAnywhere, Category = ControlSetup)
	FPhysicsControlData DefaultWorldSpaceControlData;

	UPROPERTY(EditAnywhere, Category = ControlSetup)
	FPhysicsControlData DefaultParentSpaceControlData;

	UPROPERTY(EditAnywhere, Category = ControlSetup)
	FPhysicsControlModifierData DefaultBodyModifierData;
};

#undef UE_API
