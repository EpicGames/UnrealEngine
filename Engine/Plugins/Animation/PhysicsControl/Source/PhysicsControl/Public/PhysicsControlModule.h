// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

class FPhysicsControlModule : public IModuleInterface
{
public:

	// IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
