// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// All the controls and body modifiers we create get given names, so we can keep a record in a form
// that will be useful to users. The three general uses are:
// 1. All controls or modifiers
// 2. The type of control - the obvious ones are "world space" and "parent space", but users might make others
// 3. Sets of control/modifiers - the obvious ones are limbs (e.g. a set of world-space controls on the leg)
//    but users might want to make other sets - e.g. "UpperBody"
struct FPhysicsControlNameRecords
{
	// Adds Name to SetName as well as optionally adding it to the set "All"
	void AddControl(FName Name, FName SetName, bool bAddToAll = true);
	// Adds Name to SetNames as well as optionally adding it to the set "All"
	void AddControl(FName Name, TConstArrayView<FName> SetNames, bool bAddToAll = true);
	// Adds a collection of Names to SetName as well as optionally adding them to the set "All"
	void AddControls(TConstArrayView<FName> ControlNames, const FName SetName, bool bAddToAll = true);
	// Removes Name from all Sets
	void RemoveControl(FName Name);
	const TArray<FName>& GetControlNamesInSet(FName SetName) const;

	// Adds Name to SetName as well as optionally adding it to the set "All"
	void AddBodyModifier(FName Name, FName SetName, bool bAddToAll = true);
	// Adds Name to SetNames as well as optionally adding it to the set "All"
	void AddBodyModifier(FName Name, TConstArrayView<FName> SetNames, bool bAddToAll = true);
	// Adds a collection of Names to SetName as well as optionally adding them to the set "All"
	void AddBodyModifiers(TConstArrayView<FName> BodyModifierNames, const FName SetName, bool bAddToAll = true);
	// Removes Name from all Sets
	void RemoveBodyModifier(FName Name);
	const TArray<FName>& GetBodyModifierNamesInSet(FName SetName) const;

	// Remove all control and modifier sets.
	void Reset();

	// All the control sets we've created, arranged by set name. Note that some controls will
	// not necessarily have been added to a set.
	TMap<FName, TArray<FName>> ControlSets;

	// All the body modifiers we've created, arranged by set name. Note that some modifiers will
	// not necessarily have been added to a set.
	TMap<FName, TArray<FName>> BodyModifierSets;
};

// Returns a copy of the supplied list of names in which any names that match keys in the
// supplied 'Sets' map are replaced with all the names in the array associated with that key.
PHYSICSCONTROL_API TArray<FName> ExpandName(const FName InName, const TMap<FName, TArray<FName>>& Sets);
PHYSICSCONTROL_API TArray<FName> ExpandNames(const TArray<FName>& InNames, const TMap<FName, TArray<FName>>& Sets);
