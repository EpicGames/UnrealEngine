// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Guid.h"

#define UE_API PHYSICSCONTROL_API

struct FPhysicsControlObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded,
		ControlRigIncludeDriveInJoint,
		ControlRigRemoveCurrentDataFromPhysicsComponent,
		ControlRigUseAutomaticSolver,
		ControlRigAutomaticallyAddPhysicsBodyComponents,
		ControlRigControlAddChildBodyComponentKey,
		ControlRigControlAddControlMultiplier,
		ControlRigSeparateOutJointFromBody,
		ControlRigSupportFullConstraintData,
		ControlRigSupportFullDriveConstraintData,
		ControlRigSupportNoCollisionBodies,
		ControlRigSupportBodyDamping,
		ControlRigCollisionHasMaterial,
		ControlRigBodyDynamicsHasDensity,
		ControlRigSolverSettingsIncludesCollisionBoundsExpansion,
		ControlRigDetectTeleportFromDistanceChange,
		ControlRigSpeedThresholdForReset,
		ControlRigResetCooldownFrames,
		ControlRigSimSpaceDragMultipliers,
		ControlRigAccelerationThresholdForReset,
		ControlRigRemoveResetCooldownFrames,
		ControlRigDriveRelativeToAnimation,
		ControlRigDetectExternalVelocityTurbulence,
		ControlRigBodyIncludeInChecksForReset,
		ControlRigWorldCollision,
		ControlRigCentreOfMassNudge,
		ControlRigJointEnabled,
		RemoveSkeletalAnimationVelocityMultiplier,
		RemoveUseSkeletalAnimation,
		ControlRigShapeData,
		EnableCCD,
		ControlRigAllowCCD,
		ControlRigConvexCollision,
		ControlDataDriveForceMode,    // bUseAccelerationDriveMode added to FPhysicsControlData / FPhysicsControlSparseData

		// -----<new versions can be added above this line>-----
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;

private:
	FPhysicsControlObjectVersion() {}
};

#undef UE_API
