// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "PhysicsControlLog.h"

class UPhysicsAsset;

#include "RigidBodyControlData.h"
#include "PhysicsControlRecord.h"

struct FAnimNode_RigidBodyWithControl;
struct FPhysicsControlLimbBones;
struct FPhysicsControlLimbSetupData;
struct FPhysicsControlSetUpdates;
struct FReferenceSkeleton;
struct FPhysicsControlNameRecords;

namespace UE
{
namespace PhysicsControl
{

constexpr int32 MaxNumControlsOrModifiersPerName = 16;

// Parse the skeleton tree to figure out which bones are associated with which limbs. 
PHYSICSCONTROL_API TMap<FName, FPhysicsControlLimbBones> GetLimbBones(
	const TArray<FPhysicsControlLimbSetupData>& LimbSetupDatas, 
	const FReferenceSkeleton&                   RefSkeleton, 
	const UPhysicsAsset*                        PhysicsAsset);

// Populates the supplied body modifier names, control names and name records structures with the
// names and sets that would be created for the supplied limb bones, skeleton and physics asset.
PHYSICSCONTROL_API void CollectOperatorNames(
	const FPhysicsControlCharacterSetupData&           CharacterSetupData,
	const FPhysicsControlAndBodyModifierCreationDatas& AdditionalControlsAndBodyModifiers,
	const TMap<FName, FPhysicsControlLimbBones>        AllLimbBones,
	const FReferenceSkeleton&                          RefSkeleton, 
	const UPhysicsAsset*                               PhysicsAsset, 
	TSet<FName>&                                       BodyModifierNames, 
	TSet<FName>&                                       ControlNames, 
	FPhysicsControlNameRecords&                        NameRecords);

// Creates the body modifiers, controls and sets for the supplied node, limb bones, skeleton and physics asset.
PHYSICSCONTROL_API void CreateOperatorsForNode(
	FAnimNode_RigidBodyWithControl*                    Node, 
	const FPhysicsControlCharacterSetupData&           CharacterSetupData,
	const FPhysicsControlAndBodyModifierCreationDatas& AdditionalControlsAndBodyModifiers,
	const TMap<FName, FPhysicsControlLimbBones>        AllLimbBones,
	const FReferenceSkeleton&                          RefSkeleton, 
	const UPhysicsAsset*                               PhysicsAsset, 
	FPhysicsControlNameRecords&                        NameRecords);

// Adds the specified additional sets to the supplied Name Records structure.
PHYSICSCONTROL_API void CreateAdditionalSets(
	const FPhysicsControlSetUpdates& AdditionalSets, 
	const TSet<FName>&               BodyModifierNames, 
	const TSet<FName>&               ControlNames, 
	FPhysicsControlNameRecords&      NameRecords);

// Adds the specified additional sets to the supplied Name Records structure.
PHYSICSCONTROL_API void CreateAdditionalSets(
	const FPhysicsControlSetUpdates&             AdditionalSets, 
	const TMap<FName, FRigidBodyModifierRecord>& BodyModifierRecords, 
	const TMap<FName, FRigidBodyControlRecord>&  Controls, 
	FPhysicsControlNameRecords&                  NameRecords);

// Adds the specified additional sets to the supplied Name Records structure.
PHYSICSCONTROL_API void CreateAdditionalSets(
	const FPhysicsControlSetUpdates&               AdditionalSets, 
	const TMap<FName, FPhysicsBodyModifierRecord>& BodyModifierRecords,
	const TMap<FName, FPhysicsControlRecord>&      Controls,
	FPhysicsControlNameRecords&                    NameRecords);

//======================================================================================================================
// Helper for GetUniqueName
inline bool DoesNameExist(const FName Name, const TArray<FName>& ExistingNames)
{
	return ExistingNames.Find(Name) != INDEX_NONE;
}

//======================================================================================================================
// Helper for GetUniqueName
template<typename T>
bool DoesNameExist(const FName Name, const TMap<FName, T>& ExistingNames)
{
	return ExistingNames.Find(Name) != nullptr;
}

//======================================================================================================================
// Helper for GetUniqueName
inline bool DoesNameExist(const FName Name, const TSet<FName>& ExistingNames)
{
	return ExistingNames.Find(Name) != nullptr;
}

//======================================================================================================================
// Makes a unique version of OriginalName.
// 
// Note that the convention elsewhere in UE when creating an object, and then a second identical
// one, is for the first object to have the original name, and then the next has a suffix _2, and
// so on.
template<typename CollectionType>
FName GetUniqueName(const FString& OriginalName, const CollectionType& ExistingNames, const int32 MaxNameIndex)
{
	if (!OriginalName.IsEmpty())
	{
		const FName Name(OriginalName);
		if (!DoesNameExist(Name, ExistingNames))
		{
			return Name;
		}
	}

	// If the number gets too large, almost certainly we're in some nasty situation where this is
	// getting called in a loop. Better to quit and fail, rather than allow the constraint set to
	// increase without bound. 
	for (int32 Index = 2; Index < MaxNameIndex; ++Index)
	{
		FString NameStr = OriginalName.IsEmpty()
			? FString::Format(TEXT("Unnamed_{0}"), { Index })
			: FString::Format(TEXT("{0}_{1}"), { OriginalName, Index });
		const FName Name(NameStr);
		if (!DoesNameExist(Name, ExistingNames))
		{
			return Name;
		}
	}
	UE_LOGF(LogPhysicsControl, Warning,
		"Unable to find a suitable Control/Modifier name for %ls - the limit of (%d) has been exceeded",
		*OriginalName, MaxNameIndex);

	return NAME_None;
}

//======================================================================================================================
template<typename CollectionType>
FName GetUniqueBodyModifierName(const FName BodyName, const CollectionType& ExistingNames, const FString& NamePrefix)
{
	FString Name = NamePrefix;
	if (!BodyName.IsNone())
	{
		Name += BodyName.ToString();
	}
	else
	{
		Name += TEXT("Body");
	}

	return GetUniqueName(Name, ExistingNames, MaxNumControlsOrModifiersPerName);
}

//======================================================================================================================
template<typename CollectionType>
FName GetUniqueBodyModifierName(
	const UPrimitiveComponent* Component, const FName BoneName, const CollectionType& ExistingNames, const FString& NamePrefix)
{
	// If bones are involved, make the names based on the bone, ignoring the component
	if (!BoneName.IsNone())
	{
		return GetUniqueBodyModifierName(BoneName, ExistingNames, NamePrefix);
	}

	FString Name = NamePrefix;
	if (Component)
	{
		Name += Component->GetName();
	}
	else
	{
		Name += TEXT("Body");
	}

	return GetUniqueName(Name, ExistingNames, MaxNumControlsOrModifiersPerName);
}

//======================================================================================================================
template<typename CollectionType>
FName GetUniqueControlName(
	const FName ParentName, const FName ChildName, const CollectionType& ExistingNames, const FString& NamePrefix)
{
	FString Name;
	if (!ParentName.IsNone())
	{
		Name = ParentName.ToString();
	}

	if (!ChildName.IsNone())
	{
		if (!Name.IsEmpty())
		{
			Name += TEXT("_");
		}
		Name += ChildName.ToString();
	}

	if (Name.IsEmpty())
	{
		Name = TEXT("Control");
	}

	Name = NamePrefix + Name;

	return GetUniqueName(Name, ExistingNames, MaxNumControlsOrModifiersPerName);
}

//======================================================================================================================
template<typename CollectionType>
FName GetUniqueControlName(
	const UPrimitiveComponent* ParentComponent, const FName ParentBoneName, 
	const UPrimitiveComponent* ChildComponent, const FName ChildBoneName, 
	const CollectionType& ExistingNames, const FString& NamePrefix)
{
	// If bones are involved, make the names based on the bone, ignoring the component
	if (!ParentBoneName.IsNone() || !ChildBoneName.IsNone())
	{
		return GetUniqueControlName(ParentBoneName, ChildBoneName, ExistingNames, NamePrefix);
	}

	FString Name;
	if (ParentComponent)
	{
		Name = ParentComponent->GetName();
	}

	if (ChildComponent)
	{
		if (!Name.IsEmpty())
		{
			Name += TEXT("_");
		}
		Name += ChildComponent->GetName();
	}

	if (Name.IsEmpty())
	{
		Name = TEXT("Control");
	}

	Name = NamePrefix + Name;

	return GetUniqueName(Name, ExistingNames, MaxNumControlsOrModifiersPerName);
}

} // namespace PhysicsControl
} // namespace UE
