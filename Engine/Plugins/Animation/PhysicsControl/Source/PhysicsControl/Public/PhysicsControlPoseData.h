// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"

namespace UE
{
namespace PhysicsControl
{

// @return true if the difference between the old and new TMs exceeds the teleport
// translation/rotation thresholds
bool DetectTeleport(
	const FVector& OldPosition, const FQuat& OldOrientation,
	const FVector& NewPosition, const FQuat& NewOrientation,
	float DistanceThreshold, float RotationThreshold);

// @return true if the difference between the old and new TMs exceeds the teleport
// translation/rotation thresholds
bool DetectTeleport(const FTransform& OldTM, const FTransform& NewTM,
	float DistanceThreshold, float RotationThreshold);

//======================================================================================================================
// Simple minimal implementation of a "FTransform without scale"
struct FPosQuat
{
	FPosQuat(const FVector& Pos, const FQuat& Quat) : Translation(Pos), Rotation(Quat) {}
	FPosQuat(const FQuat& Quat, const FVector& Pos) : Translation(Pos), Rotation(Quat) {}
	FPosQuat(const FRotator& Rotator, const FVector& Pos) : Translation(Pos), Rotation(Rotator) {}
	FPosQuat() : Translation(EForceInit::ForceInitToZero), Rotation(EForceInit::ForceInit) {}
	FPosQuat(const FTransform& TM) : Translation(TM.GetTranslation()), Rotation(TM.GetRotation()) {}
	FPosQuat(ENoInit) {}

	FVector GetTranslation() const { return Translation; }
	FQuat GetRotation() const { return Rotation; }

	void SetTranslation(const FVector& T) { Translation = T; }
	void SetRotation(const FQuat& Q) { Rotation = Q; }

	FTransform ToTransform() const
	{
		return FTransform(Rotation, Translation);
	}

	// Note that multiplication operates in the same sense as FQuat - i.e. in reverse compared to FTransform
	// WorldChildTM = WorldParentTM * ChildRelParentTM
	FPosQuat operator*(const FPosQuat& Other) const
	{
		FQuat OutRotation = Rotation * Other.Rotation;
		FVector OutTranslation = (Rotation * Other.Translation) + Translation;
		return FPosQuat(OutTranslation, OutRotation);
	}

	FVector operator*(const FVector& Position) const
	{
		return Translation + Rotation * Position;
	}

	FPosQuat Inverse() const
	{
		const FQuat OutRotation = Rotation.Inverse();
		return FPosQuat(OutRotation * -Translation, OutRotation);
	}

	bool ContainsNaN() const
	{
		return Translation.ContainsNaN() || Rotation.ContainsNaN();
	}

	FVector Translation;
	FQuat Rotation;
};

//======================================================================================================================
struct FBoneData
{
	FBoneData() {}
	FBoneData(const FPosQuat& InCurrentTM, const FPosQuat& InPreviousTM)
		: CurrentTM(InCurrentTM), PreviousTM(InPreviousTM) {}

	FVector CalculateLinearVelocity(float Dt) const;
	FVector CalculateAngularVelocity(float Dt) const;

	FPosQuat CurrentTM;
	FPosQuat PreviousTM;
};

//======================================================================================================================
// Caches the pose for PhysicsControlComponent
struct FPhysicsControlPoseData
{
public:
	FPhysicsControlPoseData() : ReferenceCount(0), bHasJustTeleported(true), DeltaTime(0) {}

public:
	void Update(
		USkeletalMeshComponent* SkeletalMesh,
		const float             Dt,
		const float             TeleportDistanceThreshold, 
		const float             TeleportRotationThreshold);
	void Reset() { BoneDatas.Empty(); bHasJustTeleported = true; }

	const FBoneData& GetBoneData(int32 Index) const { return BoneDatas[Index]; }
	FPosQuat GetCurrentTM(int32 Index) const { return BoneDatas[Index].CurrentTM; }
	FPosQuat GetPreviousTM(int32 Index) const { return BoneDatas[Index].PreviousTM; }

	bool IsValidIndex(const int32 Index) const { return BoneDatas.IsValidIndex(Index); }
	bool IsEmpty() const { return BoneDatas.IsEmpty(); }

	// The cached skeletal data, updated at the start of each tick
	TArray<FBoneData> BoneDatas;

	// Track when skeletal meshes are going to be used so this entry can be removed, and also so we
	// can add a tick dependency.
	int32 ReferenceCount;

	// The component transform. This is only stored so we can detect teleports. 
	FTransform ComponentTM;

	// Whether the character has just teleported, in which case velocities should not be calculated
	// based on deltas.
	bool bHasJustTeleported;

	// The DeltaTime used for the previous update (may be zero)
	float DeltaTime;
};

} // namespace PhysicsControl
} // namespace UE

