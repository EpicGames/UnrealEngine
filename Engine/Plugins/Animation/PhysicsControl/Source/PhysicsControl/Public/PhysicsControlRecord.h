// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsControlData.h"
#include "PhysicsControlPoseData.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/EngineTypes.h"

struct FConstraintInstance;
struct FBodyInstance;
class UPhysicsAsset;
class UPrimitiveComponent;

// There will be a PhysicsControlRecord created at runtime for every Control that has been created
struct FPhysicsControlRecord
{
	FPhysicsControlRecord(
		const FPhysicsControl&       InControl,
		const FPhysicsControlTarget& InControlTarget,
		UPrimitiveComponent*              InParentComponent,
		UPrimitiveComponent*              InChildComponent)
		: PhysicsControl(InControl)
		, ControlTarget(InControlTarget)
		, ParentComponent(InParentComponent)
		, ChildComponent(InChildComponent)
	{}

	// Removes any constraint and resets the state
	void ResetConstraint();

	// Returns the control point, which may be custom or automatic (centre of mass)
	FVector GetControlPoint() const;

	// Creates the constraint if necessary and stores it. Then initializes the constraint with the
	// bodies. 
	bool InitConstraint(UObject* ConstraintDebugOwner, FName ControlName, bool bWarnAboutInvalidNames);

	// Ensures the constraint frame matches the control point in the record.
	void UpdateConstraintControlPoint();

	// Sets the control point to the center of mass of the child mesh (or to zero if that fails).
	void ResetControlPoint();

	// Ensures JointConstraintIndex is valid for PhysAsset. If the physics asset has changed
	// since InitConstraint ran, re-resolves the index . Returns true if 
	// JointConstraintIndex is valid (not INDEX_NONE). ChildSKM must be non-null.
	bool RefreshJointConstraintIndex(USkeletalMeshComponent* ChildSKM, UPhysicsAsset* PhysAsset);

	// The configuration data
	FPhysicsControl PhysicsControl;

	// The position/orientation etc targets for the controls. These are procedural/explicit control targets -
	// skeletal meshes have the option to use skeletal animation as well, in which case these targets are 
	// expressed as relative to that animation.
	FPhysicsControlTarget ControlTarget;

	// The previous skeletal control target. This will have been set at the end of a previous update
	// (but only if the control was enabled etc), so to check if it is valid, check the update
	// counter. Note that explicit control targets (which contain their own velocity) will be added
	// onto this.
	UE::PhysicsControl::FPosQuat PreviousSkeletalTargetTM;

	// Only use the previous target TM if the current counter is equal to this expected counter. The
	// expected update counter will be set when the control/previous target TM has just been updated.
	FGraphTraversalCounter ExpectedUpdateCounter;

	// The mesh that will be doing the driving. Blank/non-existent means it will happen in world space
	TWeakObjectPtr<UPrimitiveComponent> ParentComponent;

	// The mesh that the control will be driving.
	TWeakObjectPtr<UPrimitiveComponent> ChildComponent;

	// The drive (motor) constraint that implements the control, created and owned by PhysicsControl.
	TSharedPtr<FConstraintInstance> ConstraintInstance;

	// Physics asset whose ConstraintSetup was used to resolve JointConstraintIndex. Stored so
	// that a SetPhysicsAsset call is detected each tick and triggers a re-resolve.
	TWeakObjectPtr<UPhysicsAsset> JointConstraintPhysAsset;

	// True when WidenLimitsForDriveTarget has left at least one axis wider than its authored
	// default. Used to skip RestoreAngularLimitsToDefault on frames where nothing was ever widened.
	bool bJointLimitsWidened = false;

	// Index into UPhysicsAsset::ConstraintSetup for the ragdoll joint constraint connecting
	// ChildBoneName to ParentBoneName. This is a separate constraint from ConstraintInstance
	// above: it belongs to the SKM and defines the joint's angular limits (swing/twist).
	// An index is stored rather than a pointer because RecreatePhysicsState invalidates
	// FConstraintInstance* addresses without changing the physics asset; the index survives that.
	// Resolved once in InitConstraint; each tick the live pointer is obtained via
	// GetConstraintInstanceByIndex. INDEX_NONE when no matching constraint was found.
	int32 JointConstraintIndex = INDEX_NONE;
};

// There will be a PhysicsBodyModifier created at runtime for every BodyInstance involved in the component
struct FPhysicsBodyModifierRecord
{
	FPhysicsBodyModifierRecord(
		TWeakObjectPtr<UPrimitiveComponent>  InComponent, 
		const FName&                    InBoneName, 
		FPhysicsControlModifierData     InBodyModifierData)
		: Component(InComponent)
		, BodyModifier(InBoneName, InBodyModifierData)
		, bResetToCachedTarget(false)
	{}

	// The mesh that will be modified.
	TWeakObjectPtr<UPrimitiveComponent> Component;

	// The core data
	FPhysicsBodyModifier BodyModifier;

	// The target transform when kinematic. 
	UE::PhysicsControl::FPosQuat KinematicTarget;

	// If true then the body will be set to the transform/velocity stored in any cached target (if that
	// exists), and then this flag will be cleared.
	uint8 bResetToCachedTarget:1;
};

// Used internally/only at runtime to track when a SkeletalMeshComponent is being controlled through
// a modifier, and to restore settings when that stops.
struct FModifiedSkeletalMeshData
{
public: 
	FModifiedSkeletalMeshData() : ReferenceCount(0) {}

public:

	// The original setting for restoration when we're deleted
	uint8 bOriginalUpdateMeshWhenKinematic : 1;

	// The original setting for restoration when we're deleted
	EKinematicBonesUpdateToPhysics::Type OriginalKinematicBonesUpdateType;

	// Track when skeletal meshes are going to be used so this entry can be removed
	int32 ReferenceCount;
};

