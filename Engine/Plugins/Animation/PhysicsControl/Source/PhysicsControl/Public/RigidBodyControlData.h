// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "PhysicsControlLimbData.h"
#include "PhysicsControlData.h"
#include "PhysicsControlPoseData.h"

#include "Physics/ImmediatePhysics/ImmediatePhysicsDeclares.h"
#include "Animation/AnimTypes.h"

#include "RigidBodyControlData.generated.h"

// Note that this file defines structures that are only used by the RigidBodyWithControl node. The RBWC node will
// also use structures defined in PhysicsControlData.h Some structures defined here may end up being shared with
// PhysicsControlComponent, so have a PhysicsControl prefix in anticipation of that.

// A single target for a control, which may be defined as an offset from the (implicit) animation target.
USTRUCT(BlueprintType)
struct FRigidBodyControlTarget
{
	GENERATED_BODY();

	FRigidBodyControlTarget()
		: TargetPosition(ForceInitToZero)
		, TargetOrientation(ForceInitToZero)
	{
	}

	// The target position of the child body, relative to the parent body
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FVector TargetPosition;

	// The target orientation of the child body, relative to the parent body
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FRotator TargetOrientation;
};

// A set of targets for controls
USTRUCT(BlueprintType)
struct FRigidBodyControlTargets
{
	GENERATED_BODY();

	// Targets to apply to the named control
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TMap<FName, FRigidBodyControlTarget> Targets;
};

// A single kinematic target, which will be applied in the KinematicTargetSpace set in the relevant body modifier.
USTRUCT(BlueprintType)
struct FRigidBodyKinematicTarget
{
	GENERATED_BODY();

	FRigidBodyKinematicTarget()
		: TargetPosition(ForceInitToZero)
		, TargetOrientation(ForceInitToZero)
	{
	}

	// The target position of the body
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FVector TargetPosition;

	// The target orientation of the body
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	FRotator TargetOrientation;
};

// A set of kinematic targets
USTRUCT(BlueprintType)
struct FRigidBodyKinematicTargets
{
	GENERATED_BODY();

	// Targets to apply to the named body modifier
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TMap<FName, FRigidBodyKinematicTarget> Targets;
};

// A set of forces and torques
USTRUCT(BlueprintType)
struct FRigidBodyForcesAndTorques
{
	GENERATED_BODY();

	// Forces (etc) to apply to the named body modifier
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicsControl)
	TMap<FName, FPhysicsControlForceAndTorqueData> ForcesAndTorques;
};

// This is the record for a control. It stores the "original" data, which will normally stay fixed.
// In addition, it will also store some updates to these original data which will determine how the actual
// controls get used.
struct FRigidBodyControlRecord
{
	FRigidBodyControlRecord(const FPhysicsControl& InControl, ImmediatePhysics::FJointHandle* InJointHandle);

	void ResetCurrent(bool bResetTarget);

	// Note that this is only correct when called during or after the update has been done
	bool IsEnabled() const { return ControlData.bEnabled; }

	// Returns the control point, which may be custom or automatic (centre of mass)
	FVector GetControlPoint(const ImmediatePhysics::FActorHandle* ChildActorHandle) const;

	// The configuration data
	FPhysicsControl Control;

	// Contains any control target that has been set
	FRigidBodyControlTarget ControlTarget;

	// The previous control target. This will have been set at the end of a previous update (but
	// only if the control was enabled etc), so to check if it is valid, check the update counter.
	UE::PhysicsControl::FPosQuat PreviousTargetTM;

	// Update counter set when the control was last updated.
	FGraphTraversalCounter ExpectedUpdateCounter;

	// This is the currently active control data. It will be updated just prior to applying
	// the controls, by setting it to the default, and then updating it with any parameters. This is
	// so that control updates can be made ephemeral.
	FPhysicsControlData ControlData;

	// This is the currently active control multiplier. It will be updated just prior to applying
	// the controls, by setting it to the default, and then updating it with any parameters. This is
	// so that control multiplier updates can be made ephemeral.
	FPhysicsControlMultiplier ControlMultiplier;

	// Cached child body index - needs to be updated whenever the bone name changes
	int32 ChildBodyIndex;

	// Cached parent body index - needs to be updated whenever the bone name changes
	int32 ParentBodyIndex;

	// The low-level physics representation
	ImmediatePhysics::FJointHandle* JointHandle;
};

// This is the record for a modifier. It stores the "original" data, which will normally stay fixed.
// In addition, it will also store some updates to these original data which will determine how the actual
// body properties get set.
struct FRigidBodyModifierRecord
{
	FRigidBodyModifierRecord(const FPhysicsBodyModifier& InModifier, ImmediatePhysics::FActorHandle* InActorHandle);

	void ResetCurrent();

	FPhysicsBodyModifier            Modifier;

	// This contains the currently active modifier data. It will be updated just prior to
	// applying the controls, by setting it to the default, and then updating it with any parameters.
	FPhysicsControlModifierData ModifierData;

	ImmediatePhysics::FActorHandle* ActorHandle;
};

