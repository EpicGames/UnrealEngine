// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PhysicsControlPoseData.h"

// Use the simulation space functions from the RBAN
#include "BoneControllers/AnimNode_RigidBody.h" 

struct FComponentSpacePoseContext;

namespace UE
{
namespace PhysicsControl
{

//======================================================================================================================
struct FOutputBoneData
{
	FOutputBoneData()
		: CompactPoseBoneIndex(INDEX_NONE), CompactPoseParentBoneIndex(INDEX_NONE)
		, BodyIndex(0), ParentBodyIndex(0)
	{}

	TArray<FCompactPoseBoneIndex> BoneIndicesToParentBody;
	FCompactPoseBoneIndex CompactPoseBoneIndex;
	FCompactPoseBoneIndex CompactPoseParentBoneIndex;
	int32 BodyIndex; // Index into Bodies - will be the same index as into the joints
	int32 ParentBodyIndex;
};


//======================================================================================================================
// Caches the pose for RigidBodyWithControl
struct FRigidBodyPoseData
{
public:
	FRigidBodyPoseData() {}

public:
	void Update(
		FComponentSpacePoseContext&    ComponentSpacePoseContext,
		const TArray<FOutputBoneData>& OutputBoneData,
		const ESimulationSpace         SimulationSpace,
		const FBoneReference&          BaseBoneRef,
		const FGraphTraversalCounter&  InUpdateCounter);

	UE::PhysicsControl::FPosQuat GetTM(int32 Index) const { 
		check(IsValidIndex(Index)); check(!BoneTMs[Index].ContainsNaN()); return BoneTMs[Index]; }
	bool IsValidIndex(const int32 Index) const { return BoneTMs.IsValidIndex(Index); }
	bool IsEmpty() const { return BoneTMs.IsEmpty(); }

	void SetSize(const int32 NumBones) { BoneTMs.SetNum(NumBones); }

	// The cached skeletal data, updated at the start of each tick
	TArray<UE::PhysicsControl::FPosQuat> BoneTMs;

	// Track when we were currently/last updated so the user can detect missing updates if calculating
	// velocity etc
	FGraphTraversalCounter UpdateCounter;
	// When the update is called we'll take the current counter, increment it, and store here so it
	// can be compared.
	FGraphTraversalCounter ExpectedUpdateCounter;
};

} // namespace PhysicsControl
} // namespace UE
