// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
public class PhysicsControlEditor : ModuleRules
{
	public PhysicsControlEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] 
			{
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AdvancedPreviewScene",
				"AnimationCore",
				"AnimationEditMode",
				"AnimGraph",
				"AnimGraphRuntime",
				"ApplicationCore",
				"AssetDefinition",
				"BlueprintGraph",
				"Chaos",
				"ComponentVisualizers",
				"Core",
				"CoreUObject",
				"EditorFramework",
				"EditorStyle",
				"Engine",
				"InputCore",
				"Persona",
				"PinnedCommandList",
				"PhysicsControl",
				"PhysicsControlUncookedOnly",
				"SkeletonEditor",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd",
			}
			);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AnimationEditor",
					"AnimGraph",
					"BlueprintGraph",
					"EditorFramework",
					"Kismet",
					"UnrealEd",
				}
			);
		}

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
