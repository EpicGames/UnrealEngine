// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_PhysicsControlAsset.h"

#include "PhysicsControlAssetEditor.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_PhysicsControlAsset"

EAssetCommandResult UAssetDefinition_PhysicsControlAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = 
		OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (UPhysicsControlAsset* PhysicsControlAsset : OpenArgs.LoadObjects<UPhysicsControlAsset>())
	{
		const TSharedRef<FPhysicsControlAssetEditor> NewEditor(new FPhysicsControlAssetEditor());
		NewEditor->InitAssetEditor(Mode, OpenArgs.ToolkitHost, PhysicsControlAsset);
	}
	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
