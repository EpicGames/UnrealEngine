// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "PhysicsControlAsset.h"
#include "AssetDefinition_PhysicsControlAsset.generated.h"

UCLASS()
class UAssetDefinition_PhysicsControlAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override
	{
		return NSLOCTEXT("AssetDefinition", "AssetDefinition_PhysicsControlAsset", "Physics Control Asset");
	}

	virtual FLinearColor GetAssetColor() const override
	{
		return FColor(255, 192, 128);
	}

	virtual TSoftClassPtr<UObject> GetAssetClass() const override
	{
		return UPhysicsControlAsset::StaticClass();
	}

	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Physics, NSLOCTEXT("AssetDefinition", "AssetDefinition_PhysicsControlAssetSubMenu", "Physical Material"), ECategoryMenuType::Section)
		};
		return Categories;
	}

	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
