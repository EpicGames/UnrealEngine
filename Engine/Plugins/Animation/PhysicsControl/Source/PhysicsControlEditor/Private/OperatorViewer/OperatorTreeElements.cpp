// Copyright Epic Games, Inc. All Rights Reserved.

#include "OperatorViewer/OperatorTreeElements.h"

#include "AnimGraphNode_RigidBodyWithControl.h"
#include "BlueprintEditorModule.h"
#include "Input/Reply.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Styling/SlateColor.h"
#include "Types/SlateEnums.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

// UE_DISABLE_OPTIMIZATION;

#define LOCTEXT_NAMESPACE "PhysicsControlOperatorViewer"

// class OperatorTreeItem //
const EHorizontalAlignment OperatorTreeItem::HorizontalAlignment = HAlign_Left;
const EVerticalAlignment OperatorTreeItem::VerticalAlignment = VAlign_Top;
const float OperatorTreeItem::HorizontalPadding = 5.0f;
const float OperatorTreeItem::VerticalPadding = 1.0f;

static const FName OperatorTagControl("Control");
static const FName OperatorTagModifier("Modifier");

bool MatchSearchText(const FString& TextToSearch, const TArray<TArray<FString>>& StructuredSearchCriteria)
{
	for (const TArray<FString>& AndCriteria : StructuredSearchCriteria)
	{
		bool bMatchFound = true;

		for (const FString& Token : AndCriteria)
		{
			bMatchFound &= TextToSearch.Contains(Token);
		}

		if (bMatchFound)
		{
			return true;
		}
	}

	return false;
}

// class OperatorTreeItem //
FReply OperatorTreeItem::OnDoubleClick()
{ 
	return FReply::Unhandled(); 
}

// class OperatorTreeControlItem //
const TArray<FName> OperatorTreeControlItem::HiddenTags = { OperatorTagControl, OperatorTagModifier };

FText OperatorTreeControlItem::Name() const
{
	return FText::FromName(NameAndTags.Key);
}

FText OperatorTreeControlItem::Description() const
{
	FString DescriptionStr;
	
	DescriptionStr += OperatorTypeName().ToString();
	DescriptionStr += FString(" ");
	DescriptionStr += NameAndTags.Key.ToString();

	for (const FName TagName : NameAndTags.Value)
	{
		if (!HiddenTags.Contains(TagName))
		{
			DescriptionStr += " " + TagName.ToString();
		}
	}

	DescriptionStr.TrimEndInline();

	return FText::FromString(DescriptionStr);
}

FName OperatorTreeControlItem::OperatorTypeName() const
{
	if (HasTag(OperatorTagControl))
	{
		return OperatorTagControl;
	}

	if (HasTag(OperatorTagModifier))
	{
		return OperatorTagModifier;
	}

	return NAME_None;
}

TSharedRef<ITableRow> OperatorTreeControlItem::GenerateRow(const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedPtr<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	// Display Operator Type Indicator
	{
		FText OperatorTypeLabel;
		FLinearColor OperatorTypeColor;

		if (HasTag(OperatorTagControl))
		{
			OperatorTypeLabel = LOCTEXT("Control", "Control");
			OperatorTypeColor = FLinearColor::Yellow;
		}

		if (HasTag(OperatorTagModifier))
		{
			OperatorTypeLabel = LOCTEXT("Modifier", "Modifier");
			OperatorTypeColor = FLinearColor::Red;
		}

		HorizontalBox->AddSlot()
			.HAlign(HorizontalAlignment)
			.Padding(HorizontalPadding, VerticalPadding)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(OperatorTypeLabel)
				.ColorAndOpacity(OperatorTypeColor)
			];
	}

	// Display Operator Name
	{
		HorizontalBox->AddSlot()
			.HAlign(HorizontalAlignment)
			.VAlign(VerticalAlignment)
			.Padding(HorizontalPadding, VerticalPadding)
			.FillWidth(0.4)
			[
				SNew(STextBlock)
				.Text(FText::FromName(NameAndTags.Key))
			];
	}

	// Display Sets
	{
		TSharedPtr<SWrapBox> TagBox = SNew(SWrapBox)
			.UseAllottedSize(true)
			.InnerSlotPadding(FVector2D(4, 4));

		for (const FName TagName : NameAndTags.Value)
		{
			if (!HiddenTags.Contains(TagName))
			{
				TagBox->AddSlot()
					[
						SNew(STextBlock)
						.Text(FText::FromName(TagName))
						.ColorAndOpacity(FLinearColor::White)
					];
			}
		}

		TagBox->SetVisibility(EVisibility::Visible);

		HorizontalBox->AddSlot()
			.HAlign(HAlign_Fill)
			.Padding(HorizontalPadding, VerticalPadding)
			[
				TagBox.ToSharedRef()
			];
	}

	return SNew(STableRow<TSharedPtr<OperatorTreeItem>>, OwnerTable)
		[
			HorizontalBox.ToSharedRef()
		];
}

bool OperatorTreeControlItem::MatchesFilterText(const TArray<TArray<FString>>& FilterStructuredCriteria) const
{
	if (FilterStructuredCriteria.IsEmpty())
	{
		return true;
	}

	FString TextToSearch = NameAndTags.Key.ToString();

	for (const FName Tag : NameAndTags.Value)
	{
		TextToSearch += Tag.ToString();
	}

	return MatchSearchText(TextToSearch, FilterStructuredCriteria);
}

bool OperatorTreeControlItem::HasTag(const FName Tag) const
{
	return NameAndTags.Value.Contains(Tag);
}

// class OperatorTreeNodeItem //
OperatorTreeNodeItem::OperatorTreeNodeItem(const UEdGraphNode* InNode, const FName& InName, const FString& InBlueprintName)
: Node(InNode)
{
	const FString Name = Node->GetNodeTitle(ENodeTitleType::FullTitle).ToString() + " [" + InBlueprintName + "]";
	NodeDisplayName = FText::FromString(Name);
}

FText OperatorTreeNodeItem::Name() const
{
	return NodeDisplayName;
}

TSharedRef<ITableRow> OperatorTreeNodeItem::GenerateRow(const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(STableRow<TSharedPtr<OperatorTreeItem>>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Text(NodeDisplayName) // TODO - update on changes
				.ColorAndOpacity_Lambda([this]() { return this->ColorAndOpacity(); })
			]
		];
}

bool OperatorTreeNodeItem::IsSelectedInGraph() const
{
	return Node && Node->IsSelected();
}

FSlateColor OperatorTreeNodeItem::ColorAndOpacity() const
{
	return IsSelectedInGraph() ? FAppStyle::Get().GetSlateColor("Colors.AccentOrange") : FAppStyle::Get().GetSlateColor("Colors.White");
}

FReply OperatorTreeNodeItem::OnDoubleClick()
{
	if (UBlueprint* const Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(Node))
	{
		if (IBlueprintEditor* BlueprintEditor = static_cast<IBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(Blueprint, true)))
		{
			BlueprintEditor->JumpToHyperlink(Node, false);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

bool operator==(const OperatorTreeNodeItem& Lhs, const OperatorTreeNodeItem& Rhs)
{
	return Lhs.GraphNode()->GetFName().IsEqual(Rhs.GraphNode()->GetFName()); // note: must use IsEqual as operator == doesn't account for the FName's number postfix.
}

// class OperatorTreeMessageItem //
OperatorTreeMessageItem::OperatorTreeMessageItem(const UEdGraphNode* InNode, const MessageType InType, const FText& InText)
	: Text(InText)
	, Node(InNode)
	, Type(InType)
{}

FText OperatorTreeMessageItem::Name() const
{
	return Text;
}

TSharedRef<ITableRow> OperatorTreeMessageItem::GenerateRow(const TSharedRef<STableViewBase>& OwnerTable) const
{
	FText MessageTypeLabel;
	FLinearColor MessageTypeColor;

	if (Type == MessageType::Log)
	{
		MessageTypeLabel = LOCTEXT("MessageItemLog", "Log");
		MessageTypeColor = FLinearColor::Gray;
	}

	if (Type == MessageType::Warning)
	{
		MessageTypeLabel = LOCTEXT("MessageItemWarning", "Warning");
		MessageTypeColor = FLinearColor::Yellow;
	}

	if (Type == MessageType::Error)
	{
		MessageTypeLabel = LOCTEXT("MessageItemError", "Error");
		MessageTypeColor = FLinearColor::Red;
	}

	return SNew(STableRow<TSharedPtr<OperatorTreeItem>>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HorizontalAlignment)
			.Padding(HorizontalPadding, VerticalPadding)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(MessageTypeLabel)
				.ColorAndOpacity(MessageTypeColor)
			]
			+ SHorizontalBox::Slot()
			.HAlign(HorizontalAlignment)
			.Padding(HorizontalPadding, VerticalPadding)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(Text)
				.ColorAndOpacity(MessageTypeColor)
			]
		];
}

template<typename TDerivedType> bool IsEqualAfterCast(const TSharedPtr<OperatorTreeItem> Lhs, const TSharedPtr<OperatorTreeItem> Rhs)
{
	TSharedPtr<TDerivedType> LhsDerived = StaticCastSharedPtr<TDerivedType>(Lhs);
	TSharedPtr<TDerivedType> RhsDerived = StaticCastSharedPtr<TDerivedType>(Rhs);

	return LhsDerived && RhsDerived && (*LhsDerived == *RhsDerived);
}

bool IsEqual(const TSharedPtr<OperatorTreeItem> Lhs, const TSharedPtr<OperatorTreeItem> Rhs)
{
	if (Lhs->GetType() == Rhs->GetType())
	{
		const OperatorTreeItem::OperatorTreeItemType ItemType = Lhs->GetType();

		if (ItemType == OperatorTreeControlItem::StaticType())
		{
			return IsEqualAfterCast<OperatorTreeControlItem>(Lhs, Rhs);
		}

		if (ItemType == OperatorTreeMessageItem::StaticType())
		{
			return IsEqualAfterCast<OperatorTreeMessageItem>(Lhs, Rhs);
		}

		if (ItemType == OperatorTreeNodeItem::StaticType())
		{
			return IsEqualAfterCast<OperatorTreeNodeItem>(Lhs, Rhs);
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
