// Copyright Epic Games, Inc. All Rights Reserved.

#include "OperatorViewer/OperatorViewer.h"
#include "OperatorViewer/SOperatorViewerTabWidget.h"
#include "OperatorViewer/OperatorViewer.h"

#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBox.h"


// UE_DISABLE_OPTIMIZATION;

static const FName PhysicsControlEditorModule_OperatorNamesTabWidget("PhysicsControlEditorModule_OperatorNamesTabWidget");

#define LOCTEXT_NAMESPACE "PhysicsControlOperatorViewer"

void FPhysicsControlOperatorViewer::Startup()
{
	// Physics Operator Names Tool
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(PhysicsControlEditorModule_OperatorNamesTabWidget, 
		FOnSpawnTab::CreateRaw(this, &FPhysicsControlOperatorViewer::OnCreateTab))
		.SetDisplayName(LOCTEXT("PhysicsAnimationEditor_OperatorNamesTabTitle", "Rigid Body With Control"))
		.SetMenuType(ETabSpawnerMenuType::Hidden);
}

void FPhysicsControlOperatorViewer::Shutdown()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(PhysicsControlEditorModule_OperatorNamesTabWidget);
}

void FPhysicsControlOperatorViewer::OpenOperatorNamesTab()
{
	OperatorNamesTab = FGlobalTabmanager::Get()->FindExistingLiveTab(PhysicsControlEditorModule_OperatorNamesTabWidget);

	if (!OperatorNamesTab)
	{
		OperatorNamesTab = FGlobalTabmanager::Get()->TryInvokeTab(PhysicsControlEditorModule_OperatorNamesTabWidget);
	}
}

void FPhysicsControlOperatorViewer::CloseOperatorNamesTab()
{
	if (!OperatorNamesTab)
	{
		OperatorNamesTab = FGlobalTabmanager::Get()->FindExistingLiveTab(PhysicsControlEditorModule_OperatorNamesTabWidget);
	}

	if (OperatorNamesTab)
	{
		OperatorNamesTab->RequestCloseTab();
		OperatorNamesTab.Reset();
	}
}

void FPhysicsControlOperatorViewer::ToggleOperatorNamesTab()
{
	if (IsOperatorNamesTabOpen())
	{
		CloseOperatorNamesTab();
	}
	else
	{
		OpenOperatorNamesTab();
	}
}

bool FPhysicsControlOperatorViewer::IsOperatorNamesTabOpen()
{
	return OperatorNamesTab.IsValid();
}

void FPhysicsControlOperatorViewer::RequestRefresh()
{
	if (PersistentTabWidget)
	{
		PersistentTabWidget->RequestRefresh();
	}
}

TSharedRef<SDockTab> FPhysicsControlOperatorViewer::OnCreateTab(const FSpawnTabArgs& SpawnTabArgs)
{
	const int TabIndex = 1;

	PersistentTabWidget = SNew(SOperatorViewerTabWidget, TabIndex);

	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.OnTabClosed_Lambda([this](TSharedRef<class SDockTab> InParentTab) { this->OnTabClosed(InParentTab); })
		[
			SNew(SBox)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				PersistentTabWidget.ToSharedRef()
			]
		];
}

void FPhysicsControlOperatorViewer::OnTabClosed(TSharedRef<SDockTab> DockTab)
{
	OperatorNamesTab.Reset();
	PersistentTabWidget.Reset();
}

#undef LOCTEXT_NAMESPACE