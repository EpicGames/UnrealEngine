// Copyright Epic Games, Inc. All Rights Reserved.

#include "OperatorViewer/SOperatorTreeWidget.h"

#include "AnimGraphNode_RigidBodyWithControl.h"
#include "BlueprintEditorModule.h"
#include "Filters/GenericFilter.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

// UE_DISABLE_OPTIMIZATION;

#define LOCTEXT_NAMESPACE "PhysicsControlOperatorViewer"

template<typename TFilterItemType> bool PassesAnyFilters(const TFilterItemType InItem, const TSharedPtr<TFilterCollection<TFilterItemType>> FilterCollection)
{
	bool bPassesAnyFilter = true;

	if (FilterCollection)
	{
		bPassesAnyFilter = false;

		for (int32 FilterIndex = 0, FilterCount = FilterCollection->Num(); !bPassesAnyFilter && (FilterIndex < FilterCount); ++FilterIndex)
		{
			bPassesAnyFilter = FilterCollection->GetFilterAtIndex(FilterIndex)->PassesFilter(InItem);
		}
	}

	return bPassesAnyFilter;
}

template<typename TFilterItemType> bool PassesAllFilters(const TFilterItemType InItem, const TSharedPtr<TFilterCollection<TFilterItemType>> FilterCollection)
{
	return !FilterCollection || FilterCollection->PassesAllFilters(InItem);
}

TArray<TArray<FString>> BuildStructuredSearchCriteria(const FText& SearchCriteriaText)
{
	const FString OROperatorChar(TEXT("|"));
	const FString OROperatorText(TEXT(" OR "));

	FString SearchCriteriaString = SearchCriteriaText.ToString();
	SearchCriteriaString.ReplaceInline(*OROperatorText, *OROperatorChar);

	TArray<FString> OrSeparatedCriteria;
	SearchCriteriaString.ParseIntoArray(OrSeparatedCriteria, *OROperatorChar, true);

	TArray<TArray<FString>> StructuredSearchCriteria;
	const TCHAR* Delimeters[2] = { TEXT(" "), TEXT("+") };

	for (const FString& CriteriaText : OrSeparatedCriteria)
	{
		StructuredSearchCriteria.Add(TArray<FString>());
		CriteriaText.ParseIntoArray(StructuredSearchCriteria.Last(), Delimeters, 2, true);
	}

	return StructuredSearchCriteria;
}

void FormatStringForClipboard(FString& CopyBfr, const TCHAR* Delimeter)
{
	// Format buffer for pasting to an array in the details panel.
	CopyBfr.ReplaceInline(Delimeter, TEXT("\",\""));
	CopyBfr.InsertAt(0, TEXT("(\""));
	CopyBfr.InsertAt(CopyBfr.Len(), TEXT("\")"));
}

// class SFilteredTreeWidget //
void SOperatorTreeWidget::Construct(const FArguments& InArgs)
{
	Construct_Internal();

	Refresh();
}

void SOperatorTreeWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	Parent::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const bool bIsScrollBarVisible = TreeView->GetScrollbarVisibility() == EVisibility::Visible;

	if (bRefreshRequested)
	{
		Refresh();
	}
}

void SOperatorTreeWidget::Construct_Internal()
{
	CreateCustomFilters();

	SAssignNew(SearchBox, SSearchBox)
		.SelectAllTextWhenFocused(true)
		.OnTextChanged(this, &SOperatorTreeWidget::OnFilterTextChanged)
		.HintText(LOCTEXT("SearchBoxHint", "Search the list of assets..."))
		.AddMetaData<FTagMetaData>(TEXT("PerfToolAssetBrowser.Search"));

	SAssignNew(FilterBar, FilterBarType)
		.OnFilterChanged(this, &SOperatorTreeWidget::OnFilterBarFilterChanged)
		.CustomFilters(CustomFilters);

	TSharedPtr<SVerticalBox> VerticalBox = SNew(SVerticalBox);
	TSharedPtr<SHorizontalBox> Toolbar = SNew(SHorizontalBox);

	// Add Filter Menu Button
	Toolbar->AddSlot()
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f, 2.0f, 0.0f)
		.AutoWidth()
		[
			SBasicFilterBar<ItemType>::MakeAddFilterButton(FilterBar.ToSharedRef())
		];

	// Add the Search Text Box
	Toolbar->AddSlot()
		.VAlign(VAlign_Center)
		[
			SearchBox.ToSharedRef()
		];

	// Add the toolbar to the main layout vertical box
	VerticalBox->AddSlot()
		.AutoHeight()
		.MaxHeight(24)
		[
			Toolbar.ToSharedRef()
		];

	// Add the filter bar to the main layout vertical box
	VerticalBox->AddSlot()
		.AutoHeight()
		.MaxHeight(24)
		.Padding(0.0f, 0.0f, 0.0f, 4.0f)
		[
			FilterBar.ToSharedRef()
		];

	SAssignNew(TreeView, STreeView<ItemType>)
		.TreeItemsSource(&TreeItems)
		.OnGetChildren(this, &SOperatorTreeWidget::OnItemGetChildren)
		.OnGenerateRow(this, &SOperatorTreeWidget::GenerateItemRow)
		.SelectionMode(ESelectionMode::Multi)
		.OnContextMenuOpening(this, &SOperatorTreeWidget::CreateContextMenu)
		.OnKeyDownHandler(this, &SOperatorTreeWidget::OnKeyDown)
		.OnMouseButtonDoubleClick(this, &SOperatorTreeWidget::OnItemDoubleClicked);

	VerticalBox->AddSlot()
		.VAlign(VAlign_Fill)
		[
			TreeView.ToSharedRef()
		];

	ChildSlot
		[
			VerticalBox.ToSharedRef()
		];
}

TSharedPtr<SWidget> SOperatorTreeWidget::CreateContextMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr, nullptr);
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ChannelDrawStyle", "Draw Style"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("OperatorTreeCopyNames", "Copy Names"),
			LOCTEXT("OperatorTreeCopyNames_Tooltip", "Copy the names of all the selected operators to the clipboard."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Copy"),
			FUIAction(FExecuteAction::CreateSP(this, &SOperatorTreeWidget::CopySelectedItemsNamesToClipboard)));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("OperatorTreeCopyAll", "Copy All"),
			LOCTEXT("OperatorTreeCopyAll_Tooltip", "Copy the full description of all the selected operators to the clipboard."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Copy"),
			FUIAction(FExecuteAction::CreateSP(this, &SOperatorTreeWidget::CopySelectedItemsDescriptionsToClipboard)));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<ITableRow> SOperatorTreeWidget::GenerateItemRow(ItemType InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return InItem->GenerateRow(OwnerTable);
}

void SOperatorTreeWidget::CopySelectedItemsNamesToClipboard() const
{
	const TCHAR* Delimeter = TEXT("\n");
	FString CopyBfr;
	uint32 ItemCount = 0;

	for (ItemType SelectedItem : TreeView->GetSelectedItems())
	{
		CopyBfr += SelectedItem->Name().ToString() + Delimeter;
		++ItemCount;
	}

	CopyBfr.RemoveFromEnd(Delimeter);

	if (ItemCount > 1)
	{
		FormatStringForClipboard(CopyBfr, Delimeter);
	}

	FPlatformApplicationMisc::ClipboardCopy(*CopyBfr);
}

void SOperatorTreeWidget::CopySelectedItemsDescriptionsToClipboard() const
{
	constexpr TCHAR Delimeter = '\n';
	FString CopyBfr;

	for (ItemType SelectedItem : TreeView->GetSelectedItems())
	{
		CopyBfr += SelectedItem->Description().ToString() + Delimeter;
	}

	CopyBfr.TrimEndInline();

	FPlatformApplicationMisc::ClipboardCopy(*CopyBfr);
}

FReply SOperatorTreeWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.IsControlDown() && (InKeyEvent.GetKey() == EKeys::C))
	{
		CopySelectedItemsNamesToClipboard();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SOperatorTreeWidget::OnItemDoubleClicked(ItemType InItem)
{
	if (!InItem->OnDoubleClick().IsEventHandled())
	{
		TreeView->SetItemSelection(InItem, true, ESelectInfo::OnMouseClick);
		CopySelectedItemsNamesToClipboard();
	}
}

void SOperatorTreeWidget::OnFilterBarFilterChanged()
{
	Refresh();
}

void SOperatorTreeWidget::OnFilterTextChanged(const FText& SearchCriteriaText)
{
	FilterStructuredCriteria = BuildStructuredSearchCriteria(SearchCriteriaText);

	for (const FName Set : SetNames)
	{
		if (MatchSearchText(Set.ToString(), FilterStructuredCriteria))
		{
			FilterSetNames.Add(Set);
		}
	}

	Refresh();
}

void SOperatorTreeWidget::RequestRefresh()
{
	bRefreshRequested = true;
}

void SOperatorTreeWidget::Refresh()
{
	using OperatorNameAndTags = OperatorTreeControlItem::OperatorNameAndTags;

	check(FilterBar.IsValid());

	TSet<ItemType> ExpandedItems;
	TreeView->GetExpandedItems(ExpandedItems);

	TreeItems.Reset();
	SetNames.Reset();

	TArray<OperatorNameAndTags> OperatorNamesAndTags;
	TSet<FString> ProcessedNodeNames;

	for (TObjectIterator<UAnimGraphNode_RigidBodyWithControl> Itr; Itr; ++Itr)
	{	
		FString BlueprintName = "[Unknown]";

		if (UAnimBlueprint* const AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(*Itr)))
		{
			BlueprintName = AnimBlueprint->GetFriendlyName();
		}

		const FString NodeUniqueName = Itr->GetName() + " [" + BlueprintName + "]";

		if (!ProcessedNodeNames.Contains(NodeUniqueName))
		{
			// This is the first time we've encountered this node.
			ProcessedNodeNames.Add(NodeUniqueName);

			OperatorNamesAndTags.Reset();
			OperatorNamesAndTags = Itr->GenerateControlsAndBodyModifierNames();

			for (const OperatorNameAndTags& Operator : OperatorNamesAndTags)
			{
				// TODO - Define a way to sort these operators.

				for (const FName SetName : Operator.Value)
				{
					SetNames.Add(SetName);
				}
			}

			// Create node item
			NodeItemPtr Node = MakeShared<OperatorTreeNodeItem>(*Itr, FName(*NodeUniqueName), BlueprintName);

			if (!IsValid(Itr->Node.OverridePhysicsAsset))
			{
				Node->AddChild(MakeShared<OperatorTreeMessageItem>(*Itr, OperatorTreeMessageItem::Warning, LOCTEXT("OperatorNoPhysicsAssetDefinedText", "Can't create operators for limbs because no Physics Asset has been defined for this node.")));
			}

			// Create items for controls and modifiers
			for (const OperatorNameAndTags& Operator : OperatorNamesAndTags)
			{
				OperatorTreeItemPtr Item = MakeShared<OperatorTreeControlItem>(Operator);

				if (MatchesSearchAndFilter(Item))
				{
					Node->AddChild(Item);
				}
			}

			if (OperatorNamesAndTags.IsEmpty())
			{
				Node->AddChild(MakeShared<OperatorTreeMessageItem>(*Itr, OperatorTreeMessageItem::Log, LOCTEXT("OperatorNoOperatorsDefinedText", "No operators defined for this node.")));
			}
			else if (!Node->HasChildren())
			{
				Node->AddChild(MakeShared<OperatorTreeMessageItem>(*Itr, OperatorTreeMessageItem::Log, LOCTEXT("OperatorNoOperatorsMatchSearchCriteria", "No operators match search criteria.")));
			}
			
			TreeItems.Add(Node);
		}
	}

	TreeView->RequestTreeRefresh();

	// Find any new tree items that match old tree items that were expanded and expand them.
	for (ItemType Item : ExpandedItems)
	{
		if (ItemType* const MatchingExpandedItem = TreeItems.FindByPredicate([Item](const ItemType InItem) -> bool { return IsEqual(InItem, Item); }))
		{
			TreeView->SetItemExpansion(*MatchingExpandedItem, true);
		}
	}

	bRefreshRequested = false;
}

void SOperatorTreeWidget::OnItemGetChildren(ItemType InItem, TArray<ItemType>& OutChildren)
{
	OutChildren = InItem->Children;
}

void SOperatorTreeWidget::CreateCustomFilters()
{
	using FilterType = FGenericFilter<ItemType>;

	TSharedPtr<FFilterCategory> OperatorTypeFilterCategory = MakeShared<FFilterCategory>(LOCTEXT("OperatorTypeFilterCategory", "Type"), FText::GetEmpty());
	TSharedPtr<FFilterCategory> SetFilterCategory = MakeShared<FFilterCategory>(LOCTEXT("SetFilterCategory", "Set"), FText::GetEmpty());

	auto CreateOperatorTypeFilter = [this, &OperatorTypeFilterCategory](const FName OperatorTypeName)
	{
		const FString FilterName = FString("OperatorType") + OperatorTypeName.ToString();

		FilterType::FOnItemFiltered FilterDelegate = FilterType::FOnItemFiltered::CreateSP(this, &SOperatorTreeWidget::HasTag, OperatorTypeName);
		TSharedPtr<FilterType> Filter = MakeShared<FilterType>(OperatorTypeFilterCategory, FilterName, FText::FromName(OperatorTypeName), FilterDelegate);
		Filter->SetToolTipText(FText::Format(LOCTEXT("OperatorTypeFilterTooltip", "Only show {0}s."), FText::FromName(OperatorTypeName)));
		this->CustomFilters.Add(Filter.ToSharedRef());
	};

	auto CreateSetFilter = [this, &SetFilterCategory](const FName SetName)
	{
		const FString FilterName = FString("Set") + SetName.ToString();

		FilterType::FOnItemFiltered FilterDelegate = FilterType::FOnItemFiltered::CreateSP(this, &SOperatorTreeWidget::HasTag, SetName);
		TSharedPtr<FilterType> Filter = MakeShared<FilterType>(SetFilterCategory, FilterName, FText::FromName(SetName), FilterDelegate);
		Filter->SetToolTipText(FText::Format(LOCTEXT("SetFilterTooltip", "Only show controls or modifiers in the {0} set."), FText::FromName(SetName)));
		this->CustomFilters.Add(Filter.ToSharedRef());
	};

	this->CustomFilters.Reset();

	CreateOperatorTypeFilter(FName("Control"));
	CreateOperatorTypeFilter(FName("Modifier"));

	CreateSetFilter(FName("ParentSpace"));
	CreateSetFilter(FName("WorldSpace"));	
}

bool SOperatorTreeWidget::HasTag(const ItemType Item, const FName InTag)
{
	return Item->HasTag(InTag);
}

bool SOperatorTreeWidget::MatchesSearchAndFilter(const ItemType InItem) const
{
	TSharedPtr<TFilterCollection<ItemType>> FilterCollection = FilterBar->GetAllActiveFilters();
	check(FilterCollection);

	return PassesAllFilters(InItem, FilterCollection) && (FilterStructuredCriteria.IsEmpty() || InItem->MatchesFilterText(FilterStructuredCriteria));
}

#undef LOCTEXT_NAMESPACE
