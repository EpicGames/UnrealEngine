// Copyright Epic Games, Inc. All Rights Reserved.

#include "OperatorViewer/SOperatorViewerTabWidget.h"
#include "OperatorViewer/SOperatorTreeWidget.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"

// UE_DISABLE_OPTIMIZATION;

#define LOCTEXT_NAMESPACE "PhysicsControlOperatorViewer"

void SOperatorViewerTabWidget::Construct(const FArguments& InArgs, int32 InTabIndex)
{
	TSharedPtr<SVerticalBox> TreeViewBox;

	ChildSlot.Padding(5, 5, 5, 5)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SAssignNew(TreeViewBox, SVerticalBox)
			]
	
		];

	// Tree View
	{
		TreeViewWidget = SNew(SOperatorTreeWidget);

		TreeViewBox->AddSlot()
			[
				TreeViewWidget.ToSharedRef()
			];
	}
}

void SOperatorViewerTabWidget::RequestRefresh()
{
	if (TreeViewWidget.IsValid())
	{
		TreeViewWidget->RequestRefresh();
	}
}

#undef LOCTEXT_NAMESPACE