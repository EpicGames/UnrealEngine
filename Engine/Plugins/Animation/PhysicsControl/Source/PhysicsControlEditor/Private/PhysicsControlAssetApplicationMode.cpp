// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAssetApplicationMode.h"
#include "PhysicsControlAssetEditor.h"
#include "PhysicsControlAssetEditorTabSummoners.h"
#include "PhysicsControlAsset.h"

#include "PersonaModule.h"
#include "ISkeletonEditorModule.h"
#include "Modules/ModuleManager.h"
#include "PersonaTabs.h"

#define LOCTEXT_NAMESPACE "PhysicsControlAssetApplicationMode"

FName FPhysicsControlAssetApplicationMode::ModeName("PhysicsControlAssetEditMode");

//======================================================================================================================
FPhysicsControlAssetApplicationMode::FPhysicsControlAssetApplicationMode(
	TSharedRef<FWorkflowCentricApplication> InHostingApp,
	TSharedPtr<ISkeletonTree>               SkeletonTree,
	TSharedRef<IPersonaPreviewScene>        InPreviewScene)
	: 
	FApplicationMode(PhysicsControlAssetEditorModes::PhysicsControlAssetEditorMode)
{
	PhysicsControlAssetEditor = StaticCastSharedRef<FPhysicsControlAssetEditor>(InHostingApp);
	TSharedRef<FPhysicsControlAssetEditor> PhysicsControlAssetEditorSharedRef = 
		StaticCastSharedRef<FPhysicsControlAssetEditor>(InHostingApp);

	if (SkeletonTree.IsValid())
	{
		ISkeletonEditorModule& SkeletonEditorModule = 
			FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
		TabFactories.RegisterFactory(
			SkeletonEditorModule.CreateSkeletonTreeTabFactory(InHostingApp, SkeletonTree.ToSharedRef()));
	}

	TArray<TSharedPtr<FExtender>> ViewportExtenders;
	ViewportExtenders.Add(MakeShared<FExtender>());

	FPersonaViewportArgs ViewportArgs(InPreviewScene);
	ViewportArgs.bAlwaysShowTransformToolbar = true;
	ViewportArgs.bShowStats = false;
	ViewportArgs.bShowTimeline = true;
	ViewportArgs.bShowLODMenu = true;
	ViewportArgs.bShowPlaySpeedMenu = true;
	ViewportArgs.bShowPhysicsMenu = true;
	ViewportArgs.ContextName = TEXT("PhysicsControlAssetEditor.Viewport");
	ViewportArgs.Extenders = ViewportExtenders;
	ViewportArgs.OnViewportCreated = FOnViewportCreated::CreateSP(
		PhysicsControlAssetEditorSharedRef, &FPhysicsControlAssetEditor::HandleViewportCreated);

	// Register Persona tabs.
	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	TabFactories.RegisterFactory(PersonaModule.CreatePersonaViewportTabFactory(InHostingApp, ViewportArgs));
	TabFactories.RegisterFactory(PersonaModule.CreateAdvancedPreviewSceneTabFactory(InHostingApp, InPreviewScene));

	TabFactories.RegisterFactory(
		MakeShared<FPhysicsControlAssetEditorSetupTabSummoner>(
			InHostingApp, 
			CastChecked<UPhysicsControlAsset>(
				(*PhysicsControlAssetEditorSharedRef->GetObjectsCurrentlyBeingEdited())[0])));

	TabFactories.RegisterFactory(
		MakeShared<FPhysicsControlAssetEditorProfileTabSummoner>(
			InHostingApp,
			CastChecked<UPhysicsControlAsset>(
				(*PhysicsControlAssetEditorSharedRef->GetObjectsCurrentlyBeingEdited())[0])));

	TabFactories.RegisterFactory(
		MakeShared<FPhysicsControlAssetEditorPreviewTabSummoner>(
			InHostingApp,
			CastChecked<UPhysicsControlAsset>(
				(*PhysicsControlAssetEditorSharedRef->GetObjectsCurrentlyBeingEdited())[0])));

	TabFactories.RegisterFactory(
		MakeShared<FPhysicsControlAssetEditorControlSetsTabSummoner>(
			InHostingApp,
			CastChecked<UPhysicsControlAsset>(
				(*PhysicsControlAssetEditorSharedRef->GetObjectsCurrentlyBeingEdited())[0])));

	TabFactories.RegisterFactory(
		MakeShared<FPhysicsControlAssetEditorBodyModifierSetsTabSummoner>(
			InHostingApp,
			CastChecked<UPhysicsControlAsset>(
				(*PhysicsControlAssetEditorSharedRef->GetObjectsCurrentlyBeingEdited())[0])));

	// For standard tabs, these are provided by Persona. For custom tabs, the contents of the tab is
	// provided by a summoner. When Summoners are made, they register their name in the constructor.
	// These names then hook into the tab names below.

	// Create tab layout.
	TabLayout = FTabManager::NewLayout("Standalone_PhysicsControlAssetEditor_Layout_v1.1")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				// Control/modifier sets
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.3f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(1.0f)
					->AddTab(
						FPhysicsControlAssetEditorBodyModifierSetsTabSummoner::TabName, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(2.0f)
					->AddTab(
						FPhysicsControlAssetEditorControlSetsTabSummoner::TabName, ETabState::OpenedTab)
				)
			)
			->Split
			(
				// Skeleton
				FTabManager::NewStack()
				->SetSizeCoefficient(0.3f)
				->AddTab(FPersonaTabs::SkeletonTreeViewID, ETabState::OpenedTab)
			)
			->Split
			(
				// Preview window
				FTabManager::NewStack()
				->SetSizeCoefficient(1.0f)
				->AddTab(FPersonaTabs::PreviewViewportID, ETabState::OpenedTab)
			)
			->Split
			(
				// the profile and detail panels
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.5f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(1.0f)
					->AddTab(FPersonaTabs::AdvancedPreviewSceneSettingsID, ETabState::OpenedTab)
					->AddTab(
						FPhysicsControlAssetEditorSetupTabSummoner::TabName, ETabState::OpenedTab)
					->AddTab(
						FPhysicsControlAssetEditorProfileTabSummoner::TabName, ETabState::OpenedTab)
					->SetForegroundTab(FPhysicsControlAssetEditorSetupTabSummoner::TabName)
				)
				->Split(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.6f)
					->AddTab(
						FPhysicsControlAssetEditorPreviewTabSummoner::TabName, ETabState::OpenedTab)
				)
			)
		);

	PersonaModule.OnRegisterTabs().Broadcast(TabFactories, InHostingApp);
	LayoutExtender = MakeShared<FLayoutExtender>();
	PersonaModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender.Get());
	TabLayout->ProcessExtensions(*LayoutExtender.Get());
}

//======================================================================================================================
void FPhysicsControlAssetApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	if (TSharedPtr<FPhysicsControlAssetEditor> Editor = PhysicsControlAssetEditor.Pin())
	{
		Editor->RegisterTabSpawners(InTabManager.ToSharedRef());
		Editor->PushTabFactories(TabFactories);
	}
}

#undef LOCTEXT_NAMESPACE
