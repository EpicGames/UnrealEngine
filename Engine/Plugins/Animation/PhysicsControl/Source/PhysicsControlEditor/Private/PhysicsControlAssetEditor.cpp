// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAssetEditor.h"

#include "PhysicsControlAsset.h"
#include "PhysicsControlAssetApplicationMode.h"
#include "PhysicsControlAssetEditorCommands.h"
#include "PhysicsControlAssetEditorData.h"
#include "PhysicsControlAssetEditorEditMode.h"
#include "PhysicsControlAssetEditorPhysicsHandleComponent.h"
#include "PhysicsControlAssetEditorSkeletalMeshComponent.h"
#include "PhysicsControlAssetEditorSkeletonTreeBuilder.h"
#include "PhysicsControlAssetEditorToolMenuContext.h"
#include "PhysicsControlComponent.h"
#include "PhysicsControlEditorModule.h"
#include "PhysicsControlLog.h"

#include "Animation/DebugSkelMeshComponent.h"
#include "AnimationEditorPreviewActor.h"
#include "AnimPreviewInstance.h"
#include "Components/StaticMeshComponent.h"
#include "PropertyHandle.h"
#include "EditorModeManager.h"
#include "IDetailsView.h"
#include "IPersonaToolkit.h"
#include "IPinnedCommandList.h"
#include "ISkeletonEditorModule.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "PersonaModule.h"
#include "PersonaToolMenuContext.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Preferences/PersonaOptions.h"
#include "PropertyEditorModule.h"
#include "SkeletonTreePhysicsControlBodyItem.h"
#include "SkeletonTreePhysicsControlShapeItem.h"
#include "SkeletonTreeSelection.h"
#include "ToolMenus.h"
#include "UICommandList_Pinnable.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "PhysicsControlAssetEditor"

const FName PhysicsControlAssetEditorModes::PhysicsControlAssetEditorMode("PhysicsControlAssetEditorMode");
const FName PhysicsControlAssetEditorAppName = FName(TEXT("PhysicsControlAssetEditorApp"));

//======================================================================================================================
namespace PhysicsControlAssetEditor
{
	static TSharedPtr<FPhysicsControlAssetEditor> GetPhysicsControlAssetEditorFromToolContext(
		const FToolMenuContext& InMenuContext)
	{
		if (UPhysicsControlAssetEditorToolMenuContext* Context = 
			InMenuContext.FindContext<UPhysicsControlAssetEditorToolMenuContext>())
		{
			return Context->PhysicsControlAssetEditor.Pin();
		}
		return TSharedPtr<FPhysicsControlAssetEditor>();
	}
}


//======================================================================================================================
void FPhysicsControlAssetEditor::InitAssetEditor(
	const EToolkitMode::Type        Mode,
	const TSharedPtr<IToolkitHost>& InitToolkitHost,
	UPhysicsControlAsset*           InPhysicsControlAsset)
{
	bIsInitialized = false;

	// Initialise EditorData
	{
		EditorData = MakeShared<FPhysicsControlAssetEditorData>();
		EditorData->PhysicsControlAsset = InPhysicsControlAsset;
		EditorData->CachePreviewMesh();
	}

	USkeletalMesh* SkeletalMesh = InPhysicsControlAsset->GetPreviewMesh();
	USkeleton* Skeleton = SkeletalMesh ? SkeletalMesh->GetSkeleton() : nullptr;

	// Create Persona toolkit
	FPersonaToolkitArgs PersonaToolkitArgs;
	PersonaToolkitArgs.OnPreviewSceneCreated = FOnPreviewSceneCreated::FDelegate::CreateSP(
		this, &FPhysicsControlAssetEditor::HandlePreviewSceneCreated);
	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	PersonaToolkit = PersonaModule.CreatePersonaToolkit(InPhysicsControlAsset, PersonaToolkitArgs, Skeleton);
	PersonaModule.RecordAssetOpened(FAssetData(InPhysicsControlAsset));

	// Make the skeleton tree
	{
		FSkeletonTreeArgs SkeletonTreeArgs;
		SkeletonTreeArgs.OnSelectionChanged = FOnSkeletonTreeSelectionChanged::CreateSP(
			this, &FPhysicsControlAssetEditor::HandleSelectionChanged);
		SkeletonTreeArgs.PreviewScene = PersonaToolkit->GetPreviewScene();
		SkeletonTreeArgs.bShowBlendProfiles = false;
		SkeletonTreeArgs.bShowDebugVisualizationOptions = true;
		SkeletonTreeArgs.bAllowMeshOperations = false;
		SkeletonTreeArgs.bAllowSkeletonOperations = false;
		SkeletonTreeArgs.bHideBonesByDefault = true;
		SkeletonTreeArgs.OnGetFilterText = FOnGetFilterText::CreateSP(
			this, &FPhysicsControlAssetEditor::HandleGetFilterLabel);
		SkeletonTreeArgs.Extenders = MakeShared<FExtender>();
		SkeletonTreeArgs.Extenders->AddMenuExtension(
			"FilterOptions", EExtensionHook::After, GetToolkitCommands(), 
			FMenuExtensionDelegate::CreateSP(this, &FPhysicsControlAssetEditor::HandleExtendFilterMenu));
		SkeletonTreeArgs.Extenders->AddMenuExtension(
			"SkeletonTreeContextMenu", EExtensionHook::After, GetToolkitCommands(), 
			FMenuExtensionDelegate::CreateSP(this, &FPhysicsControlAssetEditor::HandleExtendContextMenu));

		if (SkeletalMesh)
		{
			if (UPhysicsAsset* PhysicsAsset = SkeletalMesh->GetPhysicsAsset())
			{
				SkeletonTreeArgs.Builder = SkeletonTreeBuilder = 
					MakeShared<FPhysicsControlAssetEditorSkeletonTreeBuilder>(PhysicsAsset);
			}
		}

		if (PersonaToolkit->GetSkeleton() && SkeletonTreeArgs.Builder.IsValid())
		{
			SkeletonTreeArgs.ContextName = GetToolkitFName();
			GetMutableDefault<UPersonaOptions>()->bFlattenSkeletonHierarchyWhenFiltering = false;
			GetMutableDefault<UPersonaOptions>()->bHideParentsWhenFiltering = true;

			ISkeletonEditorModule& SkeletonEditorModule =
				FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
			SkeletonTree = SkeletonEditorModule.CreateSkeletonTree(PersonaToolkit->GetSkeleton(), SkeletonTreeArgs);
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT(
				"Error_PhysicsControlAssetHasNoSkeleton",
				"Warning: Physics Control Asset has no skeleton assigned.\n"
				"This is likely to be because there is no valid Physics Asset. "
				"Fix this by assigning a Preview Physics Asset/Mesh in the Physics Control Asset."));
		}
	}

	bSelecting = false;

	GEditor->RegisterForUndo(this);

	// Register our commands. This will only register them if not previously registered
	FPhysicsControlAssetEditorCommands::Register();

	BindCommands();

	// Initialise the asset editor
	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(
		Mode,
		InitToolkitHost,
		PhysicsControlAssetEditorAppName,
		FTabManager::FLayout::NullLayout,
		bCreateDefaultStandaloneMenu,
		bCreateDefaultToolbar,
		InPhysicsControlAsset);

	// Create and set the application mode.
	AddApplicationMode(
		PhysicsControlAssetEditorModes::PhysicsControlAssetEditorMode,
		MakeShareable(new FPhysicsControlAssetApplicationMode(
			SharedThis(this), SkeletonTree, PersonaToolkit->GetPreviewScene())));
	SetCurrentMode(PhysicsControlAssetEditorModes::PhysicsControlAssetEditorMode);

	// Activate the editor mode.
	GetEditorModeManager().SetDefaultMode(FPhysicsControlAssetEditorEditMode::ModeName);
	GetEditorModeManager().ActivateMode(FPhysicsControlAssetEditorEditMode::ModeName);
	GetEditorModeManager().ActivateMode(FPersonaEditModes::SkeletonSelection);

	FPhysicsControlAssetEditorEditMode* EditorMode = GetEditorModeManager().
		GetActiveModeTyped<FPhysicsControlAssetEditorEditMode>(FPhysicsControlAssetEditorEditMode::ModeName);
	EditorMode->SetEditorData(SharedThis(this), EditorData);

	// Need to load the module before extending menus etc
	//FPhysicsControlEditorModule& EditorModule = 
	//	FModuleManager::LoadModuleChecked<FPhysicsControlEditorModule>("PhysicsControlEditor");

	ExtendMenu();
	ExtendToolbar();
	ExtendViewportMenus();
	RegenerateMenusAndToolbars();

	bIsInitialized = true;
}

//======================================================================================================================
TSharedPtr<FPhysicsControlAssetEditorData> FPhysicsControlAssetEditor::GetEditorData() const
{
	return EditorData;
}


//======================================================================================================================
void FPhysicsControlAssetEditor::ExtendMenu()
{
	// In PhAT, entries in this function appear in the main Edit menu list
	// For the moment, we don't have anything to add.
}

//======================================================================================================================
void FPhysicsControlAssetEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FAssetEditorToolkit::InitToolMenuContext(MenuContext);

	UPhysicsControlAssetEditorToolMenuContext* PhysicsControlAssetEditorContext = 
		NewObject<UPhysicsControlAssetEditorToolMenuContext>();
	PhysicsControlAssetEditorContext->PhysicsControlAssetEditor = SharedThis(this);
	MenuContext.AddObject(PhysicsControlAssetEditorContext);

	UPersonaToolMenuContext* PersonaContext = NewObject<UPersonaToolMenuContext>();
	PersonaContext->SetToolkit(GetPersonaToolkit());
	MenuContext.AddObject(PersonaContext);

	// Danny TODO I don't think we need this
	MenuContext.AppendCommandList(ViewportCommandList);
}

//======================================================================================================================
FText FPhysicsControlAssetEditor::GetSimulationToolTip() const
{
	if (EditorData->bNoGravitySimulation)
	{
		return FPhysicsControlAssetEditorCommands::Get().SimulationNoGravity->GetDescription();
	}
	else
	{
		return FPhysicsControlAssetEditorCommands::Get().Simulation->GetDescription();
	}
}

//======================================================================================================================
FSlateIcon FPhysicsControlAssetEditor::GetSimulationIcon() const
{
	if (EditorData->bNoGravitySimulation)
	{
		return FPhysicsControlAssetEditorCommands::Get().SimulationNoGravity->GetIcon();
	}
	else
	{
		return FPhysicsControlAssetEditorCommands::Get().Simulation->GetIcon();
	}
}

//======================================================================================================================
void FPhysicsControlAssetEditor::ExtendToolbar()
{
	// Extends the toolbar to the right of the save/locate buttons, selection of preview anim etc
	struct Local
	{
		static TSharedRef< SWidget > FillSimulateOptions(TSharedRef<FUICommandList> InCommandList)
		{
			const bool bShouldCloseWindowAfterMenuSelection = true;
			FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

			const FPhysicsControlAssetEditorCommands& Commands = FPhysicsControlAssetEditorCommands::Get();

			//selected simulation
			MenuBuilder.BeginSection("SimulationOptions", LOCTEXT("SimulationOptionsHeader", "Simulation Options"));
			{
				MenuBuilder.AddMenuEntry(Commands.SimulationNoGravity);
				MenuBuilder.AddMenuEntry(Commands.SimulationFloorCollision);
			}
			MenuBuilder.EndSection();

			return MenuBuilder.MakeWidget();
		}
	};


	FName ParentName;
	static const FName MenuName = GetToolMenuToolbarName(ParentName);

	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(MenuName);
	const FToolMenuInsert SectionInsertLocation("Asset", EToolMenuInsertType::After);

	ToolMenu->AddDynamicSection("Persona", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InToolMenu)
		{
			FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
			FPersonaModule::FCommonToolbarExtensionArgs Args;
			Args.bReferencePose = true;
			PersonaModule.AddCommonToolbarExtensions(InToolMenu, Args);
		}), SectionInsertLocation);

	const FPhysicsControlAssetEditorCommands& Commands = FPhysicsControlAssetEditorCommands::Get();

	{
		FToolMenuSection& Section = ToolMenu->AddSection("Compile", FText());
		Section.AddEntry(FToolMenuEntry::InitToolBarButton(
			Commands.Compile,
			LOCTEXT("Compile_Label", "Compile"),
			LOCTEXT("Compile_ToolTip", "Compiles all the data from this and any parent asset into runtime form")
		));
	}

	ToolMenu->AddDynamicSection("Simulation", FNewToolMenuDelegate::CreateLambda([&Commands](UToolMenu* InToolMenu)
		{
			TSharedPtr<FPhysicsControlAssetEditor> PhysicsControlAssetEditor =
				PhysicsControlAssetEditor::GetPhysicsControlAssetEditorFromToolContext(InToolMenu->Context);
			if (PhysicsControlAssetEditor)
			{
				FToolMenuSection& Section = InToolMenu->AddSection("Simulation", FText());
				// Simulate
				Section.AddEntry(FToolMenuEntry::InitToolBarButton(
					Commands.Simulation,
					LOCTEXT("Simulation", "Simulate"),
					TAttribute< FText >::Create(TAttribute< FText >::FGetter::CreateSP(
						PhysicsControlAssetEditor.Get(), &FPhysicsControlAssetEditor::GetSimulationToolTip)),
					TAttribute< FSlateIcon >::Create(TAttribute< FSlateIcon >::FGetter::CreateSP(
						PhysicsControlAssetEditor.Get(), &FPhysicsControlAssetEditor::GetSimulationIcon))));

				Section.AddEntry(FToolMenuEntry::InitComboButton("SimulationMode",
					FUIAction(FExecuteAction(), FCanExecuteAction::CreateSP(
						PhysicsControlAssetEditor.Get(), &FPhysicsControlAssetEditor::IsNotRunningSimulation)),
					FOnGetContent::CreateLambda([WeakPhysicsControlAssetEditor = PhysicsControlAssetEditor.ToWeakPtr()]() -> TSharedRef<SWidget>
						{
							if (TSharedPtr<FPhysicsControlAssetEditor> Editor = WeakPhysicsControlAssetEditor.Pin())
							{
								return Local::FillSimulateOptions(Editor->GetToolkitCommands());
							}
							return SNullWidget::NullWidget;
						}),
					LOCTEXT("SimulateCombo_Label", "Simulate Options"),
					LOCTEXT("SimulateComboToolTip", "Options for Simulation"),
					FSlateIcon(),
					true));
			}
		}), SectionInsertLocation);
}

//======================================================================================================================
TSharedRef<SWidget> FPhysicsControlAssetEditor::MakeConstraintScaleWidget()
{
	return
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SNumericEntryBox<float>)
						.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
						.AllowSpin(true)
						.MinSliderValue(0.0f)
						.MaxSliderValue(4.0f)
						.Value_Lambda([this]() { return EditorData->EditorOptions->ConstraintDrawSize; })
						.OnValueChanged_Lambda([this](float InValue) 
							{ 
								EditorData->EditorOptions->ConstraintDrawSize = InValue;
								RefreshPreviewViewport(); 
							})
						.OnValueCommitted_Lambda([this](float InValue, ETextCommit::Type InCommitType)
							{
								EditorData->EditorOptions->ConstraintDrawSize = InValue;
								EditorData->EditorOptions->SaveConfig();
								ViewportCommandList->WidgetInteraction(TEXT("ConstraintScaleWidget"));
								RefreshPreviewViewport();
							})
				]
		];
}

//======================================================================================================================
TSharedRef<SWidget> FPhysicsControlAssetEditor::MakeCollisionOpacityWidget()
{
	return
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SNumericEntryBox<float>)
						.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
						.AllowSpin(true)
						.MinValue(0.0f)
						.MaxValue(1.0f)
						.MinSliderValue(0.0f)
						.MaxSliderValue(1.0f)
						.Value_Lambda([this]() { return EditorData->EditorOptions->CollisionOpacity; })
						.OnValueChanged_Lambda([this](float InValue) 
							{ 
								EditorData->EditorOptions->CollisionOpacity = InValue; 
								RefreshPreviewViewport();
							})
						.OnValueCommitted_Lambda([this](float InValue, ETextCommit::Type InCommitType)
							{
								EditorData->EditorOptions->CollisionOpacity = InValue;
								EditorData->EditorOptions->SaveConfig();
								ViewportCommandList->WidgetInteraction(TEXT("CollisionOpacityWidget"));
								RefreshPreviewViewport();
							})
				]
		];
}

//======================================================================================================================
void FPhysicsControlAssetEditor::ExtendViewportMenus()
{
	// This scope ensures that menus we add/extend are scoped to us - i.e. only dipslay in our
	// editor, even if the menu is shared by other editors.
	FToolMenuOwnerScoped OwnerScoped(this);

	// Extend the "Character" menu in the viewport. By default that just contains the "Scene Elements" section.
	static const FName CharacterMenuName("Persona.AnimViewportCharacterMenu");
	UToolMenu* ExtendableCharacterMenu = UToolMenus::Get()->ExtendMenu(CharacterMenuName);
	ExtendableCharacterMenu->AddDynamicSection(
		"PhysicsControlCharacterMenu", FNewToolMenuDelegate::CreateLambda([](UToolMenu* CharacterMenu)
			{
				TSharedPtr<FPhysicsControlAssetEditor> PhysicsControlAssetEditor =
					PhysicsControlAssetEditor::GetPhysicsControlAssetEditorFromToolContext(CharacterMenu->Context);
				if (PhysicsControlAssetEditor)
				{
					FToolMenuSection& Section = CharacterMenu->AddSection(
						"PhysicsAssetShowCommands",
						LOCTEXT("PhysicsShowCommands", "Physics Rendering"),
						FToolMenuInsert("AnimViewportSceneElements", EToolMenuInsertType::Before));
					Section.AddSubMenu(TEXT("MeshRenderModeSubMenu"), LOCTEXT("MeshRenderModeSubMenu", "Mesh"), FText::GetEmpty(),
						FNewToolMenuDelegate::CreateLambda([](UToolMenu* InSubMenu)
							{
								const FPhysicsControlAssetEditorCommands& Commands = FPhysicsControlAssetEditorCommands::Get();
								{
									FToolMenuSection& Section = InSubMenu->AddSection(
										"PhysicsControlAssetEditorRenderingMode",
										LOCTEXT("MeshRenderModeHeader", "Mesh Drawing (Edit)"));
									Section.AddMenuEntry(Commands.MeshRenderingMode_Solid);
									Section.AddMenuEntry(Commands.MeshRenderingMode_Wireframe);
									Section.AddMenuEntry(Commands.MeshRenderingMode_None);
								}

								{
									FToolMenuSection& Section = InSubMenu->AddSection(
										"PhysicsControlAssetEditorRenderingModeSim",
										LOCTEXT("MeshRenderModeSimHeader", "Mesh Drawing (Simulation)"));
									Section.AddMenuEntry(Commands.MeshRenderingMode_Simulation_Solid);
									Section.AddMenuEntry(Commands.MeshRenderingMode_Simulation_Wireframe);
									Section.AddMenuEntry(Commands.MeshRenderingMode_Simulation_None);
								}
							}));

					Section.AddSubMenu(TEXT("CollisionRenderModeSubMenu"),
						LOCTEXT("CollisionRenderModeSubMenu", "Bodies"), FText::GetEmpty(),
						FNewToolMenuDelegate::CreateLambda(
							[WeakPhysicsControlAssetEditor = PhysicsControlAssetEditor.ToWeakPtr()](UToolMenu* InSubMenu)
							{
								const FPhysicsControlAssetEditorCommands& Commands = FPhysicsControlAssetEditorCommands::Get();
								if (TSharedPtr<FPhysicsControlAssetEditor> Editor = WeakPhysicsControlAssetEditor.Pin())
								{
									FToolMenuSection& Section = InSubMenu->AddSection(
										"PhysicsControlAssetEditorCollisionRenderSettings",
										LOCTEXT("CollisionRenderSettingsHeader", "Body Drawing"));
									Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("CollisionOpacity"),
										Editor->MakeCollisionOpacityWidget(),
										LOCTEXT("CollisionOpacityLabel", "Collision Opacity")));
								}

								{
									FToolMenuSection& Section = InSubMenu->AddSection(
										"PhysicsControlAssetEditorCollisionMode",
										LOCTEXT("CollisionRenderModeHeader", "Body Drawing (Edit)"));
									Section.AddMenuEntry(Commands.CollisionRenderingMode_Solid);
									Section.AddMenuEntry(Commands.CollisionRenderingMode_Wireframe);
									Section.AddMenuEntry(Commands.CollisionRenderingMode_SolidWireframe);
									Section.AddMenuEntry(Commands.CollisionRenderingMode_None);
								}

								{
									FToolMenuSection& Section = InSubMenu->AddSection(
										"PhysicsControlAssetEditorCollisionModeSim",
										LOCTEXT("CollisionRenderModeSimHeader", "Body Drawing (Simulation)"));
									Section.AddMenuEntry(Commands.CollisionRenderingMode_Simulation_Solid);
									Section.AddMenuEntry(Commands.CollisionRenderingMode_Simulation_Wireframe);
									Section.AddMenuEntry(Commands.CollisionRenderingMode_Simulation_SolidWireframe);
									Section.AddMenuEntry(Commands.CollisionRenderingMode_Simulation_None);
								}
							}));

					Section.AddSubMenu(TEXT("ConstraintConstraintModeSubMenu"),
						LOCTEXT("ConstraintConstraintModeSubMenu", "Constraints"), FText::GetEmpty(),
						FNewToolMenuDelegate::CreateLambda(
							[WeakPhysicsControlAssetEditor = PhysicsControlAssetEditor.ToWeakPtr()](UToolMenu* InSubMenu)
							{
								const FPhysicsControlAssetEditorCommands& Commands = FPhysicsControlAssetEditorCommands::Get();
								if (TSharedPtr<FPhysicsControlAssetEditor> Editor = WeakPhysicsControlAssetEditor.Pin())
								{
									FToolMenuSection& Section = InSubMenu->AddSection(
										"PhysicsControlAssetEditorConstraints",
										LOCTEXT("ConstraintHeader", "Constraints"));
									Section.AddMenuEntry(Commands.DrawViolatedLimits);
									Section.AddEntry(FToolMenuEntry::InitWidget(TEXT(
										"ConstraintScale"), Editor->MakeConstraintScaleWidget(),
										LOCTEXT("ConstraintScaleLabel", "Constraint Scale")));
								}
								{
									FToolMenuSection& Section = InSubMenu->AddSection(
										"PhysicsControlAssetEditorConstraintMode",
										LOCTEXT("ConstraintRenderModeHeader", "Constraint Drawing (Edit)"));
									Section.AddMenuEntry(Commands.ConstraintRenderingMode_None);
									Section.AddMenuEntry(Commands.ConstraintRenderingMode_AllPositions);
									Section.AddMenuEntry(Commands.ConstraintRenderingMode_AllLimits);
								}

								{
									FToolMenuSection& Section = InSubMenu->AddSection(
										"PhysicsControlAssetEditorConstraintModeSim",
										LOCTEXT("ConstraintRenderModeSimHeader", "Constraint Drawing (Simulation)"));
									Section.AddMenuEntry(Commands.ConstraintRenderingMode_Simulation_None);
									Section.AddMenuEntry(Commands.ConstraintRenderingMode_Simulation_AllPositions);
									Section.AddMenuEntry(Commands.ConstraintRenderingMode_Simulation_AllLimits);
								}
							}));
				}
			}));

	// This extends the menu "Physics" in the viewport. This is empty by default.
	static const FName PhysicsMenuName("Persona.AnimViewportPhysicsMenu");
	UToolMenu* ExtendablePhysicsMenu = UToolMenus::Get()->ExtendMenu(PhysicsMenuName);
	ExtendablePhysicsMenu->AddDynamicSection(
		"AnimViewportPhysicsControlMenu", FNewToolMenuDelegate::CreateLambda([](UToolMenu* PhysicsMenu)
			{
				TSharedPtr<FPhysicsControlAssetEditor> PhysicsControlAssetEditor =
					PhysicsControlAssetEditor::GetPhysicsControlAssetEditorFromToolContext(PhysicsMenu->Context);
				if (PhysicsControlAssetEditor)
				{
					FToolMenuSection& Section = PhysicsMenu->AddSection(
						"AnimViewportPhysicsMenu", LOCTEXT("ViewMenu_AnimViewportPhysicsMenu", "Physics Menu"));

					FPropertyEditorModule& PropertyEditorModule =
						FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

					FDetailsViewArgs DetailsViewArgs;
					DetailsViewArgs.bAllowSearch = false;
					DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

					// I'd prefer to do this using SetIsPropertyVisibleDelegate but for some reason
					// any delegate registered gets cleared by the time the visibilty check call
					// gets made.
					DetailsViewArgs.ShouldForceHideProperty.BindLambda([](const TSharedRef<IPropertyHandle>& PropertyNode)->bool
						{
							const FName PropertyName = PropertyNode->GetProperty()->GetFName();
							if (PropertyName == GET_MEMBER_NAME_CHECKED(UPhysicsAssetEditorOptions, PhysicsBlend) ||
								PropertyName == GET_MEMBER_NAME_CHECKED(UPhysicsAssetEditorOptions, bRenderOnlySelectedConstraints) ||
								PropertyName == GET_MEMBER_NAME_CHECKED(UPhysicsAssetEditorOptions, bUpdateJointsFromAnimation) ||
								PropertyName == GET_MEMBER_NAME_CHECKED(UPhysicsAssetEditorOptions, PhysicsUpdateMode))
							{
								return true;
							}
							return false;
						});


					TSharedPtr<IDetailsView> OptionsDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
					OptionsDetailsView->SetObject(PhysicsControlAssetEditor->GetEditorData()->EditorOptions);
					OptionsDetailsView->OnFinishedChangingProperties().AddLambda(
						[WeakPhysicsControlAssetEditor = PhysicsControlAssetEditor.ToWeakPtr()](const FPropertyChangedEvent& InEvent)
						{
							if (TSharedPtr<FPhysicsControlAssetEditor> Editor = WeakPhysicsControlAssetEditor.Pin())
							{
								Editor->GetEditorData()->EditorOptions->SaveConfig();
							}
						});

					Section.AddEntry(FToolMenuEntry::InitWidget("PhysicsEditorOptions", OptionsDetailsView.ToSharedRef(), FText()));
				}
			}));
}

//======================================================================================================================
void FPhysicsControlAssetEditor::OnCompile()
{
	EditorData->PhysicsControlAsset->Compile();
}

//======================================================================================================================
bool FPhysicsControlAssetEditor::IsCompilationNeeded()
{
	return EditorData->PhysicsControlAsset->IsCompilationNeeded();
}

//======================================================================================================================
void FPhysicsControlAssetEditor::OnToggleSimulation()
{
	static const float PrevMaxFPS = GEngine->GetMaxFPS();

	if (!EditorData->bRunningSimulation)
	{
		GEngine->SetMaxFPS((float)EditorData->EditorOptions->MaxFPS);
	}
	else
	{
		GEngine->SetMaxFPS(PrevMaxFPS);
	}

	EditorData->ToggleSimulation();
}

//======================================================================================================================
void FPhysicsControlAssetEditor::RecreateControlsAndModifiers()
{
	EditorData->RecreateControlsAndModifiers();
}

//======================================================================================================================
void FPhysicsControlAssetEditor::OnToggleSimulationNoGravity()
{
	EditorData->bNoGravitySimulation = !EditorData->bNoGravitySimulation;
}

//======================================================================================================================
bool FPhysicsControlAssetEditor::IsNoGravitySimulationEnabled() const
{
	return EditorData->bNoGravitySimulation;
}

//======================================================================================================================
void FPhysicsControlAssetEditor::OnToggleSimulationFloorCollision()
{
	if (EditorData && EditorData->EditorOptions)
	{
		EditorData->EditorOptions->bSimulationFloorCollisionEnabled = !EditorData->EditorOptions->bSimulationFloorCollisionEnabled;

		// Update collision for floor
		if (PersonaToolkit)
		{
			TSharedRef<IPersonaPreviewScene> PersonaPreviewScene = PersonaToolkit->GetPreviewScene();

			if (UStaticMeshComponent* FloorMeshComponent = 
				const_cast<UStaticMeshComponent*>(PersonaPreviewScene->GetFloorMeshComponent()))
			{
				if (EditorData->EditorOptions->bSimulationFloorCollisionEnabled)
				{
					FloorMeshComponent->SetCollisionEnabled(ECollisionEnabled::QueryAndPhysics);
				}
				else
				{
					FloorMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
				}
			}
		}
	}
}

//======================================================================================================================
bool FPhysicsControlAssetEditor::IsSimulationFloorCollisionEnabled() const
{
	return EditorData && EditorData->EditorOptions && EditorData->EditorOptions->bSimulationFloorCollisionEnabled;
}

//======================================================================================================================
void FPhysicsControlAssetEditor::OnMeshRenderingMode(EPhysicsAssetEditorMeshViewMode Mode, bool bSimulation)
{
	if (bSimulation)
	{
		EditorData->EditorOptions->SimulationMeshViewMode = Mode;
	}
	else
	{
		EditorData->EditorOptions->MeshViewMode = Mode;
	}

	EditorData->EditorOptions->SaveConfig();

	// Changing the mesh rendering mode requires the skeletal mesh component to change its render
	// state, which is an operation which is deferred until after render. Hence we need to trigger
	// another viewport refresh on the following frame.
	RefreshPreviewViewport();
}

//======================================================================================================================
bool FPhysicsControlAssetEditor::IsMeshRenderingMode(EPhysicsAssetEditorMeshViewMode Mode, bool bSimulation) const
{
	return Mode == EditorData->GetCurrentMeshViewMode(bSimulation);
}

//======================================================================================================================
void FPhysicsControlAssetEditor::OnCollisionRenderingMode(EPhysicsAssetEditorCollisionViewMode Mode, bool bSimulation)
{
	if (bSimulation)
	{
		EditorData->EditorOptions->SimulationCollisionViewMode = Mode;
	}
	else
	{
		EditorData->EditorOptions->CollisionViewMode = Mode;
	}
	EditorData->EditorOptions->SaveConfig();
	RefreshPreviewViewport();
}

//======================================================================================================================
bool FPhysicsControlAssetEditor::IsCollisionRenderingMode(
	EPhysicsAssetEditorCollisionViewMode Mode, bool bSimulation) const
{
	return Mode == EditorData->GetCurrentCollisionViewMode(bSimulation);
}

//======================================================================================================================
void FPhysicsControlAssetEditor::OnConstraintRenderingMode(EPhysicsAssetEditorConstraintViewMode Mode, bool bSimulation)
{
	if (bSimulation)
	{
		EditorData->EditorOptions->SimulationConstraintViewMode = Mode;
	}
	else
	{
		EditorData->EditorOptions->ConstraintViewMode = Mode;
	}

	EditorData->EditorOptions->SaveConfig();

	RefreshPreviewViewport();
}

//======================================================================================================================
bool FPhysicsControlAssetEditor::IsConstraintRenderingMode(
	EPhysicsAssetEditorConstraintViewMode Mode, bool bSimulation) const
{
	return Mode == EditorData->GetCurrentConstraintViewMode(bSimulation);
}

//======================================================================================================================
void FPhysicsControlAssetEditor::ToggleDrawViolatedLimits()
{
	EditorData->EditorOptions->bDrawViolatedLimits = !EditorData->EditorOptions->bDrawViolatedLimits;
	EditorData->EditorOptions->SaveConfig();
	RefreshPreviewViewport();
}

//======================================================================================================================
bool FPhysicsControlAssetEditor::IsDrawingViolatedLimits() const
{
	return EditorData->EditorOptions->bDrawViolatedLimits;
}

//======================================================================================================================
bool FPhysicsControlAssetEditor::IsRunningSimulation() const
{
	return EditorData->bRunningSimulation;
}

//======================================================================================================================
bool FPhysicsControlAssetEditor::IsNotRunningSimulation() const
{
	return !EditorData->bRunningSimulation;
}

//======================================================================================================================
void FPhysicsControlAssetEditor::BindCommands()
{
	const FPhysicsControlAssetEditorCommands& Commands = FPhysicsControlAssetEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.Compile,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnCompile),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsCompilationNeeded));

	ToolkitCommands->MapAction(
		Commands.Simulation,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnToggleSimulation),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsRunningSimulation));

	ToolkitCommands->MapAction(
		Commands.SimulationNoGravity,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnToggleSimulationNoGravity),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsNoGravitySimulationEnabled));

	ToolkitCommands->MapAction(
		Commands.SimulationFloorCollision,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnToggleSimulationFloorCollision),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsSimulationFloorCollisionEnabled));

	// Viewport commands
	ViewportCommandList = MakeShared<FUICommandList_Pinnable>();

	ViewportCommandList->BeginGroup(TEXT("MeshRenderingMode"));

	ViewportCommandList->MapAction(
		Commands.MeshRenderingMode_Solid,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Solid, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Solid, false));

	ViewportCommandList->MapAction(
		Commands.MeshRenderingMode_Wireframe,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Wireframe, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Wireframe, false));

	ViewportCommandList->MapAction(
		Commands.MeshRenderingMode_None,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::None, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::None, false));

	ViewportCommandList->EndGroup();

	ViewportCommandList->BeginGroup(TEXT("CollisionRenderingMode"));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_Solid,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Solid, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Solid, false));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_Wireframe,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Wireframe, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Wireframe, false));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_SolidWireframe,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::SolidWireframe, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::SolidWireframe, false));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_None,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::None, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::None, false));

	ViewportCommandList->EndGroup();

	ViewportCommandList->BeginGroup(TEXT("ConstraintRenderingMode"));

	ViewportCommandList->MapAction(
		Commands.ConstraintRenderingMode_None,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::None, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::None, false));

	ViewportCommandList->MapAction(
		Commands.ConstraintRenderingMode_AllPositions,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllPositions, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllPositions, false));

	ViewportCommandList->MapAction(
		Commands.ConstraintRenderingMode_AllLimits,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllLimits, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllLimits, false));

	ViewportCommandList->EndGroup();

	ViewportCommandList->BeginGroup(TEXT("MeshRenderingMode_Simulation"));

	ViewportCommandList->MapAction(
		Commands.MeshRenderingMode_Simulation_Solid,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Solid, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Solid, true));

	ViewportCommandList->MapAction(
		Commands.MeshRenderingMode_Simulation_Wireframe,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Wireframe, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::Wireframe, true));

	ViewportCommandList->MapAction(
		Commands.MeshRenderingMode_Simulation_None,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::None, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsMeshRenderingMode, EPhysicsAssetEditorMeshViewMode::None, true));

	ViewportCommandList->EndGroup();

	ViewportCommandList->BeginGroup(TEXT("CollisionRenderingMode_Simulation"));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_Simulation_Solid,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Solid, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Solid, true));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_Simulation_Wireframe,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Wireframe, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::Wireframe, true));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_Simulation_SolidWireframe,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::SolidWireframe, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::SolidWireframe, true));

	ViewportCommandList->MapAction(
		Commands.CollisionRenderingMode_Simulation_None,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::None, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsCollisionRenderingMode, EPhysicsAssetEditorCollisionViewMode::None, true));

	ViewportCommandList->EndGroup();

	ViewportCommandList->BeginGroup(TEXT("ConstraintRenderingMode_Simulation"));

	ViewportCommandList->MapAction(
		Commands.ConstraintRenderingMode_Simulation_None,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::None, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::None, true));

	ViewportCommandList->MapAction(
		Commands.ConstraintRenderingMode_Simulation_AllPositions,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllPositions, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllPositions, true));

	ViewportCommandList->MapAction(
		Commands.ConstraintRenderingMode_Simulation_AllLimits,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::OnConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllLimits, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsConstraintRenderingMode, EPhysicsAssetEditorConstraintViewMode::AllLimits, true));

	ViewportCommandList->EndGroup();

	ViewportCommandList->MapAction(
		Commands.DrawViolatedLimits,
		FExecuteAction::CreateSP(this, &FPhysicsControlAssetEditor::ToggleDrawViolatedLimits),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FPhysicsControlAssetEditor::IsDrawingViolatedLimits));

}

//======================================================================================================================
void FPhysicsControlAssetEditor::BuildMenuWidgetBody(FMenuBuilder& InMenuBuilder)
{
	// TODO Danny
}

//======================================================================================================================
// Danny TODO note that these appear (on right-click) but are not yet functional
void FPhysicsControlAssetEditor::BuildMenuWidgetSelection(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.PushCommandList(GetToolkitCommands());
	{
		const FPhysicsControlAssetEditorCommands& Commands = FPhysicsControlAssetEditorCommands::Get();

		InMenuBuilder.BeginSection("Selection", LOCTEXT("Selection", "Selection"));
		InMenuBuilder.AddMenuEntry(Commands.SelectAllBodies);
		InMenuBuilder.AddMenuEntry(Commands.SelectSimulatedBodies);
		InMenuBuilder.AddMenuEntry(Commands.SelectKinematicBodies);
		InMenuBuilder.EndSection();
	}
	InMenuBuilder.PopCommandList();
}


//======================================================================================================================
// Danny TODO selection needs to be implemented/handled/made useful
void FPhysicsControlAssetEditor::HandleSelectionChanged(
	const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo)
{
	if (!bSelecting)
	{
		TGuardValue<bool> RecursionGuard(bSelecting, true);

		// Always set the details customization object, regardless of selection type
		// We do this because the tree may have been rebuilt and objects invalidated
		TArray<UObject*> Objects;
		Algo::TransformIf(InSelectedItems, Objects, 
			[](const TSharedPtr<ISkeletonTreeItem>& InItem) {
			return InItem->GetObject() != nullptr; },
			[](const TSharedPtr<ISkeletonTreeItem>& InItem) { 
				return InItem->GetObject(); }
		);

		if (DetailsView.IsValid())
		{
			DetailsView->SetObjects(Objects);
		}
	}
}

//======================================================================================================================
void FPhysicsControlAssetEditor::HandleGetFilterLabel(TArray<FText>& InOutItems) const
{
	if (!SkeletonTreeBuilder.IsValid())
	{
		return;
	}

	if (SkeletonTreeBuilder->bShowBodies)
	{
		InOutItems.Add(LOCTEXT("BodiesFilterLabel", "Bodies"));
	}

	if (SkeletonTreeBuilder->bShowPrimitives)
	{
		InOutItems.Add(LOCTEXT("PrimitivesFilterLabel", "Primitives"));
	}
}

//======================================================================================================================
// Danny TODO
void FPhysicsControlAssetEditor::HandleExtendFilterMenu(FMenuBuilder& InMenuBuilder)
{
}

//======================================================================================================================
// Danny TODO
void FPhysicsControlAssetEditor::HandleExtendContextMenu(FMenuBuilder& InMenuBuilder)
{
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedItems = SkeletonTree->GetSelectedItems();
	FSkeletonTreeSelection Selection(SelectedItems);

	TArray<TSharedPtr<FSkeletonTreePhysicsControlBodyItem>> SelectedBodies = 
		Selection.GetSelectedItems<FSkeletonTreePhysicsControlBodyItem>();
	TArray<TSharedPtr<FSkeletonTreePhysicsControlShapeItem>> SelectedShapes = 
		Selection.GetSelectedItems<FSkeletonTreePhysicsControlShapeItem>();
	TArray<TSharedPtr<ISkeletonTreeItem>> SelectedBones = Selection.GetSelectedItemsByTypeId("FSkeletonTreeBoneItem");
}


//======================================================================================================================
void FPhysicsControlAssetEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(
		LOCTEXT("WorkspaceMenu_PhysicsControlAssetEditor", "PhysicsControlAssetEditor"));
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
}

//======================================================================================================================
void FPhysicsControlAssetEditor::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
}

//======================================================================================================================
FName FPhysicsControlAssetEditor::GetToolkitFName() const
{
	return FName("PhysicsControlAssetEditor");
}

//======================================================================================================================
FText FPhysicsControlAssetEditor::GetBaseToolkitName() const
{
	return LOCTEXT("PhysicsControlAssetEditorAppLabel", "Physics Control Asset Editor");
}

//======================================================================================================================
FText FPhysicsControlAssetEditor::GetToolkitName() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("AssetName"), FText::FromString(EditorData->PhysicsControlAsset->GetName()));
	return FText::Format(LOCTEXT("PhysicsControlAssetEditorName", "{AssetName}"), Args);
}

//======================================================================================================================
FLinearColor FPhysicsControlAssetEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

//======================================================================================================================
FString FPhysicsControlAssetEditor::GetWorldCentricTabPrefix() const
{
	return TEXT("PhysicsControlAssetEditor");
}

//======================================================================================================================
void FPhysicsControlAssetEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	EditorData->AddReferencedObjects(Collector);
}

//======================================================================================================================
void FPhysicsControlAssetEditor::Tick(float DeltaTime)
{
	GetPersonaToolkit()->GetPreviewScene()->InvalidateViews();
}

//======================================================================================================================
TStatId FPhysicsControlAssetEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FPhysicsControlAssetEditor, STATGROUP_Tickables);
}

//======================================================================================================================
TSharedRef<IPersonaToolkit> FPhysicsControlAssetEditor::GetPersonaToolkit() const
{
	return PersonaToolkit.ToSharedRef();
}

//======================================================================================================================
// For inspiration here, see:
// - PhysicsControlAssetEditor
// - MLDeformerEditorToolkit
// - IKRigToolkit
// though note that they all do things differently!
void FPhysicsControlAssetEditor::HandlePreviewSceneCreated(
	const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene)
{
	EditorData->PreviewScene = InPersonaPreviewScene;

	AAnimationEditorPreviewActor* Actor = InPersonaPreviewScene->GetWorld()->SpawnActor<AAnimationEditorPreviewActor>(
		AAnimationEditorPreviewActor::StaticClass(), FTransform::Identity);
	Actor->SetFlags(RF_Transient);
	InPersonaPreviewScene->SetActor(Actor);

	// Create the preview skeletal mesh component
	EditorData->EditorSkelComp = NewObject<UPhysicsControlAssetEditorSkeletalMeshComponent>(Actor);
	EditorData->EditorSkelComp->SetCollisionEnabled(ECollisionEnabled::QueryAndPhysics);
	EditorData->EditorSkelComp->EditorData = EditorData;
	EditorData->EditorSkelComp->SetSkeletalMesh(EditorData->PhysicsControlAsset->GetPreviewMesh());
	EditorData->EditorSkelComp->SetPhysicsAsset(EditorData->PhysicsControlAsset->GetPhysicsAsset(), true);
	EditorData->EditorSkelComp->SetDisablePostProcessBlueprint(true);
	EditorData->EditorSkelComp->Stop();

	EditorData->EditorSkelComp->bSelectable = false;

	// set root component, so we can attach to it. 
	Actor->SetRootComponent(EditorData->EditorSkelComp);

	// Set the skeletal mesh on the component, using the asset. Note that this will change if/when
	// the asset doesn't hold a mesh.
	USkeletalMesh* Mesh = EditorData->PhysicsControlAsset->GetPreviewMesh();
	EditorData->EditorSkelComp->SetSkeletalMesh(Mesh);

	// apply mesh to the preview scene
	InPersonaPreviewScene->SetPreviewMeshComponent(EditorData->EditorSkelComp);
	InPersonaPreviewScene->SetAllowMeshHitProxies(false);
	InPersonaPreviewScene->SetAdditionalMeshesSelectable(false);
	InPersonaPreviewScene->SetPreviewMesh(Mesh);
	InPersonaPreviewScene->AddComponent(EditorData->EditorSkelComp, FTransform::Identity);

	EditorData->PhysicsControlComponent = NewObject<UPhysicsControlComponent>(Actor);
	InPersonaPreviewScene->AddComponent(EditorData->PhysicsControlComponent, FTransform::Identity);

	// Register handle component
	EditorData->MouseHandle->RegisterComponentWithWorld(InPersonaPreviewScene->GetWorld());
	EditorData->EnableSimulation(false);
}

//======================================================================================================================
void FPhysicsControlAssetEditor::HandleViewportCreated(const TSharedRef<IPersonaViewport>& InPersonaViewport)
{
	PersonaViewport = InPersonaViewport;
	InPersonaViewport.Get().GetPinnedCommandList()->BindCommandList(ViewportCommandList.ToSharedRef());
	InPersonaViewport.Get().GetPinnedCommandList()->RegisterCustomWidget(
		IPinnedCommandList::FOnGenerateCustomWidget::CreateSP(
		this, &FPhysicsControlAssetEditor::MakeConstraintScaleWidget), 
		TEXT("ConstraintScaleWidget"), LOCTEXT("ConstraintScaleLabel", "Constraint Scale"));
	InPersonaViewport.Get().GetPinnedCommandList()->RegisterCustomWidget(
		IPinnedCommandList::FOnGenerateCustomWidget::CreateSP(
		this, &FPhysicsControlAssetEditor::MakeCollisionOpacityWidget), 
		TEXT("CollisionOpacityWidget"), LOCTEXT("CollisionOpacityLabel", "Collision Opacity"));
}

//======================================================================================================================
void FPhysicsControlAssetEditor::ShowEmptyDetails() const
{
	DetailsView->SetObject(EditorData->PhysicsControlAsset);
}

//======================================================================================================================
void FPhysicsControlAssetEditor::HandleDetailsCreated(const TSharedRef<class IDetailsView>& InDetailsView)
{
	DetailsView = InDetailsView;
	DetailsView->OnFinishedChangingProperties().AddSP(
		this, &FPhysicsControlAssetEditor::OnFinishedChangingDetails);
	ShowEmptyDetails();
}

//======================================================================================================================
void FPhysicsControlAssetEditor::OnFinishedChangingDetails(const FPropertyChangedEvent& PropertyChangedEvent)
{
	const bool bPreviewMeshChanged = 
		PropertyChangedEvent.GetPropertyName() == UPhysicsControlAsset::GetPreviewMeshPropertyName();
	if (bPreviewMeshChanged)
	{
		USkeletalMesh* Mesh = EditorData->PhysicsControlAsset->GetPreviewMesh();
		EditorData->EditorSkelComp->SetSkeletalMesh(Mesh);
		EditorData->CachePreviewMesh();
	}
}

//======================================================================================================================
void FPhysicsControlAssetEditor::RefreshHierarchyTree()
{
	if (SkeletonTree.IsValid())
	{
		SkeletonTree->Refresh();
	}
}

//======================================================================================================================
void FPhysicsControlAssetEditor::RefreshPreviewViewport()
{
	if (PersonaToolkit.IsValid())
	{
		PersonaToolkit->GetPreviewScene()->InvalidateViews();
	}
}

//======================================================================================================================
void FPhysicsControlAssetEditor::InvokeControlProfile(FName ProfileName)
{
	// Danny TODO handle the RBWC simulation case
	UPhysicsControlComponent* PCC = GetEditorData()->PhysicsControlComponent.Get();
	if (PCC)
	{
		PCC->InvokeControlProfile(ProfileName);
	}
	PreviouslyInvokedControlProfile = ProfileName;
}

//======================================================================================================================
void FPhysicsControlAssetEditor::ReinvokeControlProfile()
{
	InvokeControlProfile(PreviouslyInvokedControlProfile);
}

#undef LOCTEXT_NAMESPACE

