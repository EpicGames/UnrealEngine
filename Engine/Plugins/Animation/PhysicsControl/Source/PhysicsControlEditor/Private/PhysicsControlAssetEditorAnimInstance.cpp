// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAssetEditorAnimInstance.h"
#include "PhysicsControlAssetEditorAnimInstanceProxy.h"

/////////////////////////////////////////////////////
// UPhysicsControlAssetEditorAnimInstance
/////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsControlAssetEditorAnimInstance)

UPhysicsControlAssetEditorAnimInstance::UPhysicsControlAssetEditorAnimInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUseMultiThreadedAnimationUpdate = true;
}

FAnimInstanceProxy* UPhysicsControlAssetEditorAnimInstance::CreateAnimInstanceProxy()
{
	return new FPhysicsControlAssetEditorAnimInstanceProxy(this);
}

void UPhysicsControlAssetEditorAnimInstance::Grab(FName InBoneName, const FVector& Location, const FRotator& Rotation, bool bRotationConstrained)
{
	FPhysicsControlAssetEditorAnimInstanceProxy& Proxy = GetProxyOnGameThread<FPhysicsControlAssetEditorAnimInstanceProxy>();
	Proxy.Grab(InBoneName, Location, Rotation, bRotationConstrained);
}

void UPhysicsControlAssetEditorAnimInstance::Ungrab()
{
	FPhysicsControlAssetEditorAnimInstanceProxy& Proxy = GetProxyOnGameThread<FPhysicsControlAssetEditorAnimInstanceProxy>();
	Proxy.Ungrab();
}

void UPhysicsControlAssetEditorAnimInstance::UpdateHandleTransform(const FTransform& NewTransform)
{
	FPhysicsControlAssetEditorAnimInstanceProxy& Proxy = GetProxyOnGameThread<FPhysicsControlAssetEditorAnimInstanceProxy>();
	Proxy.UpdateHandleTransform(NewTransform);
}

void UPhysicsControlAssetEditorAnimInstance::UpdateDriveSettings(bool bLinearSoft, float LinearStiffness, float LinearDamping)
{
	FPhysicsControlAssetEditorAnimInstanceProxy& Proxy = GetProxyOnGameThread<FPhysicsControlAssetEditorAnimInstanceProxy>();
	Proxy.UpdateDriveSettings(bLinearSoft, LinearStiffness, LinearDamping);
}

void UPhysicsControlAssetEditorAnimInstance::CreateSimulationFloor(FBodyInstance* FloorBodyInstance, const FTransform& Transform)
{
	FPhysicsControlAssetEditorAnimInstanceProxy& Proxy = GetProxyOnGameThread<FPhysicsControlAssetEditorAnimInstanceProxy>();
	Proxy.CreateSimulationFloor(FloorBodyInstance, Transform);
}

