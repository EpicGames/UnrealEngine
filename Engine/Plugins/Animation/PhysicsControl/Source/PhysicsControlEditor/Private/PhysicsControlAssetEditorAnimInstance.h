// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * Used by Preview in PhysicsControlAssetEditor
 */

#pragma once
#include "AnimPreviewInstance.h"
#include "PhysicsControlAssetEditorAnimInstance.generated.h"

class UAnimSequence;
struct FBodyInstance;

UCLASS(transient, NotBlueprintable)
class UPhysicsControlAssetEditorAnimInstance : public UAnimPreviewInstance
{
	GENERATED_UCLASS_BODY()

	void Grab(FName InBoneName, const FVector& Location, const FRotator& Rotation, bool bRotationConstrained);
	void Ungrab();
	void UpdateHandleTransform(const FTransform& NewTransform);
	void UpdateDriveSettings(bool bLinearSoft, float LinearStiffness, float LinearDamping);
	void CreateSimulationFloor(FBodyInstance* FloorBodyInstance, const FTransform& Transform);

protected:
	virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;
};
