// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAssetEditorCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "PhysicsControlAssetEditorCommands"

void FPhysicsControlAssetEditorCommands::RegisterCommands()
{
	UI_COMMAND(Compile, "Compile", "Compiles all the data from this and any parent asset into runtime form", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(Simulation, "Simulate", "Previews Physics Simulation", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SimulationNoGravity, "No Gravity Simulation", "Run Physics Simulation without gravity.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SimulationFloorCollision, "Enable Floor Collisions", "Run Physics Simulation with collisions between physics bodies and the floor enabled.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(SelectAllBodies, "Select All Bodies", "Select All Bodies", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::A));
	UI_COMMAND(SelectSimulatedBodies, "Select Simulated Bodies", "Select Simulated Bodies", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::J));
	UI_COMMAND(SelectKinematicBodies, "Select Kinematic Bodies", "Select Kinematic Bodies", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::K));

	UI_COMMAND(MeshRenderingMode_Solid, "Solid", "Solid Mesh Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(MeshRenderingMode_Wireframe, "Wireframe", "Wireframe Mesh Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(MeshRenderingMode_None, "None", "No Mesh Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_Solid, "Solid", "Solid Collision Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_Wireframe, "Wireframe", "Wireframe Collision Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_SolidWireframe, "Solid + Wireframe", "Solid + Wireframe Collision Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_None, "None", "No Collision Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ConstraintRenderingMode_None, "None", "No Constraint Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ConstraintRenderingMode_AllPositions, "All Positions", "All Positions Constraint Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ConstraintRenderingMode_AllLimits, "All Limits", "All Limits Constraint Rendering Mode (Edit)", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(MeshRenderingMode_Simulation_Solid, "Solid", "Solid Mesh Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(MeshRenderingMode_Simulation_Wireframe, "Wireframe", "Wireframe Mesh Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(MeshRenderingMode_Simulation_None, "None", "No Mesh Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_Simulation_Solid, "Solid", "Solid Collision Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_Simulation_Wireframe, "Wireframe", "Wireframe Collision Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_Simulation_SolidWireframe, "Solid + Wireframe", "Solid + Wireframe Collision Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(CollisionRenderingMode_Simulation_None, "None", "No Collision Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ConstraintRenderingMode_Simulation_None, "None", "No Constraint Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ConstraintRenderingMode_Simulation_AllPositions, "All Positions", "All Positions Constraint Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ConstraintRenderingMode_Simulation_AllLimits, "All Limits", "All Limits Constraint Rendering Mode (Simulation)", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(DrawViolatedLimits, "Draw Violated Limits", "Draw Violated Limits", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
