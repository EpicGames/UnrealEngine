// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

class FPhysicsControlAssetEditorCommands : public TCommands<FPhysicsControlAssetEditorCommands>
{
public:
	/** Constructor */
	FPhysicsControlAssetEditorCommands()
		: TCommands<FPhysicsControlAssetEditorCommands>(
			"PhysicsControlAssetEditor", 
			NSLOCTEXT("Contexts", "PhysicsControlAssetEditor", "PhysicsControlAssetEditor"), 
			NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}
	
	/** See tooltips in cpp for documentation */
	TSharedPtr<FUICommandInfo> Compile;
	TSharedPtr<FUICommandInfo> Simulation;
	TSharedPtr<FUICommandInfo> SimulationNoGravity;
	TSharedPtr<FUICommandInfo> SimulationFloorCollision;

	/** Selection commands */
	TSharedPtr<FUICommandInfo> SelectAllBodies;
	TSharedPtr<FUICommandInfo> SelectSimulatedBodies;
	TSharedPtr<FUICommandInfo> SelectKinematicBodies;

	/** Rendering commands */
	TSharedPtr<FUICommandInfo> MeshRenderingMode_Solid;
	TSharedPtr<FUICommandInfo> MeshRenderingMode_Wireframe;
	TSharedPtr<FUICommandInfo> MeshRenderingMode_None;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_Solid;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_Wireframe;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_SolidWireframe;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_None;
	TSharedPtr<FUICommandInfo> ConstraintRenderingMode_None;
	TSharedPtr<FUICommandInfo> ConstraintRenderingMode_AllPositions;
	TSharedPtr<FUICommandInfo> ConstraintRenderingMode_AllLimits;

	TSharedPtr<FUICommandInfo> MeshRenderingMode_Simulation_Solid;
	TSharedPtr<FUICommandInfo> MeshRenderingMode_Simulation_Wireframe;
	TSharedPtr<FUICommandInfo> MeshRenderingMode_Simulation_None;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_Simulation_Solid;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_Simulation_Wireframe;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_Simulation_SolidWireframe;
	TSharedPtr<FUICommandInfo> CollisionRenderingMode_Simulation_None;
	TSharedPtr<FUICommandInfo> ConstraintRenderingMode_Simulation_None;
	TSharedPtr<FUICommandInfo> ConstraintRenderingMode_Simulation_AllPositions;
	TSharedPtr<FUICommandInfo> ConstraintRenderingMode_Simulation_AllLimits;

	TSharedPtr<FUICommandInfo> DrawViolatedLimits;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};
