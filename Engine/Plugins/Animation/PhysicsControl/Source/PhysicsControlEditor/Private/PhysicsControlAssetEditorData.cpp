// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAssetEditorData.h"
#include "PhysicsControlAsset.h"
#include "PhysicsControlAssetEditorSkeletalMeshComponent.h"
#include "PhysicsControlAssetEditorPhysicsHandleComponent.h"
#include "PhysicsControlComponent.h"

#include "Components/StaticMeshComponent.h"
#include "IPersonaPreviewScene.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "AnimPreviewInstance.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/MessageDialog.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "PhysicsControlAssetEditorData"

//======================================================================================================================
FPhysicsControlAssetEditorData::FPhysicsControlAssetEditorData()
	: bSuspendSelectionBroadcast(false)
	, InsideSelChange(0)
{
	bRunningSimulation = false;
	bNoGravitySimulation = false;

	bManipulating = false;

	// Construct mouse handle
	MouseHandle = NewObject<UPhysicsControlAssetEditorPhysicsHandleComponent>();

	// Construct sim options.
	EditorOptions = NewObject<UPhysicsAssetEditorOptions>(
		GetTransientPackage(), 
		MakeUniqueObjectName(
			GetTransientPackage(), UPhysicsAssetEditorOptions::StaticClass(), 
			FName(TEXT("EditorOptions"))), RF_Transactional);
	check(EditorOptions);

	EditorOptions->LoadConfig();

	// Set some options that we don't want to have the user modify
	EditorOptions->bUpdateJointsFromAnimation = true;
	EditorOptions->PhysicsUpdateMode = EPhysicsTransformUpdateMode::ComponentTransformIsKinematic;
}

//======================================================================================================================
void FPhysicsControlAssetEditorData::Initialize(const TSharedRef<IPersonaPreviewScene>& InPreviewScene)
{
	PreviewScene = InPreviewScene;

	bRunningSimulation = false;
	bNoGravitySimulation = false;

	EditorSkelComp = nullptr;
	PhysicsControlComponent = nullptr;

	// Support undo/redo
	PhysicsControlAsset->SetFlags(RF_Transactional);
}

//======================================================================================================================
void FPhysicsControlAssetEditorData::CachePreviewMesh()
{
	// This loads it if necessary
	USkeletalMesh* PreviewMesh = PhysicsControlAsset->GetPreviewMesh();

	if (PreviewMesh == nullptr)
	{
		// Fall back to the default skeletal mesh in the EngineMeshes package.
		// This is statically loaded as the package is likely not fully loaded
		// (otherwise, it would have been found in the above iteration).
		PreviewMesh = (USkeletalMesh*)StaticLoadObject(
			USkeletalMesh::StaticClass(), NULL, 
			TEXT("/Engine/EngineMeshes/SkeletalCube.SkeletalCube"), NULL, LOAD_None, NULL);
		check(PreviewMesh);

		PhysicsControlAsset->SetPreviewMesh(PreviewMesh);

		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT(
			"Error_PhysicsControlAssetHasNoSkelMesh",
			"Warning: Physics Control Asset has no skeletal mesh assigned.\n"
			"This is likely to be because there is no valid Physics Asset. "
			"Fix this by assigning a Preview Physics Asset/Mesh in the Physics Control Asset.")
		);
	}
	else if (PreviewMesh->GetSkeleton() == nullptr)
	{
		// Fall back in the case of a deleted skeleton
		PreviewMesh = (USkeletalMesh*)StaticLoadObject(
			USkeletalMesh::StaticClass(), NULL, 
			TEXT("/Engine/EngineMeshes/SkeletalCube.SkeletalCube"), NULL, LOAD_None, NULL);
		check(PreviewMesh);

		PhysicsControlAsset->SetPreviewMesh(PreviewMesh);

		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT(
			"Error_PhysicsControlAssetHasNoSkelMeshSkeleton", 
			"Warning: Physics Control Asset has no skeletal mesh skeleton assigned.\n"
			"This is likely to be because there is no valid Physics Asset. "
			"Fix this by assigning a Preview Physics Asset/Mesh in the Physics Control Asset.")
		);
	}
}

//======================================================================================================================
void FPhysicsControlAssetEditorData::ToggleSimulation()
{
	if (!bManipulating)
	{
		EnableSimulation(!bRunningSimulation);
	}
}


//======================================================================================================================
void FPhysicsControlAssetEditorData::EnableSimulation(bool bEnableSimulation)
{
	// keep the EditorSkelComp animation asset if any set 
	UAnimationAsset* PreviewAnimationAsset = nullptr;
	if (EditorSkelComp->PreviewInstance)
	{
		PreviewAnimationAsset = EditorSkelComp->PreviewInstance->CurrentAsset;
	}

	UPhysicsAsset* PA = PhysicsControlAsset->GetPhysicsAsset();

	if (bEnableSimulation && PA)
	{
		// in Chaos, we have to manipulate the RBAN node in the Anim Instance (at least until we get
		// SkelMeshComp implemented)
		const bool bUseRBANSolver = (PA->SolverType == EPhysicsAssetSolverType::RBAN);
		MouseHandle->SetAnimInstanceMode(bUseRBANSolver);

		if (!bUseRBANSolver)
		{
			// We should not already have an instance (destroyed when stopping sim).
			EditorSkelComp->SetCollisionEnabled(ECollisionEnabled::QueryAndPhysics);
			EditorSkelComp->SetSimulatePhysics(true);
			EditorSkelComp->ResetAllBodiesSimulatePhysics();
			EditorSkelComp->SetPhysicsBlendWeight(EditorOptions->PhysicsBlend);
			// Make it start simulating
			EditorSkelComp->WakeAllRigidBodies();

			PhysicsControlComponent->PhysicsControlAsset = PhysicsControlAsset;
			PhysicsControlComponent->CreateControlsAndBodyModifiersFromPhysicsControlAsset(
				EditorSkelComp, nullptr, FName());
		}
		else
		{
			// Enable the PreviewInstance (containing the AnimNode_RigidBody)
			EditorSkelComp->SetAnimationMode(EAnimationMode::AnimationCustomMode);
			EditorSkelComp->InitAnim(true);

			// Disable main solver physics
			EditorSkelComp->SetAllBodiesSimulatePhysics(false);

			// make sure we enable the preview animation is any compatible with the skeleton
			if (PreviewAnimationAsset && EditorSkelComp->GetSkeletalMeshAsset() && 
				PreviewAnimationAsset->GetSkeleton() == EditorSkelComp->GetSkeletalMeshAsset()->GetSkeleton())
			{
				EditorSkelComp->EnablePreview(true, PreviewAnimationAsset);
				EditorSkelComp->Play(true);
			}

			// Add the floor
			TSharedPtr<IPersonaPreviewScene> Scene = PreviewScene.Pin();
			if (Scene != nullptr)
			{
				UStaticMeshComponent* FloorMeshComponent = const_cast<UStaticMeshComponent*>(Scene->GetFloorMeshComponent());
				if ((FloorMeshComponent != nullptr) && (FloorMeshComponent->GetBodyInstance() != nullptr))
				{
					EditorSkelComp->CreateSimulationFloor(FloorMeshComponent->GetBodyInstance(), FloorMeshComponent->GetBodyInstance()->GetUnrealWorldTransform());
				}
			}
		}

		if (EditorOptions->bResetClothWhenSimulating)
		{
			EditorSkelComp->RecreateClothingActors();
		}
	}
	else
	{
		// Disable the PreviewInstance
		EditorSkelComp->SetAnimationMode(EAnimationMode::AnimationSingleNode);

		// Undo ends up recreating the anim script instance, so we need to remove it here (otherwise the AnimNode_RigidBody simulation starts when we undo)
		EditorSkelComp->ClearAnimScriptInstance();

		EditorSkelComp->SetPhysicsBlendWeight(0.f);
		EditorSkelComp->ResetAllBodiesSimulatePhysics();
		EditorSkelComp->SetSimulatePhysics(false);
		ForceDisableSimulation();

		// Since simulation, actor location changes. Reset to identity 
		EditorSkelComp->SetWorldTransform(ResetTM);
		// Force an update of the skeletal mesh to get it back to ref pose
		EditorSkelComp->RefreshBoneTransforms();

		// restore the EditorSkelComp animation asset 
		if (PreviewAnimationAsset)
		{
			EditorSkelComp->EnablePreview(true, PreviewAnimationAsset);
		}

		PhysicsControlComponent->DestroyAllControlsAndBodyModifiers();

		BroadcastPreviewChanged();
	}

	bRunningSimulation = bEnableSimulation;
}

//======================================================================================================================
// Danny TODO handle the RBWC mode
void FPhysicsControlAssetEditorData::RecreateControlsAndModifiers()
{
	// Turn it off...
	PhysicsControlComponent->DestroyAllControlsAndBodyModifiers();

	// ...and back on again
	PhysicsControlComponent->PhysicsControlAsset = PhysicsControlAsset;
	PhysicsControlComponent->CreateControlsAndBodyModifiersFromPhysicsControlAsset(
		EditorSkelComp, nullptr, FName());
}

//======================================================================================================================
EPhysicsAssetEditorMeshViewMode FPhysicsControlAssetEditorData::GetCurrentMeshViewMode(bool bSimulation)
{
	if (bSimulation)
	{
		return EditorOptions->SimulationMeshViewMode;
	}
	else
	{
		return EditorOptions->MeshViewMode;
	}
}

//======================================================================================================================
EPhysicsAssetEditorCollisionViewMode FPhysicsControlAssetEditorData::GetCurrentCollisionViewMode(bool bSimulation)
{
	if (bSimulation)
	{
		return EditorOptions->SimulationCollisionViewMode;
	}
	else
	{
		return EditorOptions->CollisionViewMode;
	}
}

//======================================================================================================================
EPhysicsAssetEditorConstraintViewMode FPhysicsControlAssetEditorData::GetCurrentConstraintViewMode(bool bSimulation)
{
	if (bSimulation)
	{
		return EditorOptions->SimulationConstraintViewMode;
	}
	else
	{
		return EditorOptions->ConstraintViewMode;
	}
}

//======================================================================================================================
void FPhysicsControlAssetEditorData::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(PhysicsControlAsset);
	Collector.AddReferencedObject(EditorSkelComp);
	Collector.AddReferencedObject(PhysicsControlComponent);
	Collector.AddReferencedObject(EditorOptions);
	Collector.AddReferencedObject(MouseHandle);

	if (PreviewScene != nullptr)
	{
		PreviewScene.Pin()->AddReferencedObjects(Collector);
	}
}

//======================================================================================================================
void FPhysicsControlAssetEditorData::ForceDisableSimulation()
{
	// Reset simulation state of body instances so we dont actually simulate outside of 'simulation mode'

	UPhysicsAsset* PA = PhysicsControlAsset->GetPhysicsAsset();
	if (PA)
	{
		for (int32 BodyIdx = 0; BodyIdx < EditorSkelComp->Bodies.Num(); ++BodyIdx)
		{
			if (FBodyInstance* BodyInst = EditorSkelComp->Bodies[BodyIdx])
			{
				if (UBodySetup* PhysAssetBodySetup = PA->SkeletalBodySetups[BodyIdx])
				{
					BodyInst->SetInstanceSimulatePhysics(false);
				}
			}
		}
	}
}

//======================================================================================================================
void FPhysicsControlAssetEditorData::BroadcastPreviewChanged()
{
	PreviewChangedEvent.Broadcast();
}

//======================================================================================================================
void FPhysicsControlAssetEditorData::BroadcastSelectionChanged()
{
	if (!bSuspendSelectionBroadcast)
	{
		SelectionChangedEvent.Broadcast(SelectedBodies);
	}
}

//======================================================================================================================
void FPhysicsControlAssetEditorData::HitBone(int32 BodyIndex, EAggCollisionShape::Type PrimType, int32 PrimIndex, bool bGroupSelect)
{
	if (!bRunningSimulation)
	{
		FPhysicsControlAssetEditorData::FSelection Selection(BodyIndex, PrimType, PrimIndex);
		if (bGroupSelect)
		{
			if (IsBodySelected(Selection))
			{
				SetSelectedBody(Selection, false);
			}
			else
			{
				SetSelectedBody(Selection, true);
			}
		}
		else
		{
			ClearSelectedBody();
			SetSelectedBody(Selection, true);
		}
	}
}


//======================================================================================================================
FPhysicsControlAssetEditorData::FSelection* FPhysicsControlAssetEditorData::GetSelectedBody()
{
	int32 Count = SelectedBodies.Num();
	return Count ? &SelectedBodies[Count - 1] : NULL;
}

//======================================================================================================================
void FPhysicsControlAssetEditorData::ClearSelectedBody()
{
	SelectedBodies.Empty();
	BroadcastSelectionChanged();
}

//======================================================================================================================
void FPhysicsControlAssetEditorData::SetSelectedBody(const FSelection& Body, bool bSelected)
{
	SetSelectedBodies({ Body }, bSelected);
}

//======================================================================================================================
void FPhysicsControlAssetEditorData::SetSelectedBodies(const TArray<FSelection>& Bodies, bool bSelected)
{
	if (InsideSelChange || Bodies.Num() == 0)
	{
		return;
	}

	if (bSelected)
	{
		for (const FSelection& Body : Bodies)
		{
			SelectedBodies.AddUnique(Body);
		}
	}
	else
	{
		for (const FSelection& Body : Bodies)
		{
			SelectedBodies.Remove(Body);
		}
	}

	BroadcastSelectionChanged();

	if (!GetSelectedBody())
	{
		return;
	}

	++InsideSelChange;
	BroadcastPreviewChanged();
	--InsideSelChange;
}

//======================================================================================================================
bool FPhysicsControlAssetEditorData::IsBodySelected(const FSelection& Body) const
{
	return SelectedBodies.Contains(Body);
}


#undef LOCTEXT_NAMESPACE
