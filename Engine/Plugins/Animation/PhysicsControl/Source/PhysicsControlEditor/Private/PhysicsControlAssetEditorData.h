// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PhysicsEngine/ShapeElem.h"
#include "UObject/ObjectPtr.h"
#include "Preferences/PhysicsAssetEditorOptions.h"

class IPersonaPreviewScene;
class UPhysicsControlAsset;
class UPhysicsControlAssetEditorSkeletalMeshComponent;
class UPhysicsControlAssetEditorPhysicsHandleComponent;
class UPhysicsControlComponent;

/**
 * Helper/container for data used by the Physics Control Asset Editor 
 */
class FPhysicsControlAssetEditorData
{
public:
	FPhysicsControlAssetEditorData();

	/** Encapsulates a selected set of bodies */
	struct FSelection
	{
		int32 Index;
		EAggCollisionShape::Type PrimitiveType;
		int32 PrimitiveIndex;
		FTransform WidgetTM;
		FTransform ManipulateTM;

		FSelection(int32 GivenBodyIndex, EAggCollisionShape::Type GivenPrimitiveType, int32 GivenPrimitiveIndex) :
			Index(GivenBodyIndex), PrimitiveType(GivenPrimitiveType), PrimitiveIndex(GivenPrimitiveIndex),
			WidgetTM(FTransform::Identity), ManipulateTM(FTransform::Identity)
		{
		}

		bool operator==(const FSelection& rhs) const
		{
			return Index == rhs.Index && PrimitiveType == rhs.PrimitiveType && PrimitiveIndex == rhs.PrimitiveIndex;
		}
	};

	/** Initializes members */
	void Initialize(const TSharedRef<IPersonaPreviewScene>& InPreviewScene);

	/** Caches a preview mesh. Sets us to a default mesh if none is set yet (or if an older one got deleted) */
	void CachePreviewMesh();

	/** Accessor for mesh view mode, allows access for simulation and non-simulation modes */
	EPhysicsAssetEditorMeshViewMode GetCurrentMeshViewMode(bool bSimulation);

	/** Accessor for collision view mode, allows access for simulation and non-simulation modes */
	EPhysicsAssetEditorCollisionViewMode GetCurrentCollisionViewMode(bool bSimulation);

	/** Accessor for constraint view mode, allows access for simulation and non-simulation modes */
	EPhysicsAssetEditorConstraintViewMode GetCurrentConstraintViewMode(bool bSimulation);

	/** Prevents GC from collecting our objects */
	void AddReferencedObjects(FReferenceCollector& Collector);

	void EnableSimulation(bool bEnableSimulation);

	/** Force simulation off for all bodies, regardless of physics type */
	void ForceDisableSimulation();

	/** Toggle simulation on and off */
	void ToggleSimulation();

	/** Destroys all existing controls modifiers and then recreates them from the control asset */
	void RecreateControlsAndModifiers();

	/** broadcast a change in the preview*/
	void BroadcastPreviewChanged();

	/** broadcast a selection change ( if bSuspendSelectionBroadcast is false ) */
	void BroadcastSelectionChanged();

	/** Handle clicking on a body */
	void HitBone(int32 BodyIndex, EAggCollisionShape::Type PrimType, int32 PrimIndex, bool bGroupSelect);

	/** Selection */
	FSelection* GetSelectedBody();
	void ClearSelectedBody();
	void SetSelectedBody(const FSelection& Body, bool bSelected);
	void SetSelectedBodies(const TArray<FSelection>& Bodies, bool bSelected);
	bool IsBodySelected(const FSelection& Body) const;

public:
	/** The PhysicsControlAsset being inspected */
	TObjectPtr<UPhysicsControlAsset> PhysicsControlAsset;

	/** Skeletal mesh component specialized for this asset editor */
	TObjectPtr<UPhysicsControlAssetEditorSkeletalMeshComponent> EditorSkelComp;

	/** The physics control component used for testing/simulating on the character */
	TObjectPtr<UPhysicsControlComponent> PhysicsControlComponent;

	// viewport anim instance Danny TODO needed? This is for when running with RBAN (which we will want to do)
	//TObjectPtr<UAnimPreviewInstance> ViewportAnimInstance;

	/** Preview scene */
	TWeakPtr<IPersonaPreviewScene> PreviewScene;

	/** Editor options */
	TObjectPtr<UPhysicsAssetEditorOptions> EditorOptions;

	/** Helps define how the asset behaves given user interaction in simulation mode*/
	TObjectPtr<UPhysicsControlAssetEditorPhysicsHandleComponent> MouseHandle;

	/** Callback for triggering a refresh of the preview viewport */
	DECLARE_EVENT(FPhysicsControlAssetEditorSharedData, FPreviewChanged);
	FPreviewChanged PreviewChangedEvent;

	/** Callback for handling selection changes */
	DECLARE_EVENT_OneParam(FPhysicsAssetEditorSharedData, FSelectionChanged, const TArray<FSelection>&);
	FSelectionChanged SelectionChangedEvent;

	TArray<FSelection> SelectedBodies;

	/** Misc toggles */
	bool bRunningSimulation;
	bool bNoGravitySimulation;

	/** Manipulation (rotate, translate, scale) */
	bool bManipulating;

	/** when true, we don't broadcast every selection change - allows for bulk changes without so much overhead */
	bool bSuspendSelectionBroadcast;

	/** Used to prevent recursion with tree hierarchy ... needs to be rewritten! */
	int32 InsideSelChange;

	// Where we put the component back to after simulating. It will just be identity.
	const FTransform ResetTM;
};
