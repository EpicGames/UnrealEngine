// Copyright Epic Games, Inc. All Rights Reserved.
#include "PhysicsControlAssetEditorEditMode.h"

#include "PhysicsControlAsset.h"
#include "PhysicsControlAssetEditor.h"
#include "PhysicsControlAssetEditorData.h"
#include "PhysicsControlAssetEditorHitProxies.h"
#include "PhysicsControlAssetEditorSkeletalMeshComponent.h"
#include "PhysicsControlAssetEditorPhysicsHandleComponent.h"
#include "PhysicsControlEditorModule.h"

#include "AssetEditorModeManager.h"
#include "Engine/Font.h"
#include "GameFramework/WorldSettings.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "SEditorViewport.h"
#include "Framework/Application/SlateApplication.h"

#include "SceneView.h"

FName FPhysicsControlAssetEditorEditMode::ModeName("PhysicsControlAssetEditMode");

//======================================================================================================================
FPhysicsControlAssetEditorEditMode::FPhysicsControlAssetEditorEditMode()
	: SimHoldDistanceChangeDelta(20.0f)
	, SimMinHoldDistance(10.0f)
	, SimGrabMoveSpeed(1.0f)
{
	// Disable grid drawing for this mode as the viewport handles this
	bDrawGrid = false;

	LastClickPos = FIntPoint::ZeroValue;
	LastClickOrigin = FVector::ZeroVector;
	LastClickDirection = FVector::UpVector;
	LastClickHitPos = FVector::ZeroVector;
	LastClickHitNormal = FVector::UpVector;
	bLastClickHit = false;

	PhysicsControlAssetEditorFont = GEngine->GetSmallFont();
	check(PhysicsControlAssetEditorFont);
}


//======================================================================================================================
void FPhysicsControlAssetEditorEditMode::SetEditorData(
	const TSharedRef<FPhysicsControlAssetEditor> InPhysicsControlAssetEditor,
	TSharedPtr<FPhysicsControlAssetEditorData> InEditorData)
{
	PhysicsControlAssetEditor = InPhysicsControlAssetEditor;
	EditorData = InEditorData;
}

//======================================================================================================================
bool FPhysicsControlAssetEditorEditMode::GetCameraTarget(FSphere& OutTarget) const
{
	bool bHandled = false;

	float NumSelected = 0.0f;
	FBox Bounds(ForceInit);
	if (const UPhysicsAsset* PA = EditorData->PhysicsControlAsset->GetPhysicsAsset())
	{
		for (int32 BodyIndex = 0; BodyIndex < EditorData->SelectedBodies.Num(); ++BodyIndex)
		{
			FPhysicsControlAssetEditorData::FSelection& SelectedObject = EditorData->SelectedBodies[BodyIndex];
			int32 BoneIndex = EditorData->EditorSkelComp->GetBoneIndex(PA->SkeletalBodySetups[SelectedObject.Index]->BoneName);
			UBodySetup* BodySetup = PA->SkeletalBodySetups[SelectedObject.Index];
			const FKAggregateGeom& AggGeom = BodySetup->AggGeom;

			FTransform BoneTM = EditorData->EditorSkelComp->GetBoneTransform(BoneIndex);
			const float Scale = (float) BoneTM.GetScale3D().GetAbsMax();
			BoneTM.RemoveScaling();

			if (SelectedObject.PrimitiveType == EAggCollisionShape::Sphere)
			{
				Bounds += AggGeom.SphereElems[SelectedObject.PrimitiveIndex].CalcAABB(BoneTM, Scale);
			}
			else if (SelectedObject.PrimitiveType == EAggCollisionShape::Box)
			{
				Bounds += AggGeom.BoxElems[SelectedObject.PrimitiveIndex].CalcAABB(BoneTM, Scale);
			}
			else if (SelectedObject.PrimitiveType == EAggCollisionShape::Sphyl)
			{
				Bounds += AggGeom.SphylElems[SelectedObject.PrimitiveIndex].CalcAABB(BoneTM, Scale);
			}
			else if (SelectedObject.PrimitiveType == EAggCollisionShape::Convex)
			{
				Bounds += AggGeom.ConvexElems[SelectedObject.PrimitiveIndex].CalcAABB(BoneTM, BoneTM.GetScale3D());
			}
			else if (SelectedObject.PrimitiveType == EAggCollisionShape::TaperedCapsule)
			{
				Bounds += AggGeom.TaperedCapsuleElems[SelectedObject.PrimitiveIndex].CalcAABB(BoneTM, Scale);
			}
			else if (SelectedObject.PrimitiveType == EAggCollisionShape::LevelSet)
			{
				Bounds += AggGeom.LevelSetElems[SelectedObject.PrimitiveIndex].CalcAABB(BoneTM, BoneTM.GetScale3D());
			}
			else if (SelectedObject.PrimitiveType == EAggCollisionShape::SkinnedLevelSet)
			{
				Bounds += AggGeom.SkinnedLevelSetElems[SelectedObject.PrimitiveIndex].CalcAABB(BoneTM, BoneTM.GetScale3D());
			}

			bHandled = true;
		}
	}

	OutTarget.Center = Bounds.GetCenter();
	OutTarget.W = Bounds.GetExtent().Size();	// @TODO: calculate correct bounds

	return bHandled;
}

//======================================================================================================================
IPersonaPreviewScene& FPhysicsControlAssetEditorEditMode::GetAnimPreviewScene() const
{
	return *static_cast<IPersonaPreviewScene*>(static_cast<FAssetEditorModeManager*>(Owner)->GetPreviewScene());
}

//======================================================================================================================
void FPhysicsControlAssetEditorEditMode::Exit()
{
	IPersonaEditMode::Exit();
}

//======================================================================================================================
void FPhysicsControlAssetEditorEditMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	FEdMode::Render(View, Viewport, PDI);
	// Danny TODO EditorData needs to be a smart pointer - this can fail on shut down
	EPhysicsAssetEditorMeshViewMode MeshViewMode = EditorData->GetCurrentMeshViewMode(EditorData->bRunningSimulation);

	if (MeshViewMode != EPhysicsAssetEditorMeshViewMode::None)
	{
		EditorData->EditorSkelComp->SetVisibility(true);

		if (MeshViewMode == EPhysicsAssetEditorMeshViewMode::Wireframe)
		{
			EditorData->EditorSkelComp->SetForceWireframe(true);
		}
		else
		{
			EditorData->EditorSkelComp->SetForceWireframe(false);
		}
	}
	else
	{
		EditorData->EditorSkelComp->SetVisibility(false);
	}

	// Draw phat skeletal component.
	EditorData->EditorSkelComp->DebugDraw(View, PDI);
}

//======================================================================================================================
void FPhysicsControlAssetEditorEditMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	FEdMode::Tick(ViewportClient, DeltaTime);

	if (EditorData->bRunningSimulation)
	{
		// check if PIE disabled the realtime viewport and quit sim if so
		if (!ViewportClient->IsRealtime())
		{
			EditorData->ToggleSimulation();

			ViewportClient->Invalidate();
		}

		UWorld* World = EditorData->PreviewScene.Pin()->GetWorld();
		AWorldSettings* Setting = World->GetWorldSettings();
		Setting->WorldGravityZ = EditorData->bNoGravitySimulation ? 
			0.0f : 
			(EditorData->EditorOptions->bUseGravityOverride ? 
				EditorData->EditorOptions->GravityOverrideZ : 
				(UPhysicsSettings::Get()->DefaultGravityZ * EditorData->EditorOptions->GravScale));
		Setting->bWorldGravitySet = true;

		// We back up the transforms array now
		EditorData->EditorSkelComp->AnimationSpaceBases = EditorData->EditorSkelComp->GetComponentSpaceTransforms();

		// We don't apply the physics blend, since that comes from body modifiers.
		EditorData->EditorSkelComp->bUpdateJointsFromAnimation = EditorData->EditorOptions->bUpdateJointsFromAnimation;
		EditorData->EditorSkelComp->PhysicsTransformUpdateMode = EditorData->EditorOptions->PhysicsUpdateMode;
	}
}

//======================================================================================================================
void FPhysicsControlAssetEditorEditMode::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	FEdMode::DrawHUD(ViewportClient, Viewport, View, Canvas);
}

//======================================================================================================================
void FPhysicsControlAssetEditorEditMode::HitNothing(FEditorViewportClient* InViewportClient)
{
	if (InViewportClient->IsCtrlPressed() == false)	//we only want to deselect if Ctrl is not used
	{
		EditorData->ClearSelectedBody();
	}

	InViewportClient->Invalidate();
	if (TSharedPtr<FPhysicsControlAssetEditor> Editor = PhysicsControlAssetEditor.Pin())
	{
		Editor->RefreshHierarchyTree();
	}
}

//======================================================================================================================
bool FPhysicsControlAssetEditorEditMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	if (!EditorData->bRunningSimulation)
	{
		if (Click.GetKey() == EKeys::LeftMouseButton)
		{
			if (HitProxy && HitProxy->IsA(HPhysicsControlAssetEditorEdBoneProxy::StaticGetType()))
			{
				HPhysicsControlAssetEditorEdBoneProxy* BoneProxy = (HPhysicsControlAssetEditorEdBoneProxy*)HitProxy;

				EditorData->HitBone(BoneProxy->BodyIndex, BoneProxy->PrimType, BoneProxy->PrimIndex, 
					InViewportClient->IsCtrlPressed() || InViewportClient->IsShiftPressed());
				return true;
			}
			else
			{
				HitNothing(InViewportClient);
			}
		}
		else if (Click.GetKey() == EKeys::RightMouseButton)
		{
			if (HitProxy && HitProxy->IsA(HPhysicsControlAssetEditorEdBoneProxy::StaticGetType()))
			{
				HPhysicsControlAssetEditorEdBoneProxy* BoneProxy = (HPhysicsControlAssetEditorEdBoneProxy*)HitProxy;

				// Select body under cursor if not already selected	(if ctrl is held down we only add, not remove)
				FPhysicsControlAssetEditorData::FSelection Selection(BoneProxy->BodyIndex, BoneProxy->PrimType, BoneProxy->PrimIndex);
				if (!EditorData->IsBodySelected(Selection))
				{
					if (!InViewportClient->IsCtrlPressed())
					{
						EditorData->ClearSelectedBody();
					}

					EditorData->SetSelectedBody(Selection, true);
				}

				// Pop up menu, if we have a body selected.
				if (EditorData->GetSelectedBody())
				{
					OpenBodyMenu(InViewportClient);
				}

				return true;
			}
			else
			{
				OpenSelectionMenu(InViewportClient);
				return true;
			}
		}
	}

	return false;
}

//======================================================================================================================
/** Helper function to open a viewport context menu */
static void OpenContextMenu(
	const TSharedRef<FPhysicsControlAssetEditor>& PhysicsControlAssetEditor,
	FEditorViewportClient*                        InViewportClient, 
	TFunctionRef<void(FMenuBuilder&)>             InBuildMenu)
{
	FMenuBuilder MenuBuilder(true, PhysicsControlAssetEditor->GetToolkitCommands());

	InBuildMenu(MenuBuilder);

	TSharedPtr<SWidget> MenuWidget = MenuBuilder.MakeWidget();
	TSharedPtr<SWidget> ParentWidget = InViewportClient->GetEditorViewportWidget();

	if (MenuWidget.IsValid() && ParentWidget.IsValid())
	{
		const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();

		FSlateApplication::Get().PushMenu(
			ParentWidget.ToSharedRef(),
			FWidgetPath(),
			MenuWidget.ToSharedRef(),
			MouseCursorLocation,
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
	}
}

//======================================================================================================================
void FPhysicsControlAssetEditorEditMode::OpenBodyMenu(FEditorViewportClient* InViewportClient)
{
	TSharedPtr<FPhysicsControlAssetEditor> Editor = PhysicsControlAssetEditor.Pin();
	if (!Editor)
	{
		return;
	}
	OpenContextMenu(Editor.ToSharedRef(), InViewportClient,
		[Editor](FMenuBuilder& InMenuBuilder)
		{
			Editor->BuildMenuWidgetBody(InMenuBuilder);
			Editor->BuildMenuWidgetSelection(InMenuBuilder);
		});
}

//======================================================================================================================
void FPhysicsControlAssetEditorEditMode::OpenSelectionMenu(FEditorViewportClient* InViewportClient)
{
	TSharedPtr<FPhysicsControlAssetEditor> Editor = PhysicsControlAssetEditor.Pin();
	if (!Editor)
	{
		return;
	}
	OpenContextMenu(Editor.ToSharedRef(), InViewportClient,
		[Editor](FMenuBuilder& InMenuBuilder)
		{
			Editor->BuildMenuWidgetSelection(InMenuBuilder);
		});
}


//======================================================================================================================
bool FPhysicsControlAssetEditorEditMode::ReceivedFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
	return IPersonaEditMode::ReceivedFocus(ViewportClient, Viewport);
}

//======================================================================================================================
bool FPhysicsControlAssetEditorEditMode::LostFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
	return IPersonaEditMode::LostFocus(ViewportClient, Viewport);
}

//======================================================================================================================
bool FPhysicsControlAssetEditorEditMode::InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey Key, EInputEvent Event)
{
	int32 HitX = InViewport->GetMouseX();
	int32 HitY = InViewport->GetMouseY();
	bool bCtrlDown = InViewport->KeyState(EKeys::LeftControl) || InViewport->KeyState(EKeys::RightControl);
	bool bShiftDown = InViewport->KeyState(EKeys::LeftShift) || InViewport->KeyState(EKeys::RightShift);

	bool bHandled = false;
	if (EditorData->bRunningSimulation)
	{
		if (Key == EKeys::RightMouseButton || Key == EKeys::LeftMouseButton)
		{
			if (Event == IE_Pressed)
			{
				bHandled = SimMousePress(InViewportClient, Key);
			}
			else if (Event == IE_Released)
			{
				bHandled = SimMouseRelease();
			}
			else
			{
				// Handle repeats/double clicks etc. so we dont fall through
				bHandled = true;
			}
		}
		else if (Key == EKeys::MouseScrollUp)
		{
			bHandled = SimMouseWheelUp(InViewportClient);
		}
		else if (Key == EKeys::MouseScrollDown)
		{
			bHandled = SimMouseWheelDown(InViewportClient);
		}
		else if (InViewportClient->IsFlightCameraActive())
		{
			// If the flight camera is active (user is looking or moving around the scene)
			// consume the event so hotkeys don't fire.
			bHandled = true;
		}
	}

	if (!bHandled)
	{
		bHandled = IPersonaEditMode::InputKey(InViewportClient, InViewport, Key, Event);
	}

	if (bHandled)
	{
		InViewportClient->Invalidate();
	}

	return bHandled;
}

//======================================================================================================================
bool FPhysicsControlAssetEditorEditMode::InputAxis(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime)
{
	bool bHandled = false;
	// If we are 'manipulating' don't move the camera but do something else with mouse input.
	if (EditorData->bManipulating)
	{
		bool bCtrlDown = InViewport->KeyState(EKeys::LeftControl) || InViewport->KeyState(EKeys::RightControl);

		if (EditorData->bRunningSimulation)
		{
			if (Key == EKeys::MouseX)
			{
				SimMouseMove(InViewportClient, Delta, 0.0f);
			}
			else if (Key == EKeys::MouseY)
			{
				SimMouseMove(InViewportClient, 0.0f, Delta);
			}
			bHandled = true;
		}
	}

	if (!bHandled)
	{
		bHandled = IPersonaEditMode::InputAxis(InViewportClient, InViewport, ControllerId, Key, Delta, DeltaTime);
	}

	InViewportClient->Invalidate();

	return bHandled;
}


//======================================================================================================================
bool FPhysicsControlAssetEditorEditMode::SimMousePress(FEditorViewportClient* InViewportClient, FKey Key)
{
	bool bHandled = false;

	FViewport* Viewport = InViewportClient->Viewport;

	bool bCtrlDown = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
	bool bShiftDown = Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift);

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		Viewport, InViewportClient->GetScene(), InViewportClient->EngineShowFlags));
	FSceneView* View = InViewportClient->CalcSceneView(&ViewFamily);

	const FViewportClick Click(
		View, InViewportClient, EKeys::Invalid, IE_Released, Viewport->GetMouseX(), Viewport->GetMouseY());
	FHitResult Result(1.f);
	bool bHit = EditorData->EditorSkelComp->LineTraceComponent(
		Result, Click.GetOrigin(), Click.GetOrigin() + Click.GetDirection() * EditorData->EditorOptions->InteractionDistance, FCollisionQueryParams(NAME_None, true));

	LastClickPos = Click.GetClickPos();
	LastClickOrigin = Click.GetOrigin();
	LastClickDirection = Click.GetDirection();
	bLastClickHit = bHit;
	if (bHit)
	{
		LastClickHitPos = Result.Location;
		LastClickHitNormal = Result.Normal;
	}

	const UPhysicsAsset* PA = EditorData->PhysicsControlAsset->GetPhysicsAsset();
	if (bHit && PA)
	{
		check(Result.Item != INDEX_NONE);
		FName BoneName = PA->SkeletalBodySetups[Result.Item]->BoneName;

		UE_LOGF(LogPhysics, Log, "Physics Asset Editor Click Hit Bone (%ls)", *BoneName.ToString());

		if (bCtrlDown || bShiftDown)
		{
			// Right mouse is for dragging things around
			if (Key == EKeys::RightMouseButton)
			{
				EditorData->bManipulating = true;
				DragX = 0.0f;
				DragY = 0.0f;
				SimGrabPush = 0.0f;

				// Update mouse force properties from sim options.
				EditorData->MouseHandle->LinearDamping = EditorData->EditorOptions->HandleLinearDamping;
				EditorData->MouseHandle->LinearStiffness = EditorData->EditorOptions->HandleLinearStiffness;
				EditorData->MouseHandle->AngularDamping = EditorData->EditorOptions->HandleAngularDamping;
				EditorData->MouseHandle->AngularStiffness = EditorData->EditorOptions->HandleAngularStiffness;
				EditorData->MouseHandle->InterpolationSpeed = EditorData->EditorOptions->InterpolationSpeed;

				// Create handle to object.
				EditorData->MouseHandle->GrabComponentAtLocationWithRotation(
					EditorData->EditorSkelComp, BoneName, Result.Location, FRotator::ZeroRotator);

				FMatrix	InvViewMatrix = View->ViewMatrices.GetViewToWorld();

				SimGrabMinPush = SimMinHoldDistance - (Result.Time * EditorData->EditorOptions->InteractionDistance);

				SimGrabLocation = Result.Location;
				SimGrabX = InvViewMatrix.GetUnitAxis(EAxis::X);
				SimGrabY = InvViewMatrix.GetUnitAxis(EAxis::Y);
				SimGrabZ = InvViewMatrix.GetUnitAxis(EAxis::Z);
			}
			// Left mouse is for poking things
			else if (Key == EKeys::LeftMouseButton)
			{
				EditorData->MouseHandle->AddImpulseAtLocation(
					EditorData->EditorSkelComp,
					Click.GetDirection() * EditorData->EditorOptions->PokeStrength, Result.Location, BoneName);
			}

			bHandled = true;
		}
	}

	return bHandled;
}

//======================================================================================================================
void FPhysicsControlAssetEditorEditMode::SimMouseMove(FEditorViewportClient* InViewportClient, float DeltaX, float DeltaY)
{
	DragX = (float) (InViewportClient->Viewport->GetMouseX() - LastClickPos.X);
	DragY = (float) (InViewportClient->Viewport->GetMouseY() - LastClickPos.Y);

	if (!EditorData->MouseHandle->GrabbedComponent)
	{
		return;
	}

	//We need to convert Pixel Delta into Screen position (deal with different viewport sizes)
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(InViewportClient->Viewport, EditorData->PreviewScene.Pin()->GetScene(), InViewportClient->EngineShowFlags));
	FSceneView* View = InViewportClient->CalcSceneView(&ViewFamily);
	FVector4 ScreenOldPos = View->PixelToScreen((float) LastClickPos.X, (float) LastClickPos.Y, 1.f);
	FVector4 ScreenNewPos = View->PixelToScreen(DragX + (float) LastClickPos.X, DragY + (float) LastClickPos.Y, 1.f);
	FVector4 ScreenDelta = ScreenNewPos - ScreenOldPos;
	FVector4 ProjectedDelta = View->ScreenToWorld(ScreenDelta);
	FVector4 WorldDelta;

	//Now we project new ScreenPos to xy-plane of SimGrabLocation
	FVector LocalOffset = View->ViewMatrices.GetWorldToView().TransformPosition(SimGrabLocation + SimGrabZ * SimGrabPush);
	//in the ortho case we don't need to do any fixup because there is no perspective
	double ZDistance = InViewportClient->GetViewportType() == ELevelViewportType::LVT_Perspective ? FMath::Abs(LocalOffset.Z) : 1.0;
	WorldDelta = ProjectedDelta * ZDistance;

	//Now we convert back into WorldPos
	FVector WorldPos = SimGrabLocation + WorldDelta + SimGrabZ * SimGrabPush;
	FVector NewLocation = WorldPos;
	float QuickRadius = 5 - SimGrabPush / SimHoldDistanceChangeDelta;
	QuickRadius = QuickRadius < 2 ? 2 : QuickRadius;

	DrawDebugPoint(GetWorld(), NewLocation, QuickRadius, FColorList::Red, false, 0.3f);

	EditorData->MouseHandle->SetTargetLocation(NewLocation);
	EditorData->MouseHandle->GrabbedComponent->WakeRigidBody(EditorData->MouseHandle->GrabbedBoneName);
}

//======================================================================================================================
bool FPhysicsControlAssetEditorEditMode::SimMouseRelease()
{
	EditorData->bManipulating = false;

	if (!EditorData->MouseHandle->GrabbedComponent)
	{
		return false;
	}

	EditorData->MouseHandle->GrabbedComponent->WakeRigidBody(EditorData->MouseHandle->GrabbedBoneName);
	EditorData->MouseHandle->ReleaseComponent();

	return true;
}

//======================================================================================================================
bool FPhysicsControlAssetEditorEditMode::SimMouseWheelUp(FEditorViewportClient* InViewportClient)
{
	if (!EditorData->MouseHandle->GrabbedComponent)
	{
		return false;
	}

	SimGrabPush += SimHoldDistanceChangeDelta;

	SimMouseMove(InViewportClient, 0.0f, 0.0f);

	return true;
}

//======================================================================================================================
bool FPhysicsControlAssetEditorEditMode::SimMouseWheelDown(FEditorViewportClient* InViewportClient)
{
	if (!EditorData->MouseHandle->GrabbedComponent)
	{
		return false;
	}

	SimGrabPush -= SimHoldDistanceChangeDelta;
	SimGrabPush = FMath::Max(SimGrabMinPush, SimGrabPush);

	SimMouseMove(InViewportClient, 0.0f, 0.0f);

	return true;
}
