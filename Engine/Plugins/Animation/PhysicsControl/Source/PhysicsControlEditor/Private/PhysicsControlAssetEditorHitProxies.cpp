// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAssetEditorHitProxies.h"

IMPLEMENT_HIT_PROXY(HPhysicsControlAssetEditorEdBoneProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HPhysicsControlAssetEditorEdBoneNameProxy,HHitProxy);
