// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/ICursor.h"
#include "HAL/Platform.h"
#include "HitProxies.h"
#include "PhysicsEngine/ShapeElem.h"

/*-----------------------------------------------------------------------------
   Hit Proxies
-----------------------------------------------------------------------------*/

struct HPhysicsControlAssetEditorEdBoneProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	int32							BodyIndex;
	EAggCollisionShape::Type		PrimType;
	int32							PrimIndex;

	HPhysicsControlAssetEditorEdBoneProxy(int32 InBodyIndex, EAggCollisionShape::Type InPrimType, int32 InPrimIndex)
		: HHitProxy(HPP_World)
		, BodyIndex(InBodyIndex)
		, PrimType(InPrimType)
		, PrimIndex(InPrimIndex) {}

	virtual EMouseCursor::Type GetMouseCursor()
	{
		return EMouseCursor::Crosshairs;
	}

	virtual bool AlwaysAllowsTranslucentPrimitives() const override
	{
		return true;
	}
};

struct HPhysicsControlAssetEditorEdBoneNameProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	int32			BoneIndex;

	HPhysicsControlAssetEditorEdBoneNameProxy(int32 InBoneIndex)
		: HHitProxy(HPP_Foreground)
		, BoneIndex(InBoneIndex) {}

	virtual bool AlwaysAllowsTranslucentPrimitives() const override
	{
		return true;
	}
};
