// Copyright Epic Games, Inc. All Rights Reserved.
#include "PhysicsControlAssetEditorPhysicsHandleComponent.h"

#include "PhysicsControlAssetEditorSkeletalMeshComponent.h"

//======================================================================================================================

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsControlAssetEditorPhysicsHandleComponent)
UPhysicsControlAssetEditorPhysicsHandleComponent::UPhysicsControlAssetEditorPhysicsHandleComponent(
	const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAnimInstanceMode(false)
{
}

//======================================================================================================================
void UPhysicsControlAssetEditorPhysicsHandleComponent::UpdateHandleTransform(const FTransform& NewTransform)
{
	Super::UpdateHandleTransform(NewTransform);
	if (bAnimInstanceMode)
	{
		UPhysicsControlAssetEditorSkeletalMeshComponent* PhysicsControlAssetEditorSkeletalMeshComponent = 
			Cast<UPhysicsControlAssetEditorSkeletalMeshComponent>(GrabbedComponent);
		if (PhysicsControlAssetEditorSkeletalMeshComponent != nullptr)
		{
			PhysicsControlAssetEditorSkeletalMeshComponent->UpdateHandleTransform(NewTransform);
		}
	}
}

//======================================================================================================================
void UPhysicsControlAssetEditorPhysicsHandleComponent::UpdateDriveSettings()
{
	Super::UpdateDriveSettings();
	if (bAnimInstanceMode)
	{
		UPhysicsControlAssetEditorSkeletalMeshComponent* PhysicsControlAssetEditorSkeletalMeshComponent = 
			Cast<UPhysicsControlAssetEditorSkeletalMeshComponent>(GrabbedComponent);
		if (PhysicsControlAssetEditorSkeletalMeshComponent != nullptr)
		{
			PhysicsControlAssetEditorSkeletalMeshComponent->UpdateDriveSettings(
				bSoftLinearConstraint, LinearStiffness, LinearDamping);
		}
	}
}

//======================================================================================================================
void UPhysicsControlAssetEditorPhysicsHandleComponent::SetAnimInstanceMode(bool bInAnimInstanceMode)
{
	bAnimInstanceMode = bInAnimInstanceMode;
}

//======================================================================================================================
void UPhysicsControlAssetEditorPhysicsHandleComponent::GrabComponentImp(
	class UPrimitiveComponent* Component, FName InBoneName, 
	const FVector& Location, const FRotator& Rotation, bool InbRotationConstrained)
{
	Super::GrabComponentImp(Component, InBoneName, Location, Rotation, InbRotationConstrained);
	if (bAnimInstanceMode)
	{
		TargetTransform = CurrentTransform = FTransform(Rotation, Location);

		UPhysicsControlAssetEditorSkeletalMeshComponent* PhysicsControlAssetEditorSkeletalMeshComponent = 
			Cast<UPhysicsControlAssetEditorSkeletalMeshComponent>(Component);
		if (PhysicsControlAssetEditorSkeletalMeshComponent != nullptr)
		{
			PhysicsControlAssetEditorSkeletalMeshComponent->Grab(InBoneName, Location, Rotation, InbRotationConstrained);
		}
	}
}

//======================================================================================================================
void UPhysicsControlAssetEditorPhysicsHandleComponent::ReleaseComponent()
{
	if (bAnimInstanceMode)
	{
		UPhysicsControlAssetEditorSkeletalMeshComponent* PhysicsControlAssetEditorSkeletalMeshComponent = 
			Cast<UPhysicsControlAssetEditorSkeletalMeshComponent>(GrabbedComponent);
		if (PhysicsControlAssetEditorSkeletalMeshComponent != nullptr)
		{
			PhysicsControlAssetEditorSkeletalMeshComponent->Ungrab();
		}
	}
	Super::ReleaseComponent();
}

//======================================================================================================================
void UPhysicsControlAssetEditorPhysicsHandleComponent::AddImpulseAtLocation(
	UPhysicsControlAssetEditorSkeletalMeshComponent* PhysicsControlAssetEditorSkeletalMeshComponent, 
	FVector Impulse, FVector Location, FName BoneName)
{
	if (bAnimInstanceMode)
	{
		PhysicsControlAssetEditorSkeletalMeshComponent->AddImpulseAtLocation(Impulse, Location, BoneName);
	}
	else
	{
		FBodyInstance* BodyInstance = PhysicsControlAssetEditorSkeletalMeshComponent->GetBodyInstance(BoneName);
		if (!BodyInstance)
		{
			return;
		}
		// If the grabbed body is welded to another, poke that instead
		if (BodyInstance->WeldParent != nullptr)
		{
			BodyInstance = BodyInstance->WeldParent;
		}
		BodyInstance->AddImpulseAtPosition(Impulse, Location);
	}
}
