// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAssetEditorSkeletalMeshComponent.h"
#include "PhysicsControlAsset.h"
#include "PhysicsControlAssetEditorData.h"
#include "PhysicsControlAssetEditorHitProxies.h"
#include "PhysicsControlAssetEditorSkeletalMeshComponent.h"
#include "PhysicsControlAssetEditorAnimInstance.h"
#include "PhysicsControlLog.h"

#include "AnimPreviewInstance.h"
#include "Chaos/Core.h"
#include "Chaos/Levelset.h"
#include "Chaos/WeightedLatticeImplicitObject.h"
#include "Components/SkeletalMeshComponent.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "PhysicsAssetRenderUtils.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Preferences/PhysicsAssetEditorOptions.h"
#include "PrimitiveDrawingUtils.h"
#include "SkeletalMeshTypes.h"
#include "Styling/AppStyle.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsControlAssetEditorSkeletalMeshComponent)

UPhysicsControlAssetEditorSkeletalMeshComponent::UPhysicsControlAssetEditorSkeletalMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, BoneUnselectedColor(170, 155, 225)
	, NoCollisionColor(200, 200, 200)
	, FixedColor(125, 125, 0)
	, ConstraintBone1Color(255, 166, 0)
	, ConstraintBone2Color(0, 150, 150)
	, HierarchyDrawColor(220, 255, 220)
	, AnimSkelDrawColor(255, 64, 64)
	, COMRenderSize(5.0f)
	, InfluenceLineLength(2.0f)
	, InfluenceLineColor(0, 255, 0)
{
	if (!HasAnyFlags(RF_DefaultSubObject | RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		// Body materials
		UMaterialInterface* BaseElemSelectedMaterial = LoadObject<UMaterialInterface>(NULL, TEXT("/Engine/EditorMaterials/PhAT_ElemSelectedMaterial.PhAT_ElemSelectedMaterial"), NULL, LOAD_None, NULL);
		ElemSelectedMaterial = UMaterialInstanceDynamic::Create(BaseElemSelectedMaterial, GetTransientPackage());
		check(ElemSelectedMaterial);

		BoneMaterialHit = UMaterial::GetDefaultMaterial(MD_Surface);
		check(BoneMaterialHit);

		UMaterialInterface* BaseBoneUnselectedMaterial = LoadObject<UMaterialInterface>(NULL, TEXT("/Engine/EditorMaterials/PhAT_UnselectedMaterial.PhAT_UnselectedMaterial"), NULL, LOAD_None, NULL);
		BoneUnselectedMaterial = UMaterialInstanceDynamic::Create(BaseBoneUnselectedMaterial, GetTransientPackage());
		check(BoneUnselectedMaterial);

		UMaterialInterface* BaseBoneNoCollisionMaterial = LoadObject<UMaterialInterface>(NULL, TEXT("/Engine/EditorMaterials/PhAT_NoCollisionMaterial.PhAT_NoCollisionMaterial"), NULL, LOAD_None, NULL);
		BoneNoCollisionMaterial = UMaterialInstanceDynamic::Create(BaseBoneNoCollisionMaterial, GetTransientPackage());
		check(BoneNoCollisionMaterial);

		// this is because in phat editor, you'd like to see fixed bones to be fixed without animation force update
		KinematicBonesUpdateType = EKinematicBonesUpdateToPhysics::SkipSimulatingBones;
		bUpdateJointsFromAnimation = false;
		SetForcedLOD(1);

		static FName CollisionProfileName(TEXT("PhysicsActor"));
		SetCollisionProfileName(CollisionProfileName);
	}

	bSelectable = false;
}

TObjectPtr<UAnimPreviewInstance> UPhysicsControlAssetEditorSkeletalMeshComponent::CreatePreviewInstance()
{
	return NewObject<UPhysicsControlAssetEditorAnimInstance>(this, TEXT("PhysicsAssetEditorPreviewInstance"));
}

void UPhysicsControlAssetEditorSkeletalMeshComponent::DebugDraw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	if (!EditorData)
	{
		// Nothing to draw if our editor data is not available
		return;
	}

	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();

	if (!PhysicsAsset)
	{
		// Nothing to draw without an asset, this can happen if the preview scene has no skeletal mesh
		return;
	}

	EPhysicsAssetEditorCollisionViewMode CollisionViewMode = EditorData->GetCurrentCollisionViewMode(EditorData->bRunningSimulation);

	// set opacity of our materials
	static FName OpacityName(TEXT("Opacity"));
	ElemSelectedMaterial->SetScalarParameterValue(OpacityName, EditorData->EditorOptions->CollisionOpacity);
	BoneUnselectedMaterial->SetScalarParameterValue(OpacityName, EditorData->EditorOptions->bSolidRenderingForSelectedOnly ? 0.0f : EditorData->EditorOptions->CollisionOpacity);
	BoneNoCollisionMaterial->SetScalarParameterValue(OpacityName, EditorData->EditorOptions->bSolidRenderingForSelectedOnly ? 0.0f : EditorData->EditorOptions->CollisionOpacity);

	static FName SelectionColorName(TEXT("SelectionColor"));
	const FSlateColor SelectionColor = FAppStyle::GetSlateColor(SelectionColorName);
	const FLinearColor LinearSelectionColor(SelectionColor.IsColorSpecified() ? SelectionColor.GetSpecifiedColor() : FLinearColor::White);

	ElemSelectedMaterial->SetVectorParameterValue(SelectionColorName, LinearSelectionColor);

	FPhysicsAssetRenderSettings* const RenderSettings = UPhysicsAssetRenderUtilities::GetSettings(PhysicsAsset);
	
	if (RenderSettings)
	{
		// Copy render settings from editor viewport. These settings must be applied to the rendering in all editors 
		// when an asset is open in the Physics Asset Editor but should not persist after the editor has been closed.
		RenderSettings->CollisionViewMode = EditorData->GetCurrentCollisionViewMode(EditorData->bRunningSimulation);
		RenderSettings->ConstraintViewMode = EditorData->GetCurrentConstraintViewMode(EditorData->bRunningSimulation);
		RenderSettings->ConstraintDrawSize = EditorData->EditorOptions->ConstraintDrawSize;
		RenderSettings->PhysicsBlend = EditorData->EditorOptions->PhysicsBlend;
		RenderSettings->bHideKinematicBodies = EditorData->EditorOptions->bHideKinematicBodies;
		RenderSettings->bHideSimulatedBodies = EditorData->EditorOptions->bHideSimulatedBodies;
		RenderSettings->bDrawViolatedLimits = EditorData->EditorOptions->bDrawViolatedLimits;

		// Draw Bodies.
		{
			auto TransformFn = [this](
				const UPhysicsAsset* PhysicsAsset, const FTransform& BoneTM, const int32 BodyIndex, 
				const EAggCollisionShape::Type PrimType, const int32 PrimIndex, const float Scale) 
				{ 
					return this->GetPrimitiveTransform(BoneTM, BodyIndex, PrimType, PrimIndex, Scale);  
				};
			auto ColorFn = [this](
				const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex, 
				const FPhysicsAssetRenderSettings& Settings) 
				{ 
					return this->GetPrimitiveColor(BodyIndex, PrimitiveType, PrimitiveIndex); 
				};
			auto MaterialFn = [this](
				const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex, 
				const FPhysicsAssetRenderSettings& Settings) 
				{ 
					return this->GetPrimitiveMaterial(BodyIndex, PrimitiveType, PrimitiveIndex); 
				};
			auto HitProxyFn = [](
				const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex)
				{
					return new HPhysicsControlAssetEditorEdBoneProxy(BodyIndex, PrimitiveType, PrimitiveIndex); 
				};

			PhysicsAssetRender::DebugDrawBodies(this, PhysicsAsset, PDI, ColorFn, MaterialFn, TransformFn, HitProxyFn);
		}

		// Draw Constraints.
		{
			auto HitProxyFn = [](const int32 InConstraintIndex) { return nullptr; };
			auto IsConstraintSelectedFn = [this](const uint32 InConstraintIndex) { return false; };

			PhysicsAssetRender::DebugDrawConstraints(this, PhysicsAsset, PDI, IsConstraintSelectedFn, EditorData->bRunningSimulation, HitProxyFn);
		}
	}
}

FPrimitiveSceneProxy* UPhysicsControlAssetEditorSkeletalMeshComponent::CreateSceneProxy()
{
	if (!EditorData)
	{
		// EditorData may not be set up yet, or may have been torn down, so fall back to the
		// default behavior rather than dereferencing it
		return UDebugSkelMeshComponent::CreateSceneProxy();
	}

	FPrimitiveSceneProxy* Proxy = NULL;
	EPhysicsAssetEditorMeshViewMode MeshViewMode = EditorData->GetCurrentMeshViewMode(EditorData->bRunningSimulation);
	if (MeshViewMode != EPhysicsAssetEditorMeshViewMode::None)
	{
		Proxy = UDebugSkelMeshComponent::CreateSceneProxy();
	}

	return Proxy;
}

FTransform UPhysicsControlAssetEditorSkeletalMeshComponent::GetPrimitiveTransform(const FTransform& BoneTM, const int32 BodyIndex, const EAggCollisionShape::Type PrimType, const int32 PrimIndex, const float Scale) const
{
	if (!EditorData)
	{
		return FTransform::Identity;
	}

	UPhysicsControlAsset* PhysicsControlAsset = EditorData->PhysicsControlAsset;
	UPhysicsAsset* PhysicsAsset = PhysicsControlAsset ? PhysicsControlAsset->GetPhysicsAsset() : nullptr;
	if (!PhysicsAsset)
	{
		UE_LOGF(LogPhysicsControl, Warning, "GetPrimitiveTransform - no physics asset");
		return FTransform::Identity;
	}

	if (!PhysicsAsset->SkeletalBodySetups.IsValidIndex(BodyIndex))
	{
		UE_LOGF(LogPhysicsControl, Warning, "GetPrimitiveTransform - body index out of range");
		return FTransform::Identity;
	}

	UBodySetup* SharedBodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex];
	if (!SharedBodySetup)
	{
		UE_LOGF(LogPhysicsControl, Warning, "GetPrimitiveTransform - no body setup");
		return FTransform::Identity;
	}

	FVector Scale3D(Scale);

	FTransform ManTM = FTransform::Identity;

	if (PrimType == EAggCollisionShape::Sphere)
	{
		FTransform PrimTM = ManTM * SharedBodySetup->AggGeom.SphereElems[PrimIndex].GetTransform();
		PrimTM.ScaleTranslation(Scale3D);
		return PrimTM * BoneTM;
	}
	else if (PrimType == EAggCollisionShape::Box)
	{
		FTransform PrimTM = ManTM * SharedBodySetup->AggGeom.BoxElems[PrimIndex].GetTransform();
		PrimTM.ScaleTranslation(Scale3D);
		return PrimTM * BoneTM;
	}
	else if (PrimType == EAggCollisionShape::Sphyl)
	{
		FTransform PrimTM = ManTM * SharedBodySetup->AggGeom.SphylElems[PrimIndex].GetTransform();
		PrimTM.ScaleTranslation(Scale3D);
		return PrimTM * BoneTM;
	}
	else if (PrimType == EAggCollisionShape::Convex)
	{
		FTransform PrimTM = ManTM * SharedBodySetup->AggGeom.ConvexElems[PrimIndex].GetTransform();
		PrimTM.ScaleTranslation(Scale3D);
		return PrimTM * BoneTM;
	}
	else if (PrimType == EAggCollisionShape::TaperedCapsule)
	{
		FTransform PrimTM = ManTM * SharedBodySetup->AggGeom.TaperedCapsuleElems[PrimIndex].GetTransform();
		PrimTM.ScaleTranslation(Scale3D);
		return PrimTM * BoneTM;
	}
	else if (PrimType == EAggCollisionShape::LevelSet)
	{
		FTransform PrimTM = ManTM * SharedBodySetup->AggGeom.LevelSetElems[PrimIndex].GetTransform();
		PrimTM.ScaleTranslation(Scale3D);
		return PrimTM * BoneTM;
	}
	else if (PrimType == EAggCollisionShape::SkinnedLevelSet)
	{
		FTransform PrimTM = ManTM * SharedBodySetup->AggGeom.SkinnedLevelSetElems[PrimIndex].GetTransform();
		PrimTM.ScaleTranslation(Scale3D);
		return PrimTM * BoneTM;
	}

	// Should never reach here
	check(0);
	return FTransform::Identity;
}

FColor UPhysicsControlAssetEditorSkeletalMeshComponent::GetPrimitiveColor(const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex) const
{
	if (!EditorData)
	{
		return BoneUnselectedColor;
	}

	UPhysicsControlAsset* PhysicsControlAsset = EditorData->PhysicsControlAsset;
	UPhysicsAsset* PhysicsAsset = PhysicsControlAsset ? PhysicsControlAsset->GetPhysicsAsset() : nullptr;
	if (!PhysicsAsset)
	{
		UE_LOGF(LogPhysicsControl, Warning, "GetPrimitiveColor - no physics asset");
		return BoneUnselectedColor;
	}

	if (!PhysicsAsset->SkeletalBodySetups.IsValidIndex(BodyIndex))
	{
		UE_LOGF(LogPhysicsControl, Warning, "GetPrimitiveColor - body index out of range");
		return BoneUnselectedColor;
	}

	UBodySetup* SharedBodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex];
	if (!SharedBodySetup)
	{
		UE_LOGF(LogPhysicsControl, Warning, "GetPrimitiveColor - no body setup");
		return BoneUnselectedColor;
	}

	FPhysicsControlAssetEditorData::FSelection Body(BodyIndex, PrimitiveType, PrimitiveIndex);

	static FName SelectionColorName(TEXT("SelectionColor"));
	const FSlateColor SelectionColor = FAppStyle::GetSlateColor(SelectionColorName);
	const FLinearColor SelectionColorLinear(SelectionColor.IsColorSpecified() ? SelectionColor.GetSpecifiedColor() : FLinearColor::White);
	const FColor ElemSelectedColor = SelectionColorLinear.ToFColor(true);
	const FColor ElemSelectedBodyColor = (SelectionColorLinear* 0.5f).ToFColor(true);

	bool bInBody = false;
	for (int32 i = 0; i<EditorData->SelectedBodies.Num(); ++i)
	{
		if (BodyIndex == EditorData->SelectedBodies[i].Index)
		{
			bInBody = true;
		}

		if (Body == EditorData->SelectedBodies[i] && !EditorData->bRunningSimulation)
		{
			return ElemSelectedColor;
		}
	}

	if (bInBody && !EditorData->bRunningSimulation)	//this primitive is in a body that's currently selected, but this primitive itself isn't selected
	{
		return ElemSelectedBodyColor;
	}
	if(PrimitiveType == EAggCollisionShape::TaperedCapsule)
	{
		return NoCollisionColor;
	}

	if (EditorData->bRunningSimulation)
	{
		const bool bIsSimulatedAtAll = SharedBodySetup->PhysicsType == PhysType_Simulated || (SharedBodySetup->PhysicsType == PhysType_Default && EditorData->EditorOptions->PhysicsBlend > 0.f);
		if (!bIsSimulatedAtAll)
		{
			return FixedColor;
		}
	}

	return BoneUnselectedColor;
}

UMaterialInterface* UPhysicsControlAssetEditorSkeletalMeshComponent::GetPrimitiveMaterial(const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex) const
{
	if (!EditorData)
	{
		return BoneUnselectedMaterial;
	}

	if (EditorData->bRunningSimulation)
	{
		return PrimitiveType == EAggCollisionShape::TaperedCapsule ? BoneNoCollisionMaterial : BoneUnselectedMaterial;
	}

	FPhysicsControlAssetEditorData::FSelection Body(BodyIndex, PrimitiveType, PrimitiveIndex);

	for (int32 i = 0; i < EditorData->SelectedBodies.Num(); ++i)
	{
		if (Body == EditorData->SelectedBodies[i] && !EditorData->bRunningSimulation)
		{
			return ElemSelectedMaterial;
		}
	}

	if (PrimitiveType == EAggCollisionShape::TaperedCapsule)
	{
		return BoneNoCollisionMaterial;
	}

	return BoneUnselectedMaterial;
}

void UPhysicsControlAssetEditorSkeletalMeshComponent::RefreshBoneTransforms(FActorComponentTickFunction* TickFunction)
{
	Super::RefreshBoneTransforms(TickFunction);

	// Horrible kludge, but we need to flip the buffer back here as we need to wait on the physics tick group.
	// However UDebugSkelMeshComponent passes NULL to force non-threaded work, which assumes a flip is needed straight away
	if (ShouldBlendPhysicsBones())
	{
		bNeedToFlipSpaceBaseBuffers = true;
		FinalizeBoneTransform();
		bNeedToFlipSpaceBaseBuffers = true;
	}
	UpdateSkinnedLevelSets();
}

void UPhysicsControlAssetEditorSkeletalMeshComponent::AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName)
{
	if (PreviewInstance != nullptr)
	{
		PreviewInstance->AddImpulseAtLocation(Impulse, Location, BoneName);
	}
}

bool UPhysicsControlAssetEditorSkeletalMeshComponent::ShouldCreatePhysicsState() const
{
	// @todo(chaos): the main physics scene is not running (and never runs) in the physics editor,
	// and currently this means it will accumulate body create/destroy commands every time
	// we hit "Simulate". Fix this!  However, we still need physics state for mouse ray hit detection 
	// on the bodies so we can't just avoid creating physics state...
	return Super::ShouldCreatePhysicsState();
}


void UPhysicsControlAssetEditorSkeletalMeshComponent::Grab(FName InBoneName, const FVector& Location, const FRotator& Rotation, bool bRotationConstrained)
{
	UPhysicsControlAssetEditorAnimInstance* PhysicsControlAssetEditorAnimInstance = 
		Cast<UPhysicsControlAssetEditorAnimInstance>(PreviewInstance);
	if (PhysicsControlAssetEditorAnimInstance != nullptr)
	{
		PhysicsControlAssetEditorAnimInstance->Grab(InBoneName, Location, Rotation, bRotationConstrained);
	}
}

void UPhysicsControlAssetEditorSkeletalMeshComponent::Ungrab()
{
	UPhysicsControlAssetEditorAnimInstance* PhysicsControlAssetEditorAnimInstance = 
		Cast<UPhysicsControlAssetEditorAnimInstance>(PreviewInstance);
	if (PhysicsControlAssetEditorAnimInstance != nullptr)
	{
		PhysicsControlAssetEditorAnimInstance->Ungrab();
	}
}

void UPhysicsControlAssetEditorSkeletalMeshComponent::UpdateHandleTransform(const FTransform& NewTransform)
{
	UPhysicsControlAssetEditorAnimInstance* PhysicsControlAssetEditorAnimInstance = 
		Cast<UPhysicsControlAssetEditorAnimInstance>(PreviewInstance);
	if (PhysicsControlAssetEditorAnimInstance != nullptr)
	{
		PhysicsControlAssetEditorAnimInstance->UpdateHandleTransform(NewTransform);
	}
}

void UPhysicsControlAssetEditorSkeletalMeshComponent::UpdateDriveSettings(
	bool bLinearSoft, float LinearStiffness, float LinearDamping)
{
	UPhysicsControlAssetEditorAnimInstance* PhysicsControlAssetEditorAnimInstance = 
		Cast<UPhysicsControlAssetEditorAnimInstance>(PreviewInstance);
	if (PhysicsControlAssetEditorAnimInstance != nullptr)
	{
		PhysicsControlAssetEditorAnimInstance->UpdateDriveSettings(bLinearSoft, LinearStiffness, LinearDamping);
	}
}

void UPhysicsControlAssetEditorSkeletalMeshComponent::CreateSimulationFloor(
	FBodyInstance* FloorBodyInstance, const FTransform& Transform)
{
	UPhysicsControlAssetEditorAnimInstance* PhysicsControlAssetEditorAnimInstance = 
		Cast<UPhysicsControlAssetEditorAnimInstance>(PreviewInstance);
	if (PhysicsControlAssetEditorAnimInstance != nullptr)
	{
		PhysicsControlAssetEditorAnimInstance->CreateSimulationFloor(FloorBodyInstance, Transform);
	}
}

void UPhysicsControlAssetEditorSkeletalMeshComponent::UpdateSkinnedLevelSets()
{
	UPhysicsAsset* const PhysicsAsset = GetPhysicsAsset();
	if (!PhysicsAsset)
	{
		return;
	}
	for (int32 i = 0; i < PhysicsAsset->SkeletalBodySetups.Num(); ++i)
	{
		const int32 BoneIndex = GetBoneIndex(PhysicsAsset->SkeletalBodySetups[i]->BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			FKAggregateGeom* const AggGeom = &PhysicsAsset->SkeletalBodySetups[i]->AggGeom;
			if (AggGeom)
			{
				for (FKSkinnedLevelSetElem& SkinnedLevelSet : AggGeom->SkinnedLevelSetElems)
				{
					if (SkinnedLevelSet.WeightedLevelSet().IsValid())
					{
						const TArray<FName>& UsedBoneNames = SkinnedLevelSet.WeightedLevelSet()->GetUsedBones();

						const FTransform RootTransformInv = GetBoneTransform(BoneIndex, FTransform::Identity).Inverse();
						TArray<FTransform> Transforms;
						Transforms.SetNum(UsedBoneNames.Num());

						for (int32 LocalIdx = 0; LocalIdx < UsedBoneNames.Num(); ++LocalIdx)
						{
							const int32 LocalBoneIndex = GetBoneIndex(UsedBoneNames[LocalIdx]);
							if (LocalBoneIndex != INDEX_NONE)
							{
								const FTransform BoneTransformTimesRootTransformInv = GetBoneTransform(LocalBoneIndex, RootTransformInv);
								Transforms[LocalIdx] = BoneTransformTimesRootTransformInv;
							}
							else
							{
								Transforms[LocalIdx] = RootTransformInv;
							}
						}

						SkinnedLevelSet.WeightedLevelSet()->DeformPoints(Transforms);
					}
				}
			}
		}
	}
}
