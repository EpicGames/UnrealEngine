// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsEngine/ShapeElem.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "PhysicsControlAssetEditorSkeletalMeshComponent.generated.h"

class FPrimitiveDrawInterface;
class UMaterialInterface;
class UMaterialInstanceDynamic;
class FPhysicsControlAssetEditorData;

// This is an internal component used only by the Physics Control Asset editor preview scene. It is
// marked Hidden so it does not appear in editor class pickers (e.g. the component class dropdown),
// as it is not intended to be selected or used outside of that editor.
UCLASS(Hidden)
class UPhysicsControlAssetEditorSkeletalMeshComponent : public UDebugSkelMeshComponent
{
	GENERATED_UCLASS_BODY()

	/** Data and methods shared across multiple classes */
	TSharedPtr<FPhysicsControlAssetEditorData> EditorData;

	// Draw colors

	FColor BoneUnselectedColor;
	FColor NoCollisionColor;
	FColor FixedColor;
	FColor ConstraintBone1Color;
	FColor ConstraintBone2Color;
	FColor HierarchyDrawColor;
	FColor AnimSkelDrawColor;
	float COMRenderSize;
	float InfluenceLineLength;
	FColor InfluenceLineColor;

	// Materials

	UPROPERTY(transient)
	TObjectPtr<UMaterialInstanceDynamic> ElemSelectedMaterial;
	UPROPERTY(transient)
	TObjectPtr<UMaterialInstanceDynamic> BoneUnselectedMaterial;
	UPROPERTY(transient)
	TObjectPtr<UMaterialInstanceDynamic> BoneNoCollisionMaterial;

	UPROPERTY(transient)
	TObjectPtr<UMaterialInterface> BoneMaterialHit;

	/** Mesh-space matrices showing state of just animation (ie before physics) - useful for debugging! */
	TArray<FTransform> AnimationSpaceBases;

	/** UDebugSkelMeshComponent interface */
	virtual TObjectPtr<UAnimPreviewInstance> CreatePreviewInstance() override;

	/** UPrimitiveComponent interface */
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual void AddImpulseAtLocation(FVector Impulse, FVector Location, FName BoneName = NAME_None) override;
	virtual bool ShouldCreatePhysicsState() const override;

	/** USkinnedMeshComponent interface */
	virtual void RefreshBoneTransforms(FActorComponentTickFunction* TickFunction = nullptr) override;
	
	/** Debug drawing */
	void DebugDraw(const FSceneView* View, FPrimitiveDrawInterface* PDI);

	/** Accessors/helper methods */
	FTransform GetPrimitiveTransform(const FTransform& BoneTM, const int32 BodyIndex, const EAggCollisionShape::Type PrimType, const int32 PrimIndex, const float Scale) const;
	FColor GetPrimitiveColor(const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex) const;
	UMaterialInterface* GetPrimitiveMaterial(const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex) const;

	/** Manipulator methods */
	void Grab(FName InBoneName, const FVector& Location, const FRotator& Rotation, bool bRotationConstrained);
	void Ungrab();
	void UpdateHandleTransform(const FTransform& NewTransform);
	void UpdateDriveSettings(bool bLinearSoft, float LinearStiffness, float LinearDamping);

	/** Sim setup */
	void CreateSimulationFloor(FBodyInstance* FloorBodyInstance, const FTransform& Transform);

public:
	virtual bool CanOverrideCollisionProfile() const override { return false;  }

private:
	void UpdateSkinnedLevelSets();
};
