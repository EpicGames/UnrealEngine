// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAssetEditorSkeletonTreeBuilder.h"
#include "SkeletonTreePhysicsControlBodyItem.h"
#include "SkeletonTreePhysicsControlShapeItem.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "IPersonaPreviewScene.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Misc/TextFilterExpressionEvaluator.h"

#define LOCTEXT_NAMESPACE "PhysicsControlAssetEditorSkeletonTreeBuilder"

FPhysicsControlAssetEditorSkeletonTreeBuilder::FPhysicsControlAssetEditorSkeletonTreeBuilder(
	UPhysicsAsset*                  InPhysicsAsset, 
	const FSkeletonTreeBuilderArgs& InSkeletonTreeBuilderArgs)
	: FSkeletonTreeBuilder(InSkeletonTreeBuilderArgs)
	, bShowBodies(true)
	, bShowKinematicBodies(true)
	, bShowSimulatedBodies(true)
	, bShowPrimitives(false)
	, PhysicsAsset(InPhysicsAsset)
{
}

void FPhysicsControlAssetEditorSkeletonTreeBuilder::Build(FSkeletonTreeBuilderOutput& Output)
{
	if (BuilderArgs.bShowBones)
	{
		AddBones(Output);
	}

	AddBodies(Output);

	if (BuilderArgs.bShowAttachedAssets)
	{
		AddAttachedAssets(Output);
	}
}

ESkeletonTreeFilterResult FPhysicsControlAssetEditorSkeletonTreeBuilder::FilterItem(const FSkeletonTreeFilterArgs& InArgs, const TSharedPtr<class ISkeletonTreeItem>& InItem)
{
	if (InItem->IsOfType<FSkeletonTreePhysicsControlBodyItem>() || InItem->IsOfType<FSkeletonTreePhysicsControlShapeItem>())
	{
		ESkeletonTreeFilterResult Result = ESkeletonTreeFilterResult::Shown;

		if (InArgs.TextFilter.IsValid())
		{
			if (InArgs.TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(InItem->GetRowItemName().ToString())))
			{
				Result = ESkeletonTreeFilterResult::ShownHighlighted;
			}
			else
			{
				Result = ESkeletonTreeFilterResult::Hidden;
			}
		}

		if (InItem->IsOfType<FSkeletonTreePhysicsControlBodyItem>())
		{
			bool bShouldHideBody = false;
			if (!bShowBodies)
			{
				bShouldHideBody = true;
			}
			else
			{
				if (UBodySetup* BodySetup = Cast<UBodySetup>(InItem->GetObject()))
				{
					if (BodySetup->PhysicsType == EPhysicsType::PhysType_Simulated && !bShowSimulatedBodies)
					{
						bShouldHideBody = true;
					}
					else if (BodySetup->PhysicsType == EPhysicsType::PhysType_Kinematic && !bShowKinematicBodies)
					{
						bShouldHideBody = true;
					}
				}
			}
			if (bShouldHideBody)
			{
				Result = ESkeletonTreeFilterResult::Hidden;
			}
		}
		else if (InItem->IsOfType<FSkeletonTreePhysicsControlShapeItem>())
		{
			if (!bShowPrimitives)
			{
				Result = ESkeletonTreeFilterResult::Hidden;
			}
		}

		return Result;
	}

	return FSkeletonTreeBuilder::FilterItem(InArgs, InItem);
}

void FPhysicsControlAssetEditorSkeletonTreeBuilder::AddBodies(FSkeletonTreeBuilderOutput& Output)
{
	if (!PreviewScenePtr.IsValid())
	{
		return;
	}
	UDebugSkelMeshComponent* PreviewMeshComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
	if (!PreviewMeshComponent || !PreviewMeshComponent->GetSkeletalMeshAsset())
	{
		return;
	}
	FReferenceSkeleton& RefSkeleton = PreviewMeshComponent->GetSkeletalMeshAsset()->GetRefSkeleton();

	for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetRawBoneNum(); ++BoneIndex)
	{
		const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);
		int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
		const FName ParentName = ParentIndex == INDEX_NONE ? NAME_None : RefSkeleton.GetBoneName(ParentIndex);

		bool bHasBodySetup = false;
		for (int32 BodySetupIndex = 0; BodySetupIndex < PhysicsAsset->SkeletalBodySetups.Num(); ++BodySetupIndex)
		{
			if (!ensure(PhysicsAsset->SkeletalBodySetups[BodySetupIndex]))
			{
				continue;
			}
			if (BoneName != PhysicsAsset->SkeletalBodySetups[BodySetupIndex]->BoneName)
			{
				continue;
			}

			USkeletalBodySetup* BodySetup = PhysicsAsset->SkeletalBodySetups[BodySetupIndex];

			bHasBodySetup = true;

			const FKAggregateGeom& AggGeom = PhysicsAsset->SkeletalBodySetups[BodySetupIndex]->AggGeom;

			bool bHasShapes = AggGeom.GetElementCount() > 0;

			if (bHasShapes)
			{
				Output.Add(MakeShared<FSkeletonTreePhysicsControlBodyItem>(
					BodySetup, BodySetupIndex, BoneName, true, bHasShapes, PhysicsAsset, 
					SkeletonTreePtr.Pin().ToSharedRef()), BoneName, "FSkeletonTreeBoneItem", true);

				int32 ShapeIndex;
				for (ShapeIndex = 0; ShapeIndex < AggGeom.SphereElems.Num(); ++ShapeIndex)
				{
					Output.Add(MakeShared<FSkeletonTreePhysicsControlShapeItem>(
						BodySetup, BoneName, BodySetupIndex, EAggCollisionShape::Sphere, ShapeIndex, 
						SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsControlBodyItem::GetTypeId());
				}

				for (ShapeIndex = 0; ShapeIndex < AggGeom.BoxElems.Num(); ++ShapeIndex)
				{
					Output.Add(MakeShared<FSkeletonTreePhysicsControlShapeItem>(
						BodySetup, BoneName, BodySetupIndex, EAggCollisionShape::Box, ShapeIndex, 
						SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsControlBodyItem::GetTypeId());
				}

				for (ShapeIndex = 0; ShapeIndex < AggGeom.SphylElems.Num(); ++ShapeIndex)
				{
					Output.Add(MakeShared<FSkeletonTreePhysicsControlShapeItem>(
						BodySetup, BoneName, BodySetupIndex, EAggCollisionShape::Sphyl, ShapeIndex, 
						SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsControlBodyItem::GetTypeId());
				}

				for (ShapeIndex = 0; ShapeIndex < AggGeom.ConvexElems.Num(); ++ShapeIndex)
				{
					Output.Add(MakeShared<FSkeletonTreePhysicsControlShapeItem>(
						BodySetup, BoneName, BodySetupIndex, EAggCollisionShape::Convex, ShapeIndex, 
						SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsControlBodyItem::GetTypeId());
				}

				for (ShapeIndex = 0; ShapeIndex < AggGeom.TaperedCapsuleElems.Num(); ++ShapeIndex)
				{
					Output.Add(MakeShared<FSkeletonTreePhysicsControlShapeItem>(
						BodySetup, BoneName, BodySetupIndex, EAggCollisionShape::TaperedCapsule, ShapeIndex, 
						SkeletonTreePtr.Pin().ToSharedRef()), BoneName, FSkeletonTreePhysicsControlBodyItem::GetTypeId());
				}
			}

			break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
