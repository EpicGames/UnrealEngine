// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAssetEditorTabSummoners.h"
#include "IDocumentation.h"
#include "PhysicsControlAssetEditor.h"
#include "PhysicsControlAsset.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "PhysicsControlAssetSetupDetailsCustomization.h"
#include "PhysicsControlAssetProfileDetailsCustomization.h"
#include "PhysicsControlAssetPreviewDetailsCustomization.h"
#include "PhysicsControlAssetInfoDetailsCustomization.h"

#define LOCTEXT_NAMESPACE "PhysicsControlAssetEditorTabSummoner"

//======================================================================================================================
FName FPhysicsControlAssetEditorSetupTabSummoner::TabName = FName(
	"PhysicsControlAssetEditorSetupTab");
FName FPhysicsControlAssetEditorProfileTabSummoner::TabName = FName(
	"PhysicsControlAssetEditorProfileTab");
FName FPhysicsControlAssetEditorPreviewTabSummoner::TabName = FName(
	"PhysicsControlAssetEditorPreviewTab");
FName FPhysicsControlAssetEditorControlSetsTabSummoner::TabName = FName(
	"PhysicsControlAssetEditorControlSetsTab");
FName FPhysicsControlAssetEditorBodyModifierSetsTabSummoner::TabName = FName(
	"PhysicsControlAssetEditorBodyModifierSetsTab");

//======================================================================================================================
FPhysicsControlAssetEditorSetupTabSummoner::FPhysicsControlAssetEditorSetupTabSummoner(
	TSharedPtr<FAssetEditorToolkit> InHostingApp, UPhysicsControlAsset* InPhysicsControlAsset)
	: FWorkflowTabFactory(FPhysicsControlAssetEditorSetupTabSummoner::TabName, InHostingApp)
	, PhysicsControlAsset(InPhysicsControlAsset)
{
	TabLabel = LOCTEXT("PhysicsControlAssetEditorSetupTabTitle", "Setup");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "PhysicsAssetEditor.Tabs.Profiles");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("PhysicsControlAssetEditorSetup", "Setup");
	ViewMenuTooltip = LOCTEXT("PhysicsControlAssetEditorSetup_ToolTip", "Shows the Control Asset Setup tab");
}

//======================================================================================================================
TSharedPtr<SToolTip> FPhysicsControlAssetEditorSetupTabSummoner::CreateTabToolTipWidget(
	const FWorkflowTabSpawnInfo& Info) const
{
	return IDocumentation::Get()->CreateToolTip(LOCTEXT(
		"PhysicsControlAssetEditorSetupToolTip", 
		"The Physics Control Asset Setup tab lets you edit the physics control asset relating to setting up controls."), 
		NULL, 
		TEXT("Shared/Editors/PhysicsControlAssetEditor"), TEXT("PhysicsControlAssetProfiles_Window"));
}

//======================================================================================================================
TSharedRef<SWidget> FPhysicsControlAssetEditorSetupTabSummoner::CreateTabBody(
	const FWorkflowTabSpawnInfo& Info) const
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bAllowSearch = false;

	FPropertyEditorModule& PropertyEditorModule = 
		FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	TWeakPtr<FPhysicsControlAssetEditor> PhysicsControlAssetEditor = 
		StaticCastSharedPtr<FPhysicsControlAssetEditor>(HostingApp.Pin());
	DetailsView->RegisterInstancedCustomPropertyLayout(
		UPhysicsControlAsset::StaticClass(), 
		FOnGetDetailCustomizationInstance::CreateStatic(
			&FPhysicsControlAssetSetupDetailsCustomization::MakeInstance, PhysicsControlAssetEditor));
	DetailsView->SetObject(PhysicsControlAsset.Get());
	return DetailsView;
}

//======================================================================================================================
FPhysicsControlAssetEditorProfileTabSummoner::FPhysicsControlAssetEditorProfileTabSummoner(
	TSharedPtr<FAssetEditorToolkit> InHostingApp, UPhysicsControlAsset* InPhysicsControlAsset)
	: FWorkflowTabFactory(FPhysicsControlAssetEditorProfileTabSummoner::TabName, InHostingApp)
	, PhysicsControlAsset(InPhysicsControlAsset)
{
	TabLabel = LOCTEXT("PhysicsControlAssetEditorProfileTabTitle", "Profiles");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "PhysicsAssetEditor.Tabs.Profiles");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("PhysicsControlAssetEditorProfile", "Profiles");
	ViewMenuTooltip = LOCTEXT("PhysicsControlAssetEditorProfile_ToolTip", "Shows the Control Asset Profile Edit tab");
}

//======================================================================================================================
TSharedPtr<SToolTip> FPhysicsControlAssetEditorProfileTabSummoner::CreateTabToolTipWidget(
	const FWorkflowTabSpawnInfo& Info) const
{
	return IDocumentation::Get()->CreateToolTip(LOCTEXT(
		"PhysicsControlAssetEditorProfileToolTip",
		"The Physics Control Asset Profile Edit tab lets you edit the physics control asset relating to setting up profiles."),
		NULL,
		TEXT("Shared/Editors/PhysicsControlAssetEditor"), TEXT("PhysicsControlAssetProfiles_Window"));
}

//======================================================================================================================
TSharedRef<SWidget> FPhysicsControlAssetEditorProfileTabSummoner::CreateTabBody(
	const FWorkflowTabSpawnInfo& Info) const
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bAllowSearch = false;

	FPropertyEditorModule& PropertyEditorModule =
		FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	TWeakPtr<FPhysicsControlAssetEditor> PhysicsControlAssetEditor =
		StaticCastSharedPtr<FPhysicsControlAssetEditor>(HostingApp.Pin());
	DetailsView->RegisterInstancedCustomPropertyLayout(
		UPhysicsControlAsset::StaticClass(), 
		FOnGetDetailCustomizationInstance::CreateStatic(
			&FPhysicsControlAssetProfileDetailsCustomization::MakeInstance, PhysicsControlAssetEditor));
	DetailsView->SetObject(PhysicsControlAsset.Get());
	return DetailsView;
}

//======================================================================================================================
FPhysicsControlAssetEditorPreviewTabSummoner::FPhysicsControlAssetEditorPreviewTabSummoner(
	TSharedPtr<FAssetEditorToolkit> InHostingApp, UPhysicsControlAsset* InPhysicsControlAsset)
	: FWorkflowTabFactory(FPhysicsControlAssetEditorPreviewTabSummoner::TabName, InHostingApp)
	, PhysicsControlAsset(InPhysicsControlAsset)
{
	TabLabel = LOCTEXT("PhysicsControlAssetEditorPreviewTabTitle", "Preview");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "PhysicsAssetEditor.Tabs.Profiles");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("PhysicsControlAssetEditorPreview", "Preview");
	ViewMenuTooltip = LOCTEXT("PhysicsControlAssetEditorPreview_ToolTip", "Shows the Control Asset Preview tab");
}

//======================================================================================================================
TSharedPtr<SToolTip> FPhysicsControlAssetEditorPreviewTabSummoner::CreateTabToolTipWidget(
	const FWorkflowTabSpawnInfo& Info) const
{
	return IDocumentation::Get()->CreateToolTip(LOCTEXT(
		"PhysicsControlAssetEditorPreviewToolTip",
		"The Physics Control Asset Preview tab lets you preview the physics control asset setup and profiles."),
		NULL,
		TEXT("Shared/Editors/PhysicsControlAssetEditor"), TEXT("PhysicsControlAssetProfiles_Window"));
}

//======================================================================================================================
TSharedRef<SWidget> FPhysicsControlAssetEditorPreviewTabSummoner::CreateTabBody(
	const FWorkflowTabSpawnInfo& Info) const
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bAllowSearch = false;

	FPropertyEditorModule& PropertyEditorModule =
		FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	TWeakPtr<FPhysicsControlAssetEditor> PhysicsControlAssetEditor =
		StaticCastSharedPtr<FPhysicsControlAssetEditor>(HostingApp.Pin());
	DetailsView->RegisterInstancedCustomPropertyLayout(
		UPhysicsControlAsset::StaticClass(), 
		FOnGetDetailCustomizationInstance::CreateStatic(
			&FPhysicsControlAssetPreviewDetailsCustomization::MakeInstance, PhysicsControlAssetEditor));
	DetailsView->SetObject(PhysicsControlAsset.Get());
	return DetailsView;
}

//======================================================================================================================
FPhysicsControlAssetEditorControlSetsTabSummoner::FPhysicsControlAssetEditorControlSetsTabSummoner(
	TSharedPtr<FAssetEditorToolkit> InHostingApp, UPhysicsControlAsset* InPhysicsControlAsset)
	: FWorkflowTabFactory(FPhysicsControlAssetEditorControlSetsTabSummoner::TabName, InHostingApp)
	, PhysicsControlAsset(InPhysicsControlAsset)
{
	TabLabel = LOCTEXT("PhysicsControlAssetEditorControlSetsTabTitle", "ControlSets");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "PhysicsAssetEditor.Tabs.Profiles");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("PhysicsControlAssetEditorControlSets", "ControlSets");
	ViewMenuTooltip = LOCTEXT("PhysicsControlAssetEditorControlSets_ToolTip", "Shows the Control Asset ControlSets tab");
}

//======================================================================================================================
TSharedPtr<SToolTip> FPhysicsControlAssetEditorControlSetsTabSummoner::CreateTabToolTipWidget(
	const FWorkflowTabSpawnInfo& Info) const
{
	return IDocumentation::Get()->CreateToolTip(LOCTEXT(
		"PhysicsControlAssetEditorControlSetsToolTip",
		"The Physics Control Asset Control Sets tab lets you see the control sets."),
		NULL,
		TEXT("Shared/Editors/PhysicsControlAssetEditor"), TEXT("PhysicsControlAssetProfiles_Window"));
}

//======================================================================================================================
TSharedRef<SWidget> FPhysicsControlAssetEditorControlSetsTabSummoner::CreateTabBody(
	const FWorkflowTabSpawnInfo& Info) const
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bAllowSearch = false;

	FPropertyEditorModule& PropertyEditorModule =
		FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	TWeakPtr<FPhysicsControlAssetEditor> PhysicsControlAssetEditor =
		StaticCastSharedPtr<FPhysicsControlAssetEditor>(HostingApp.Pin());
	DetailsView->RegisterInstancedCustomPropertyLayout(
		UPhysicsControlAsset::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(
			&FPhysicsControlAssetInfoDetailsCustomization::MakeInstance, 
			PhysicsControlAssetEditor,
			FPhysicsControlAssetInfoDetailsCustomization::EInfoType::Controls));
	DetailsView->SetObject(PhysicsControlAsset.Get());
	return DetailsView;
}

//======================================================================================================================
FPhysicsControlAssetEditorBodyModifierSetsTabSummoner::FPhysicsControlAssetEditorBodyModifierSetsTabSummoner(
	TSharedPtr<FAssetEditorToolkit> InHostingApp, UPhysicsControlAsset* InPhysicsControlAsset)
	: FWorkflowTabFactory(FPhysicsControlAssetEditorBodyModifierSetsTabSummoner::TabName, InHostingApp)
	, PhysicsControlAsset(InPhysicsControlAsset)
{
	TabLabel = LOCTEXT("PhysicsControlAssetEditorBodyModifierSetsTabTitle", "BodyModifierSets");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "PhysicsAssetEditor.Tabs.Profiles");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("PhysicsControlAssetEditorBodyModifierSets", "BodyModifierSets");
	ViewMenuTooltip = LOCTEXT("PhysicsControlAssetEditorBodyModifierSets_ToolTip", "Shows the Control Asset BodyModifierSets tab");
}

//======================================================================================================================
TSharedPtr<SToolTip> FPhysicsControlAssetEditorBodyModifierSetsTabSummoner::CreateTabToolTipWidget(
	const FWorkflowTabSpawnInfo& Info) const
{
	return IDocumentation::Get()->CreateToolTip(LOCTEXT(
		"PhysicsControlAssetEditorBodyModifierSetsToolTip",
		"The Physics Control Asset Control Sets tab lets you see the body modifier sets."),
		NULL,
		TEXT("Shared/Editors/PhysicsControlAssetEditor"), TEXT("PhysicsControlAssetProfiles_Window"));
}

//======================================================================================================================
TSharedRef<SWidget> FPhysicsControlAssetEditorBodyModifierSetsTabSummoner::CreateTabBody(
	const FWorkflowTabSpawnInfo& Info) const
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bAllowSearch = false;

	FPropertyEditorModule& PropertyEditorModule =
		FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	TWeakPtr<FPhysicsControlAssetEditor> PhysicsControlAssetEditor =
		StaticCastSharedPtr<FPhysicsControlAssetEditor>(HostingApp.Pin());
	DetailsView->RegisterInstancedCustomPropertyLayout(
		UPhysicsControlAsset::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(
			&FPhysicsControlAssetInfoDetailsCustomization::MakeInstance,
			PhysicsControlAssetEditor,
			FPhysicsControlAssetInfoDetailsCustomization::EInfoType::BodyModifiers));
	DetailsView->SetObject(PhysicsControlAsset.Get());
	return DetailsView;
}


#undef LOCTEXT_NAMESPACE