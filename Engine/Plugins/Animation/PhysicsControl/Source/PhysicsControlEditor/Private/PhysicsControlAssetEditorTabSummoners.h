// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class UPhysicsControlAsset;

//======================================================================================================================
struct FPhysicsControlAssetEditorSetupTabSummoner : public FWorkflowTabFactory
{
public:
	static FName TabName;

	FPhysicsControlAssetEditorSetupTabSummoner(
		TSharedPtr<FAssetEditorToolkit> InHostingApp, UPhysicsControlAsset* InPhysicsControlAsset);

	/** FWorkflowTabFactory interface */
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;

	/** Reference to our Physics Asset */
	TWeakObjectPtr<UPhysicsControlAsset> PhysicsControlAsset;
};

//======================================================================================================================
struct FPhysicsControlAssetEditorProfileTabSummoner : public FWorkflowTabFactory
{
public:
	static FName TabName;

	FPhysicsControlAssetEditorProfileTabSummoner(
		TSharedPtr<FAssetEditorToolkit> InHostingApp, UPhysicsControlAsset* InPhysicsControlAsset);

	/** FWorkflowTabFactory interface */
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;

	/** Reference to our Physics Asset */
	TWeakObjectPtr<UPhysicsControlAsset> PhysicsControlAsset;
};

//======================================================================================================================
struct FPhysicsControlAssetEditorPreviewTabSummoner : public FWorkflowTabFactory
{
public:
	static FName TabName;

	FPhysicsControlAssetEditorPreviewTabSummoner(
		TSharedPtr<FAssetEditorToolkit> InHostingApp, UPhysicsControlAsset* InPhysicsControlAsset);

	/** FWorkflowTabFactory interface */
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;

	/** Reference to our Physics Asset */
	TWeakObjectPtr<UPhysicsControlAsset> PhysicsControlAsset;
};

//======================================================================================================================
struct FPhysicsControlAssetEditorControlSetsTabSummoner : public FWorkflowTabFactory
{
public:
	static FName TabName;

	FPhysicsControlAssetEditorControlSetsTabSummoner(
		TSharedPtr<FAssetEditorToolkit> InHostingApp, UPhysicsControlAsset* InPhysicsControlAsset);

	/** FWorkflowTabFactory interface */
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;

	/** Reference to our Physics Asset */
	TWeakObjectPtr<UPhysicsControlAsset> PhysicsControlAsset;
};

//======================================================================================================================
struct FPhysicsControlAssetEditorBodyModifierSetsTabSummoner : public FWorkflowTabFactory
{
public:
	static FName TabName;

	FPhysicsControlAssetEditorBodyModifierSetsTabSummoner(
		TSharedPtr<FAssetEditorToolkit> InHostingApp, UPhysicsControlAsset* InPhysicsControlAsset);

	/** FWorkflowTabFactory interface */
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;

	/** Reference to our Physics Asset */
	TWeakObjectPtr<UPhysicsControlAsset> PhysicsControlAsset;
};
