// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PhysicsControlAssetEditorToolMenuContext.generated.h"

class FPhysicsControlAssetEditor;

UCLASS()
class UPhysicsControlAssetEditorToolMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<FPhysicsControlAssetEditor> PhysicsControlAssetEditor;
};
