// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAssetFactory.h"
#include "PhysicsControlAsset.h"

//======================================================================================================================

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsControlAssetFactory)
UPhysicsControlAssetFactory::UPhysicsControlAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UPhysicsControlAsset::StaticClass();
	bCreateNew = true;
	bEditAfterNew = false;
}

//======================================================================================================================
UObject* UPhysicsControlAssetFactory::FactoryCreateNew(
	UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UPhysicsControlAsset>(InParent, Class, Name, Flags, Context);
}
