// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "PhysicsControlAssetFactory.generated.h"

UCLASS()
class UPhysicsControlAssetFactory : public UFactory
{
	GENERATED_BODY()
public:
	UPhysicsControlAssetFactory(const FObjectInitializer& ObjectInitializer);
	UObject* FactoryCreateNew(
		UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn);
};
