// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAssetInfoDetailsCustomization.h"

#include "PhysicsControlAssetEditor.h"
#include "PhysicsControlAsset.h"
#include "PhysicsControlAssetEditorData.h"
#include "PhysicsControlAssetEditorSkeletalMeshComponent.h"
#include "PhysicsControlOperatorNameGeneration.h"
#include "PhysicsControlNameRecords.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailPropertyRow.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "PhysicsControlAssetInfoDetailsCustomization"

//======================================================================================================================
TSharedRef<IDetailCustomization> FPhysicsControlAssetInfoDetailsCustomization::MakeInstance(
	TWeakPtr<FPhysicsControlAssetEditor> InPhysicsControlAssetEditor, EInfoType InfoType)
{
	return MakeShared<FPhysicsControlAssetInfoDetailsCustomization>(InPhysicsControlAssetEditor, InfoType);
}

//======================================================================================================================
void FPhysicsControlAssetInfoDetailsCustomization::CustomizeDetails(
	const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder)
{
	DetailLayoutBuilderWeak = InDetailBuilder;
	FPhysicsControlAssetInfoDetailsCustomization::CustomizeDetails(*InDetailBuilder);

	if (TSharedPtr<FPhysicsControlAssetEditor> PCAE = PhysicsControlAssetEditor.Pin())
	{
		if (UPhysicsControlAsset* PhysicsControlAsset = PCAE->GetEditorData()->PhysicsControlAsset.Get())
		{
			PhysicsControlAsset->OnControlAssetCompiled().AddSP(
				this, &FPhysicsControlAssetInfoDetailsCustomization::OnControlAssetCompiled);
		}
	}
}

//======================================================================================================================
void FPhysicsControlAssetInfoDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	TArray<FName> CategoryNames;
	DetailLayoutBuilder.GetCategoryNames(CategoryNames);
	for (FName Category : CategoryNames)
	{
		DetailLayoutBuilder.HideCategory(Category);
	}
	DetailLayoutBuilder.HideCategory(TEXT("Actions"));

	TSharedPtr<const FPhysicsControlAssetEditor> PCAE = PhysicsControlAssetEditor.Pin();
	if (!PCAE)
	{
		return;
	}
	TSharedPtr<const FPhysicsControlAssetEditorData> EditorData = PCAE->GetEditorData();
	const UPhysicsControlAsset* PCA = EditorData->PhysicsControlAsset.Get();
	if (!PCA)
	{
		return;
	}
	const UPhysicsAsset* PA = PCA->GetPhysicsAsset();
	const USkeletalMeshComponent* SKMC = EditorData->EditorSkelComp.Get();
	if (!PA || !SKMC)
	{
		return;
	}
	const USkeletalMesh* SKM = SKMC->GetSkeletalMeshAsset();
	if (!SKM)
	{
		return;
	}
	const FReferenceSkeleton& RefSkeleton = SKM->GetRefSkeleton();

	// Process the asset to get all the controls/body modifiers and sets
	TMap<FName, FPhysicsControlLimbBones> LimbBones = 
		UE::PhysicsControl::GetLimbBones(PCA->CharacterSetupData.LimbSetupData,RefSkeleton, PA);

	TSet<FName> BodyModifierNames;
	TSet<FName> ControlNames;
	FPhysicsControlNameRecords NameRecords;

	UE::PhysicsControl::CollectOperatorNames(
		PCA->CharacterSetupData, PCA->AdditionalControlsAndModifiers,
		LimbBones, RefSkeleton, PA, BodyModifierNames, ControlNames, NameRecords);

	UE::PhysicsControl::CreateAdditionalSets(
		PCA->AdditionalSets, BodyModifierNames, ControlNames, NameRecords);

	TMap<FName, TArray<FName>>& SetsToShow = 
		(InfoType == EInfoType::Controls) ? NameRecords.ControlSets : NameRecords.BodyModifierSets;

	SetsToShow.KeySort([](FName A, FName B) { return A.ToString() < B.ToString(); });

	for (const TPair<FName, TArray<FName>>& Set : SetsToShow)
	{
		const FName SetName = Set.Key;
		const TArray<FName>& Names = Set.Value;

		FText SectionName = FText::FromName(SetName);
		IDetailCategoryBuilder& DetailCategoryBuilder = 
			DetailLayoutBuilder.EditCategory(SetName, FText::FromName(SetName));
		DetailCategoryBuilder.InitiallyCollapsed(true);

		for (FName Name : Names)
		{
			FDetailWidgetRow& Row = DetailCategoryBuilder.AddCustomRow(
				FText::Format(LOCTEXT("NameRow", "Name_{0}"), FText::FromName(Name)));
			Row.WholeRowContent()
				[
					SNew(STextBlock).Text(FText::FromName(Name))
				];
		}
	}
}

//======================================================================================================================
void FPhysicsControlAssetInfoDetailsCustomization::OnControlAssetCompiled(bool bProfileListChanged)
{
	if (DetailLayoutBuilderWeak.IsValid())
	{
		if (IDetailLayoutBuilder* DetailLayoutBuilder = DetailLayoutBuilderWeak.Pin().Get())
		{
			DetailLayoutBuilder->ForceRefreshDetails();
		}
	}
}


#undef LOCTEXT_NAMESPACE
