// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Types/SlateEnums.h"

class FPhysicsControlAssetEditor;
class IDetailLayoutBuilder;
class FUICommandInfo;
class SWidget;
class IPropertyHandle;
class FUICommandList;
class SEditableTextBox;

class FPhysicsControlAssetInfoDetailsCustomization : public IDetailCustomization
{
public:
	enum class EInfoType : uint8
	{
		Controls,
		BodyModifiers
	};

	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance(
		TWeakPtr<FPhysicsControlAssetEditor> InPhysicsControlAssetEditor, EInfoType InfoType);

	FPhysicsControlAssetInfoDetailsCustomization(
		TWeakPtr<FPhysicsControlAssetEditor> InPhysicsControlAssetEditor,
		EInfoType                            InInfoType)
		: PhysicsControlAssetEditor(InPhysicsControlAssetEditor)
		, InfoType(InInfoType)
	{}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder) override;
	//~ End IDetailCustomization

protected:
	void OnControlAssetCompiled(bool bProfileListChanged);

protected:
	TWeakPtr<FPhysicsControlAssetEditor> PhysicsControlAssetEditor;

	TWeakPtr<IDetailLayoutBuilder> DetailLayoutBuilderWeak;

	EInfoType InfoType;
};
