// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAssetPreviewDetailsCustomization.h"

#include "PhysicsControlAssetEditor.h"
#include "PhysicsControlAssetEditorData.h"
#include "PhysicsControlAssetEditorCommands.h"
#include "PhysicsControlAsset.h"
#include "PhysicsControlComponent.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailPropertyRow.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "PhysicsControlAssetPreviewDetailsCustomization"

//======================================================================================================================
TSharedRef<IDetailCustomization> FPhysicsControlAssetPreviewDetailsCustomization::MakeInstance(
	TWeakPtr<FPhysicsControlAssetEditor> InPhysicsControlAssetEditor)
{
	return MakeShared<FPhysicsControlAssetPreviewDetailsCustomization>(InPhysicsControlAssetEditor);
}

//======================================================================================================================
FPhysicsControlAssetPreviewDetailsCustomization::FPhysicsControlAssetPreviewDetailsCustomization(
	TWeakPtr<FPhysicsControlAssetEditor> InPhysicsControlAssetEditor)
	: PhysicsControlAssetEditor(InPhysicsControlAssetEditor)
{
}

//======================================================================================================================
void FPhysicsControlAssetPreviewDetailsCustomization::CustomizeDetails(
	const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder)
{
	DetailLayoutBuilderWeak = InDetailBuilder;
	FPhysicsControlAssetPreviewDetailsCustomization::CustomizeDetails(*InDetailBuilder);

	if (TSharedPtr<FPhysicsControlAssetEditor> PCAE = PhysicsControlAssetEditor.Pin())
	{
		if (UPhysicsControlAsset* PhysicsControlAsset = PCAE->GetEditorData()->PhysicsControlAsset.Get())
		{
			PhysicsControlAsset->OnControlAssetCompiled().AddSP(
				this, &FPhysicsControlAssetPreviewDetailsCustomization::OnControlAssetCompiled);
		}
	}
}

//======================================================================================================================
void FPhysicsControlAssetPreviewDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	TArray<FName> CategoryNames;
	DetailLayoutBuilder.GetCategoryNames(CategoryNames);
	for (FName Category : CategoryNames)
	{
		DetailLayoutBuilder.HideCategory(Category);
	}
	DetailLayoutBuilder.HideCategory(TEXT("Actions"));

	TSharedPtr<FPhysicsControlAssetEditor> PCAE = PhysicsControlAssetEditor.Pin();
	if (!PCAE)
	{
		return;
	}

	TSharedPtr<FPhysicsControlAssetEditorData> EditorData = PCAE->GetEditorData();
	UPhysicsControlAsset* PCA = EditorData->PhysicsControlAsset.Get();
	if (PCA)
	{
		IDetailCategoryBuilder& DetailCategoryBuilder = DetailLayoutBuilder.EditCategory(TEXT("Preview Profiles"));

		for (const TPair<FName, FPhysicsControlControlAndModifierUpdates>& ProfilePair : PCA->Profiles)
		{
			const FName ProfileName = ProfilePair.Key;
			FDetailWidgetRow& Row = DetailCategoryBuilder.AddCustomRow(FText::FromName(ProfileName));
			Row.WholeRowContent()
				[
					SNew(SButton)
						.Text(FText::FromName(ProfileName))
						.OnClicked(this, &FPhysicsControlAssetPreviewDetailsCustomization::InvokeControlProfile, ProfileName)
				];
		}
	}
}

//======================================================================================================================
FReply FPhysicsControlAssetPreviewDetailsCustomization::InvokeControlProfile(FName ProfileName)
{
	if (TSharedPtr<FPhysicsControlAssetEditor> PCAE = PhysicsControlAssetEditor.Pin())
	{
		PCAE->InvokeControlProfile(ProfileName);
	}
	return FReply::Handled();
}

//======================================================================================================================
void FPhysicsControlAssetPreviewDetailsCustomization::OnControlAssetCompiled(bool bProfileListChanged)
{
	if (bProfileListChanged)
	{
		if (DetailLayoutBuilderWeak.IsValid())
		{
			if (IDetailLayoutBuilder* DetailLayoutBuilder = DetailLayoutBuilderWeak.Pin().Get())
			{
				DetailLayoutBuilder->ForceRefreshDetails();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
