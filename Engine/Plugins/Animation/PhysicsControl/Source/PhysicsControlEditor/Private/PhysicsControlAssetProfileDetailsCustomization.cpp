// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAssetProfileDetailsCustomization.h"

#include "PhysicsControlAssetEditor.h"
#include "PhysicsControlAsset.h"
#include "PhysicsControlComponent.h"
#include "PhysicsControlAssetEditorData.h"

#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "PhysicsControlAssetProfileDetailsCustomization"

//======================================================================================================================
TSharedRef<IDetailCustomization> FPhysicsControlAssetProfileDetailsCustomization::MakeInstance(
	TWeakPtr<FPhysicsControlAssetEditor> InPhysicsControlAssetEditor)
{
	return MakeShared<FPhysicsControlAssetProfileDetailsCustomization>(InPhysicsControlAssetEditor);
}

//======================================================================================================================
void FPhysicsControlAssetProfileDetailsCustomization::CustomizeDetails(
	const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder)
{
	DetailLayoutBuilderWeak = InDetailBuilder;
	FPhysicsControlAssetProfileDetailsCustomization::CustomizeDetails(*InDetailBuilder);
}

//======================================================================================================================
void FPhysicsControlAssetProfileDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	DetailLayoutBuilder.HideCategory(TEXT("PreviewMesh"));
	DetailLayoutBuilder.HideCategory(TEXT("Actions"));
	DetailLayoutBuilder.HideCategory(TEXT("Inheritance"));
	DetailLayoutBuilder.HideCategory(TEXT("Setup"));
	DetailLayoutBuilder.HideCategory(TEXT("SetupEditing"));

	const TSharedRef<IPropertyHandle> ProfilesProperty = DetailLayoutBuilder.GetProperty(
		GET_MEMBER_NAME_CHECKED(UPhysicsControlAsset, MyProfiles));

	ProfilesProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(
		this, &FPhysicsControlAssetProfileDetailsCustomization::OnProfilesChanged));

	ProfilesProperty->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(
			this, &FPhysicsControlAssetProfileDetailsCustomization::OnProfileDetailsChanged));
}

//======================================================================================================================
// This is called when a parameter in one of the profiles changes
void FPhysicsControlAssetProfileDetailsCustomization::OnProfileDetailsChanged()
{
	if (TSharedPtr<FPhysicsControlAssetEditor> PCAE = PhysicsControlAssetEditor.Pin())
	{
		UPhysicsControlAsset* PhysicsControlAsset = PCAE->GetEditorData()->PhysicsControlAsset.Get();
		if (!PhysicsControlAsset)
		{
			return;
		}
		if (PhysicsControlAsset->bAutoCompileProfiles)
		{
			TArray<FName> DirtyProfiles;
			if (PhysicsControlAsset->bAutoInvokeProfiles && PCAE->IsRunningSimulation())
			{
				DirtyProfiles = PhysicsControlAsset->GetDirtyProfiles();
			}

			PhysicsControlAsset->Compile();

			for (FName DirtyProfile : DirtyProfiles)
			{
				InvokeControlProfile(DirtyProfile);
			}
		}
	}
}

//======================================================================================================================
// This is called when the list of profiles changes (i.e. profile added/removed)
void FPhysicsControlAssetProfileDetailsCustomization::OnProfilesChanged()
{
	if (TSharedPtr<FPhysicsControlAssetEditor> PCAE = PhysicsControlAssetEditor.Pin())
	{
		UPhysicsControlAsset* PhysicsControlAsset = PCAE->GetEditorData()->PhysicsControlAsset.Get();
		if (PhysicsControlAsset && PhysicsControlAsset->bAutoCompileProfiles)
		{
			PhysicsControlAsset->Compile();
		}
	}
}

//======================================================================================================================
FReply FPhysicsControlAssetProfileDetailsCustomization::InvokeControlProfile(FName ProfileName)
{
	if (TSharedPtr<FPhysicsControlAssetEditor> PCAE = PhysicsControlAssetEditor.Pin())
	{
		// Danny TODO handle the RBWC mode
		UPhysicsControlComponent* PCC = PCAE->GetEditorData()->PhysicsControlComponent.Get();
		if (PCC)
		{
			PCC->InvokeControlProfile(ProfileName);
		}
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
