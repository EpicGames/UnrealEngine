// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Types/SlateEnums.h"
#include "Input/Reply.h"

class UPhysicsControlAsset;
class FPhysicsControlAssetEditor;

class IDetailLayoutBuilder;
class FUICommandInfo;
class SWidget;
class IPropertyHandle;
class FUICommandList;
class SEditableTextBox;

struct FPhysicsControlControlAndModifierUpdates;

class FPhysicsControlAssetProfileDetailsCustomization : public IDetailCustomization
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance(TWeakPtr<FPhysicsControlAssetEditor> InPhysicsControlAssetEditor);

	FPhysicsControlAssetProfileDetailsCustomization(TWeakPtr<FPhysicsControlAssetEditor> InPhysicsControlAssetEditor)
		: PhysicsControlAssetEditor(InPhysicsControlAssetEditor)
	{}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailBuilder) override;
	//~ End IDetailCustomization

protected:
	void OnProfilesChanged();
	void OnProfileDetailsChanged();

protected:
	FReply InvokeControlProfile(FName ProfileName);

	TWeakPtr<FPhysicsControlAssetEditor> PhysicsControlAssetEditor;

	TWeakPtr<IDetailLayoutBuilder> DetailLayoutBuilderWeak;
};
