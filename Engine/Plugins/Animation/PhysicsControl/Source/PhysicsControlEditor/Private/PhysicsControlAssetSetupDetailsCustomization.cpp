// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlAssetSetupDetailsCustomization.h"

#include "PhysicsControlAssetEditor.h"
#include "PhysicsControlAsset.h"
#include "PhysicsControlAssetEditorData.h"

#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "PhysicsControlAssetSetupDetailsCustomization"

//======================================================================================================================
TSharedRef<IDetailCustomization> FPhysicsControlAssetSetupDetailsCustomization::MakeInstance(
	TWeakPtr<FPhysicsControlAssetEditor> InPhysicsControlAssetEditor)
{
	return MakeShared<FPhysicsControlAssetSetupDetailsCustomization>(InPhysicsControlAssetEditor);
}

//======================================================================================================================
void FPhysicsControlAssetSetupDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	DetailLayoutBuilder.HideCategory(TEXT("Profiles"));
	DetailLayoutBuilder.HideCategory(TEXT("ProfileEditing"));

	TArray<TSharedRef<IPropertyHandle>> Properties;
	Properties.Push(DetailLayoutBuilder.GetProperty(
		GET_MEMBER_NAME_CHECKED(UPhysicsControlAsset, MyCharacterSetupData)));

	Properties.Push(DetailLayoutBuilder.GetProperty(
		GET_MEMBER_NAME_CHECKED(UPhysicsControlAsset, MyAdditionalControlsAndModifiers)));

	Properties.Push(DetailLayoutBuilder.GetProperty(
		GET_MEMBER_NAME_CHECKED(UPhysicsControlAsset, MyAdditionalSets)));

	Properties.Push(DetailLayoutBuilder.GetProperty(
		GET_MEMBER_NAME_CHECKED(UPhysicsControlAsset, MyInitialControlAndModifierUpdates)));

	for (TSharedRef<IPropertyHandle> Property : Properties)
	{
		Property->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(
			this, &FPhysicsControlAssetSetupDetailsCustomization::OnSetupChanged));

		Property->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(
			this, &FPhysicsControlAssetSetupDetailsCustomization::OnSetupDetailsChanged));
	}
}

//======================================================================================================================
// This is called when a property within one of the setup data structs changes (e.g. a field on
// MyCharacterSetupData, MyAdditionalControlsAndModifiers, MyAdditionalSets, or
// MyInitialControlAndModifierUpdates).
void FPhysicsControlAssetSetupDetailsCustomization::OnSetupDetailsChanged()
{
	if (TSharedPtr<FPhysicsControlAssetEditor> PCAE = PhysicsControlAssetEditor.Pin())
	{
		UPhysicsControlAsset* PhysicsControlAsset = PCAE->GetEditorData()->PhysicsControlAsset.Get();
		if (!PhysicsControlAsset)
		{
			return;
		}

		bool bNeedToReinitialize = false;
		if (PhysicsControlAsset->bAutoReinitSetup && PCAE->IsRunningSimulation())
		{
			bNeedToReinitialize = PhysicsControlAsset->IsSetupDirty();
		}

		if (PhysicsControlAsset->bAutoCompileSetup)
		{
			PhysicsControlAsset->Compile();
		}

		if (bNeedToReinitialize)
		{
			PCAE->RecreateControlsAndModifiers();
			if (PhysicsControlAsset->bAutoInvokeProfileAfterSetup)
			{
				PCAE->ReinvokeControlProfile();
			}
		}
	}
}

//======================================================================================================================
// This is called when one of the registered setup properties is replaced wholesale (e.g. an entry
// added/removed/reordered in an array, or a struct value reset).
void FPhysicsControlAssetSetupDetailsCustomization::OnSetupChanged()
{
	OnSetupDetailsChanged();
}
#undef LOCTEXT_NAMESPACE
