// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Types/SlateEnums.h"

class FPhysicsControlAssetEditor;
class IDetailLayoutBuilder;
class FUICommandInfo;
class SWidget;
class IPropertyHandle;
class FUICommandList;
class SEditableTextBox;

class FPhysicsControlAssetSetupDetailsCustomization : public IDetailCustomization
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance(TWeakPtr<FPhysicsControlAssetEditor> InPhysicsControlAssetEditor);

	FPhysicsControlAssetSetupDetailsCustomization(TWeakPtr<FPhysicsControlAssetEditor> InPhysicsControlAssetEditor)
		: PhysicsControlAssetEditor(InPhysicsControlAssetEditor)
	{}

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

protected:
	void OnSetupChanged();
	void OnSetupDetailsChanged();

protected:
	TWeakPtr<FPhysicsControlAssetEditor> PhysicsControlAssetEditor;
};
