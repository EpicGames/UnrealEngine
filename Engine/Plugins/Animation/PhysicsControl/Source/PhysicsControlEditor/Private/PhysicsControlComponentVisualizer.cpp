// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlComponentVisualizer.h"
#include "PhysicsControlComponent.h"

//======================================================================================================================
void FPhysicsControlComponentVisualizer::DrawVisualization(
	const UActorComponent*   Component,
	const FSceneView*        View,
	FPrimitiveDrawInterface* PDI)
{
	if (const UPhysicsControlComponent* PCC = Cast<const UPhysicsControlComponent>(Component))
	{
		PCC->DebugDraw(PDI);
	}
}
