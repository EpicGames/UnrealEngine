// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ComponentVisualizer.h"

#define UE_API PHYSICSCONTROLEDITOR_API

class FPhysicsControlComponentVisualizer : public FComponentVisualizer
{
private:
	UE_API virtual void DrawVisualization(
		const UActorComponent*   Component, 
		const FSceneView*        View, 
		FPrimitiveDrawInterface* PDI) override;
};

#undef UE_API
