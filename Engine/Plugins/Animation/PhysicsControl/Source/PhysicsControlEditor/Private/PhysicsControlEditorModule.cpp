// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsControlEditorModule.h"
#include "PhysicsControlComponent.h"
#include "PhysicsControlComponentVisualizer.h"
#include "OperatorViewer/OperatorViewer.h"
#include "PhysicsControlAssetEditorEditMode.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "PhysicsControlModule"

//======================================================================================================================
void FPhysicsControlEditorModule::StartupModule()
{
	FEditorModeRegistry::Get().RegisterMode<FPhysicsControlAssetEditorEditMode>(
		FPhysicsControlAssetEditorEditMode::ModeName, 
		LOCTEXT("PhysicsControlAssetEditorMode", "PhysicsControlAsset"), 
		FSlateIcon(), false);

	if (GUnrealEd)
	{
		TSharedPtr<FPhysicsControlComponentVisualizer> Visualizer = MakeShared<FPhysicsControlComponentVisualizer>();
		GUnrealEd->RegisterComponentVisualizer(UPhysicsControlComponent::StaticClass()->GetFName(), Visualizer);
		// This call should maybe be inside the RegisterComponentVisualizer call above, but since it's not,
		// we'll put it here.
		Visualizer->OnRegister();
		VisualizersToUnregisterOnShutdown.Add(UPhysicsControlComponent::StaticClass()->GetFName());
	}

	if (!EditorInterface)
	{
		EditorInterface = new FPhysicsControlOperatorViewer;
	}

	if (EditorInterface)
	{
		EditorInterface->Startup();
		IModularFeatures::Get().RegisterModularFeature(IPhysicsControlOperatorViewerInterface::GetModularFeatureName(), EditorInterface);
	}
}

//======================================================================================================================
void FPhysicsControlEditorModule::ShutdownModule()
{
	if (EditorInterface)
	{
		EditorInterface->Shutdown();
		IModularFeatures::Get().UnregisterModularFeature(IPhysicsControlOperatorViewerInterface::GetModularFeatureName(), EditorInterface);
		delete EditorInterface;
		EditorInterface = nullptr;
	}
	
	FEditorModeRegistry::Get().UnregisterMode(FPhysicsControlAssetEditorEditMode::ModeName);

	if (GUnrealEd)
	{
		for (const FName& Name : VisualizersToUnregisterOnShutdown)
		{
			GUnrealEd->UnregisterComponentVisualizer(Name);
		}
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FPhysicsControlEditorModule, PhysicsControlEditor)
