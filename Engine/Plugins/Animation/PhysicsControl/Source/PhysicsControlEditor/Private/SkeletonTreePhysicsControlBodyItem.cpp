// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreePhysicsControlBodyItem.h"
#include "Styling/AppStyle.h"
#include "PhysicsAssetRenderUtils.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"

#define LOCTEXT_NAMESPACE "FSkeletonTreePhysicsControlBodyItem"

FSkeletonTreePhysicsControlBodyItem::FSkeletonTreePhysicsControlBodyItem(USkeletalBodySetup* InBodySetup, int32 InBodySetupIndex, const FName& InBoneName, bool bInHasBodySetup, bool bInHasShapes, class UPhysicsAsset* const InPhysicsAsset, const TSharedRef<class ISkeletonTree>& InSkeletonTree)
	: FSkeletonTreePhysicsControlItem(InPhysicsAsset, InSkeletonTree)
	, BodySetup(InBodySetup)
	, BodySetupIndex(InBodySetupIndex)
	, bHasBodySetup(bInHasBodySetup)
	, bHasShapes(bInHasShapes)
{
	DisplayName = InBoneName;
}

UObject* FSkeletonTreePhysicsControlBodyItem::GetObject() const
{
	return BodySetup;
}

void FSkeletonTreePhysicsControlBodyItem::OnToggleItemDisplayed(ECheckBoxState InCheckboxState)
{
	if (FPhysicsAssetRenderSettings* RenderSettings = GetRenderSettings())
	{
		RenderSettings->ToggleShowBody(BodySetupIndex);
	}
}

ECheckBoxState FSkeletonTreePhysicsControlBodyItem::IsItemDisplayed() const
{
	if (FPhysicsAssetRenderSettings* RenderSettings = GetRenderSettings())
	{
		return RenderSettings->IsBodyHidden(BodySetupIndex) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
	}

	return ECheckBoxState::Undetermined;
}

const FSlateBrush* FSkeletonTreePhysicsControlBodyItem::GetBrush() const
{
	return BodySetup->PhysicsType == EPhysicsType::PhysType_Kinematic ? FAppStyle::GetBrush("PhysicsAssetEditor.Tree.KinematicBody") : FAppStyle::GetBrush("PhysicsAssetEditor.Tree.Body");
}

FSlateColor FSkeletonTreePhysicsControlBodyItem::GetTextColor() const
{
	FLinearColor Color(1.0f, 1.0f, 1.0f);

	if (FilterResult == ESkeletonTreeFilterResult::ShownDescendant)
	{
		Color = FLinearColor::Gray * 0.5f;
	}
	return FSlateColor(Color);
}

FText FSkeletonTreePhysicsControlBodyItem::GetNameColumnToolTip() const
{
	return FText::Format(LOCTEXT("BodyTooltip", "Aggregate physics body for bone '{0}'. Bodies can consist of multiple shapes."), FText::FromName(GetRowItemName()));
}

#undef LOCTEXT_NAMESPACE
