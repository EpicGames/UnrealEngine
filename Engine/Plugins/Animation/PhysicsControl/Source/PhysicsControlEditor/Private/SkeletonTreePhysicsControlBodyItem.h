// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "SkeletonTreePhysicsControlItem.h"

class FSkeletonTreePhysicsControlBodyItem : public FSkeletonTreePhysicsControlItem
{
public:
	SKELETON_TREE_ITEM_TYPE(FSkeletonTreePhysicsControlBodyItem, FSkeletonTreePhysicsControlItem);

	FSkeletonTreePhysicsControlBodyItem(class USkeletalBodySetup* InBodySetup, int32 InBodySetupIndex, const FName& InBoneName, bool bInHasBodySetup, bool bInHasShapes, class UPhysicsAsset* const InPhysicsAsset, const TSharedRef<class ISkeletonTree>& InSkeletonTree);

	virtual UObject* GetObject() const override;

	/** Get the index of the body setup in the physics asset */
	int32 GetBodySetupIndex() const { return BodySetupIndex; }

	/** UI Callbacks */
	virtual void OnToggleItemDisplayed(ECheckBoxState InCheckboxState) override;
	virtual ECheckBoxState IsItemDisplayed() const override;

private:
	/** Gets the icon to display for this body */
	virtual const FSlateBrush* GetBrush() const override;

	/** Gets the color to display the item's text */
	virtual FSlateColor GetTextColor() const override;

	/** Gets the tool tip to display on hovering over the item's name */
	virtual FText GetNameColumnToolTip() const override;

	/** The body setup we are representing */
	USkeletalBodySetup* BodySetup;

	/** The index of the body setup in the physics asset */
	int32 BodySetupIndex;

	/** Whether there is a body set up for this bone */
	bool bHasBodySetup;

	/** Whether this body has shapes */
	bool bHasShapes;
};
