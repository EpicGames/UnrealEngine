// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreePhysicsControlItem.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "PhysicsAssetRenderUtils.h"

#define LOCTEXT_NAMESPACE "FSkeletonTreePhysicsControlItem"

FSkeletonTreePhysicsControlItem::FSkeletonTreePhysicsControlItem(class UPhysicsAsset* const InPhysicsAsset, const TSharedRef<class ISkeletonTree>& InSkeletonTree)
	: FSkeletonTreeItem(InSkeletonTree)
	, PhysicsAssetPathNameHash(UPhysicsAssetRenderUtilities::GetPathNameHash(InPhysicsAsset))
{}

void FSkeletonTreePhysicsControlItem::GenerateWidgetForNameColumn(TSharedPtr< SHorizontalBox > Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected)
{
	Box->AddSlot()
		.AutoWidth()
		.Padding(FMargin(0.0f, 1.0f))
		[
			SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(GetBrush())
		];

	Box->AddSlot()
		.AutoWidth()
		.Padding(2, 0, 0, 0)
		[
			SNew(STextBlock)
				.ColorAndOpacity(this, &FSkeletonTreePhysicsControlItem::GetTextColor)
				.Text(FText::FromName(GetRowItemName()))
				.HighlightText(FilterText)
				.Font(FAppStyle::GetFontStyle("PhysicsAssetEditor.Tree.Font"))
				.ToolTipText(GetNameColumnToolTip())
		];
}


TSharedRef< SWidget > FSkeletonTreePhysicsControlItem::GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected)
{
	if (DataColumnName == ISkeletonTree::DebugVisualizationOptionsName())
	{
		// Create a widget that displays an open/closed eye icon when the body is shown/hidden in the viewport debug draw.
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SCheckBox)
					.ToolTipText(LOCTEXT("SkeletonTreePhysicsItemVisibilityCheckBoxToolTip", "Click to toggle visibility of debug draw for this item"))
					.OnCheckStateChanged(this, &FSkeletonTreePhysicsControlItem::OnToggleItemDisplayed)
					.IsChecked(this, &FSkeletonTreePhysicsControlItem::IsItemDisplayed)
					.CheckedHoveredImage(FAppStyle::GetBrush("Level.VisibleIcon16x"))
					.CheckedImage(FAppStyle::GetBrush("Level.VisibleIcon16x"))
					.CheckedPressedImage(FAppStyle::GetBrush("Level.VisibleIcon16x"))
					.UncheckedHoveredImage(FAppStyle::GetBrush("Level.NotVisibleIcon16x"))
					.UncheckedImage(FAppStyle::GetBrush("Level.NotVisibleIcon16x"))
					.UncheckedPressedImage(FAppStyle::GetBrush("Level.NotVisibleIcon16x"))
			];
	}

	return SNullWidget::NullWidget;
}


FPhysicsAssetRenderSettings* FSkeletonTreePhysicsControlItem::GetRenderSettings() const
{
	return UPhysicsAssetRenderUtilities::GetSettings(PhysicsAssetPathNameHash);
}

#undef LOCTEXT_NAMESPACE
