// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "SkeletonTreeItem.h"

class FSkeletonTreePhysicsControlItem : public FSkeletonTreeItem
{
public:
	SKELETON_TREE_ITEM_TYPE(FSkeletonTreePhysicsControlItem, FSkeletonTreeItem);

	FSkeletonTreePhysicsControlItem(class UPhysicsAsset* const InPhysicsAsset, const TSharedRef<class ISkeletonTree>& InSkeletonTree);

	/** ISkeletonTreeItem interface */
	virtual void GenerateWidgetForNameColumn(TSharedPtr< SHorizontalBox > Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected) override;
	virtual TSharedRef< SWidget > GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected) override;
	virtual FName GetRowItemName() const override { return DisplayName; }

	/** UI Callbacks */
	virtual void OnToggleItemDisplayed(ECheckBoxState InCheckboxState) = 0;
	virtual ECheckBoxState IsItemDisplayed() const = 0;

private:
	/** Gets the icon to display for this body */
	virtual const FSlateBrush* GetBrush() const = 0;

	/** Gets the color to display the item's text */
	virtual FSlateColor GetTextColor() const = 0;

	/** Gets the tool tip to display on hovering over the item's name */
	virtual FText GetNameColumnToolTip() const = 0;

protected:
	/** Gets the physics asset render settings for this physics item */
	struct FPhysicsAssetRenderSettings* GetRenderSettings() const;

	/** Unique ID of the physics asset for this physics item */
	uint32 PhysicsAssetPathNameHash;

	/** The name of the item in the tree */
	FName DisplayName;
};
