// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonTreePhysicsControlShapeItem.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "ScopedTransaction.h"
#include "PhysicsEngine/SkeletalBodySetup.h"

#define LOCTEXT_NAMESPACE "FSkeletonTreePhysicsControlShapeItem"

FSkeletonTreePhysicsControlShapeItem::FSkeletonTreePhysicsControlShapeItem(USkeletalBodySetup* InBodySetup, const FName& InBoneName, int32 InBodySetupIndex, EAggCollisionShape::Type InShapeType, int32 InShapeIndex, const TSharedRef<class ISkeletonTree>& InSkeletonTree)
	: FSkeletonTreeItem(InSkeletonTree)
	, BodySetup(InBodySetup)
	, BodySetupIndex(InBodySetupIndex)
	, ShapeType(InShapeType)
	, ShapeIndex(InShapeIndex)
{
	switch (ShapeType)
	{
	case EAggCollisionShape::Sphere:
		ShapeBrush = FAppStyle::GetBrush("PhysicsAssetEditor.Tree.Sphere");
		DefaultLabel = *FText::Format(LOCTEXT("SphereLabel", "{0} Sphere {1}"), FText::FromName(InBoneName), FText::AsNumber(ShapeIndex)).ToString();
		break;
	case EAggCollisionShape::Box:
		ShapeBrush = FAppStyle::GetBrush("PhysicsAssetEditor.Tree.Box");
		DefaultLabel = *FText::Format(LOCTEXT("BoxLabel", "{0} Box {1}"), FText::FromName(InBoneName), FText::AsNumber(ShapeIndex)).ToString();
		break;
	case EAggCollisionShape::Sphyl:
		ShapeBrush = FAppStyle::GetBrush("PhysicsAssetEditor.Tree.Sphyl");
		DefaultLabel = *FText::Format(LOCTEXT("CapsuleLabel", "{0} Capsule {1}"), FText::FromName(InBoneName), FText::AsNumber(ShapeIndex)).ToString();
		break;
	case EAggCollisionShape::Convex:
		ShapeBrush = FAppStyle::GetBrush("PhysicsAssetEditor.Tree.Convex");
		DefaultLabel = *FText::Format(LOCTEXT("ConvexLabel", "{0} Convex {1}"), FText::FromName(InBoneName), FText::AsNumber(ShapeIndex)).ToString();
		break;
	case EAggCollisionShape::TaperedCapsule:
		ShapeBrush = FAppStyle::GetBrush("PhysicsAssetEditor.Tree.TaperedCapsule");
		DefaultLabel = *FText::Format(LOCTEXT("TaperedCapsuleLabel", "{0} Tapered Capsule {1}"), FText::FromName(InBoneName), FText::AsNumber(ShapeIndex)).ToString();
		break;
	default:
		check(false);
		break;
	}
}

void FSkeletonTreePhysicsControlShapeItem::GenerateWidgetForNameColumn(TSharedPtr< SHorizontalBox > Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected)
{
	Box->AddSlot()
		.AutoWidth()
		.Padding(FMargin(0.0f, 1.0f))
		[
			SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(ShapeBrush)
		];

	TSharedRef<SInlineEditableTextBlock> InlineWidget = SNew(SInlineEditableTextBlock)
		.ColorAndOpacity(FSlateColor::UseForeground())
		.Text(this, &FSkeletonTreePhysicsControlShapeItem::GetNameAsText)
		.ToolTipText(this, &FSkeletonTreePhysicsControlShapeItem::GetNameAsText)
		.HighlightText(FilterText)
		.Font(FAppStyle::GetFontStyle("PhysicsAssetEditor.Tree.Font"))
		.OnTextCommitted(this, &FSkeletonTreePhysicsControlShapeItem::HandleTextCommitted)
		.IsSelected(InIsSelected);

	OnRenameRequested.BindSP(&InlineWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);

	Box->AddSlot()
		.AutoWidth()
		.Padding(2, 0, 0, 0)
		[
			InlineWidget
		];
}

TSharedRef< SWidget > FSkeletonTreePhysicsControlShapeItem::GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected)
{
	return SNullWidget::NullWidget;
}

FName FSkeletonTreePhysicsControlShapeItem::GetRowItemName() const
{
	FString NameAsString = GetNameAsString();
	return *NameAsString;
}

UObject* FSkeletonTreePhysicsControlShapeItem::GetObject() const
{
	return BodySetup;
}

bool FSkeletonTreePhysicsControlShapeItem::CanRenameItem() const
{
	return true;
}

void FSkeletonTreePhysicsControlShapeItem::OnItemDoubleClicked()
{
	OnRenameRequested.ExecuteIfBound();
}

void FSkeletonTreePhysicsControlShapeItem::RequestRename()
{
	OnRenameRequested.ExecuteIfBound();
}

FString FSkeletonTreePhysicsControlShapeItem::GetNameAsString() const
{
	FString StringName;

	switch (ShapeType)
	{
	case EAggCollisionShape::Sphere:
		if (BodySetup->AggGeom.SphereElems.IsValidIndex(ShapeIndex))
		{
			StringName = BodySetup->AggGeom.SphereElems[ShapeIndex].GetName().GetPlainNameString();
		}
		break;
	case EAggCollisionShape::Box:
		if (BodySetup->AggGeom.BoxElems.IsValidIndex(ShapeIndex))
		{
			StringName = BodySetup->AggGeom.BoxElems[ShapeIndex].GetName().GetPlainNameString();
		}
		break;
	case EAggCollisionShape::Sphyl:
		if (BodySetup->AggGeom.SphylElems.IsValidIndex(ShapeIndex))
		{
			StringName = BodySetup->AggGeom.SphylElems[ShapeIndex].GetName().GetPlainNameString();
		}
		break;
	case EAggCollisionShape::Convex:
		if (BodySetup->AggGeom.ConvexElems.IsValidIndex(ShapeIndex))
		{
			StringName = BodySetup->AggGeom.ConvexElems[ShapeIndex].GetName().GetPlainNameString();
		}
		break;
	case EAggCollisionShape::TaperedCapsule:
		if (BodySetup->AggGeom.TaperedCapsuleElems.IsValidIndex(ShapeIndex))
		{
			StringName = BodySetup->AggGeom.TaperedCapsuleElems[ShapeIndex].GetName().GetPlainNameString();
		}
		break;
	}

	if (StringName.IsEmpty())
	{
		StringName = DefaultLabel.ToString();
	}

	return StringName;
}

FText FSkeletonTreePhysicsControlShapeItem::GetNameAsText() const
{
	return FText::FromString(GetNameAsString());
}

void FSkeletonTreePhysicsControlShapeItem::HandleTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	if (!InText.IsEmpty())
	{
		FScopedTransaction Transaction(LOCTEXT("RenameShapeTransaction", "Rename Shape"));

		BodySetup->Modify();

		switch (ShapeType)
		{
		case EAggCollisionShape::Sphere:
			if (BodySetup->AggGeom.SphereElems.IsValidIndex(ShapeIndex))
			{
				BodySetup->AggGeom.SphereElems[ShapeIndex].SetName(*InText.ToString());
			}
			break;
		case EAggCollisionShape::Box:
			if (BodySetup->AggGeom.BoxElems.IsValidIndex(ShapeIndex))
			{
				BodySetup->AggGeom.BoxElems[ShapeIndex].SetName(*InText.ToString());
			}
			break;
		case EAggCollisionShape::Sphyl:
			if (BodySetup->AggGeom.SphylElems.IsValidIndex(ShapeIndex))
			{
				BodySetup->AggGeom.SphylElems[ShapeIndex].SetName(*InText.ToString());
			}
			break;
		case EAggCollisionShape::Convex:
			if (BodySetup->AggGeom.ConvexElems.IsValidIndex(ShapeIndex))
			{
				BodySetup->AggGeom.ConvexElems[ShapeIndex].SetName(*InText.ToString());
			}
			break;
		case EAggCollisionShape::TaperedCapsule:
			if (BodySetup->AggGeom.TaperedCapsuleElems.IsValidIndex(ShapeIndex))
			{
				BodySetup->AggGeom.TaperedCapsuleElems[ShapeIndex].SetName(*InText.ToString());
			}
			break;
		default:
			check(false);
			break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
