// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum EHorizontalAlignment : int;
enum EVerticalAlignment : int;

class FReply;
class FString;
class ITableRow;
class STableViewBase;
class UEdGraphNode;

struct FSlateColor;

class OperatorTreeItem;

using OperatorTreeItemPtr = TSharedPtr<OperatorTreeItem>;

class OperatorTreeItem
{
public:

	virtual ~OperatorTreeItem() = default;

	enum OperatorTreeItemType
	{
		ControlType,
		ModifierType,
		TargetType,
		NodeType,
		MessageType,
		None
	};

	virtual FText Name() const = 0;
	virtual FText Description() const { return Name(); }

	virtual TSharedRef<ITableRow> GenerateRow(const TSharedRef<STableViewBase>& OwnerTable) const = 0;
	virtual FReply OnDoubleClick();
	virtual bool MatchesFilterText(const TArray<TArray<FString>>& FilterStructuredCriteria) const { return true; };
	virtual bool HasTag(const FName Tag) const { return false; }
	virtual bool IsExpandable() const { return false; }
	virtual bool IsExpanded() const { return true; }

	void AddChild(OperatorTreeItemPtr ItemPtr) { Children.Add(ItemPtr); }
	bool HasChildren() const { return !Children.IsEmpty(); }

	virtual OperatorTreeItemType GetType() const { return None; }
	bool HasType(const OperatorTreeItemType InType) const { return InType == GetType(); }

	TArray<OperatorTreeItemPtr> Children;

	static const EHorizontalAlignment HorizontalAlignment;
	static const EVerticalAlignment VerticalAlignment;
	static const float HorizontalPadding;
	static const float VerticalPadding;
};

class OperatorTreeControlItem : public OperatorTreeItem
{
	using Super = OperatorTreeItem;

public:

	using OperatorNameAndTags = TPair<FName, TArray<FName>>;

	OperatorTreeControlItem()
	{}

	OperatorTreeControlItem(const OperatorTreeControlItem&) = default;

	OperatorTreeControlItem(const OperatorNameAndTags& InOperatorNameAndTags)
	: NameAndTags(InOperatorNameAndTags)
	{}

	virtual FText Name() const override;
	virtual FText Description() const override;

	virtual TSharedRef<ITableRow> GenerateRow(const TSharedRef<STableViewBase>& OwnerTable) const override;
	virtual bool MatchesFilterText(const TArray<TArray<FString>>& FilterStructuredCriteria) const override;
	virtual bool HasTag(const FName Tag) const override;
	FName OperatorTypeName() const;

	virtual OperatorTreeItemType GetType() const override { return StaticType(); }
	static OperatorTreeItemType StaticType() { return OperatorTreeItem::ControlType; }

	OperatorNameAndTags NameAndTags;

	static const TArray<FName> HiddenTags;
};

using ControlItemPtr = TSharedPtr<OperatorTreeControlItem>;

inline bool operator==(const OperatorTreeControlItem& Lhs, const OperatorTreeControlItem& Rhs)
{
	return (Lhs.Name().ToString() == Rhs.Name().ToString()) && (Lhs.HasTag(FName("Modifier")) == Rhs.HasTag(FName("Modifier")));
}

class OperatorTreeNodeItem : public OperatorTreeItem
{
	using Super = OperatorTreeItem;

public:

	OperatorTreeNodeItem()
	{}

	OperatorTreeNodeItem(const OperatorTreeNodeItem&) = default;

	OperatorTreeNodeItem(const UEdGraphNode* InNode, const FName& InName, const FString& InBlueprintName);

	virtual FText Name() const override;

	virtual TSharedRef<ITableRow> GenerateRow(const TSharedRef<STableViewBase>& OwnerTable) const override;
	virtual FReply OnDoubleClick() override;

	virtual OperatorTreeItemType GetType() const override { return StaticType(); }
	static OperatorTreeItemType StaticType() { return OperatorTreeItem::NodeType; }

	const UEdGraphNode* GraphNode() const { return Node; }
	bool IsSelectedInGraph() const;
	FSlateColor ColorAndOpacity() const;

private:
	const UEdGraphNode* Node = nullptr;
	FText NodeDisplayName;

};

using NodeItemPtr = TSharedPtr<OperatorTreeNodeItem>;

bool operator==(const OperatorTreeNodeItem& Lhs, const OperatorTreeNodeItem& Rhs);

class OperatorTreeMessageItem : public OperatorTreeItem
{
	using Super = OperatorTreeItem;

public:

	enum MessageType
	{
		Log,
		Warning,
		Error
	};

	OperatorTreeMessageItem() = default;
	OperatorTreeMessageItem(const OperatorTreeMessageItem&) = default;
	OperatorTreeMessageItem(const UEdGraphNode* InNode, const MessageType InType, const FText& InText);

	virtual FText Name() const override;

	virtual TSharedRef<ITableRow> GenerateRow(const TSharedRef<STableViewBase>& OwnerTable) const override;

	virtual OperatorTreeItemType GetType() const override { return StaticType(); }
	static OperatorTreeItemType StaticType() { return OperatorTreeItem::MessageType; }

	const UEdGraphNode* GraphNode() const { return Node; }

private:
	FText Text;
	const UEdGraphNode* Node = nullptr;
	MessageType Type = Log;
};

using MessageItemPtr = TSharedPtr<OperatorTreeMessageItem>;

inline bool operator==(const OperatorTreeMessageItem& Lhs, const OperatorTreeMessageItem& Rhs)
{
	return Lhs.Name().ToString() == Rhs.Name().ToString(); // TODO - don't convert to strings + incomplete
}

template<typename DerivedType> TSharedPtr<DerivedType> StaticCastOperatorItemPtr(const TSharedPtr<OperatorTreeItem> InItem)
{
	if (InItem->GetType() == DerivedType::StaticType())
	{
		return StaticCastSharedPtr<DerivedType>(InItem);
	}

	return TSharedPtr<DerivedType>();
}

bool IsEqual(const TSharedPtr<OperatorTreeItem> Lhs, const TSharedPtr<OperatorTreeItem> Rhs);

// Returns true if the TextToSearch string matches the search criteria. Used to identify operator or set names that match user specified search strings in the editor.
bool MatchSearchText(const FString& TextToSearch, const TArray<TArray<FString>>& StructuredSearchCriteria);
