// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPhysicsControlOperatorViewerInterface.h"
#include "Containers/Ticker.h"

class FSpawnTabArgs;
class SOperatorViewerTabWidget;
class SDockTab;

class FPhysicsControlOperatorViewer : public IPhysicsControlOperatorViewerInterface
{
public:

	virtual ~FPhysicsControlOperatorViewer() = default;

	virtual void OpenOperatorNamesTab() override;
	virtual void CloseOperatorNamesTab() override;
	virtual void ToggleOperatorNamesTab() override;
	virtual bool IsOperatorNamesTabOpen() override;
	virtual void RequestRefresh() override;

	void Startup();
	void Shutdown();

	void OnTabClosed(TSharedRef<SDockTab> DockTab);

private:
	TSharedRef<SDockTab> OnCreateTab(const FSpawnTabArgs& SpawnTabArgs);

	TSharedPtr<SOperatorViewerTabWidget> PersistentTabWidget;
	TSharedPtr<SDockTab> OperatorNamesTab;
};
