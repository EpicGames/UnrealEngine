// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/SCompoundWidget.h"
#include "Filters/SFilterSearchBox.h"
#include "Filters/SBasicFilterBar.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"

#include "Widgets/Views/STreeView.h"

#include "AnimNode_RigidBodyWithControl.h"

#include "OperatorViewer/OperatorTreeElements.h"

/**
 * class SOperatorTreeWidget
 *
 * 
 */
class SOperatorTreeWidget : public SCompoundWidget
{
	using Parent = SCompoundWidget;

public:
	using ItemType = OperatorTreeItemPtr;

	using FilterBarType = SBasicFilterBar<ItemType>;

	SLATE_BEGIN_ARGS(SOperatorTreeWidget) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
	
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	TSharedRef<ITableRow> GenerateItemRow(ItemType InItem, const TSharedRef<STableViewBase>& OwnerTable);
	void OnItemGetChildren(ItemType InItem, TArray<ItemType>& OutChildren);
	void OnFilterBarFilterChanged();
	void OnFilterTextChanged(const FText& SearchText);

	bool MatchesSearchAndFilter(const ItemType InItem) const;

	bool HasTag(const ItemType Item, const FName Tag);

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	void OnItemDoubleClicked(ItemType InItem);
	TSharedPtr<SWidget> CreateContextMenu();

	void RequestRefresh();

	void CopySelectedItemsNamesToClipboard() const;
	void CopySelectedItemsDescriptionsToClipboard() const;

protected:

	void Construct_Internal();
	virtual void CreateCustomFilters();
	virtual void Refresh();

	TSharedPtr<STreeView<ItemType>> TreeView;
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<FilterBarType> FilterBar;

	TArray<TSharedRef<FFilterBase<ItemType>>> CustomFilters;
	TArray<ItemType> TreeItems;
	TSet<FName> SetNames;
	FText FilterText;

	TArray<TArray<FString>> FilterStructuredCriteria;
	TSet<FName> FilterSetNames;

	bool bRefreshRequested = false;
};
