// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
//#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SCompoundWidget;
class SOperatorTreeWidget;

class SOperatorViewerTabWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SOperatorViewerTabWidget)
	{}

	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, int32 InTabIndex);
	void RequestRefresh();

private:
	TSharedPtr<SOperatorTreeWidget> TreeViewWidget;
};
