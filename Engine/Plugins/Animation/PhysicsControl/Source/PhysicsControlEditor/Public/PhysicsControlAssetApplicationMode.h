// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

#define UE_API PHYSICSCONTROLEDITOR_API

class FWorkflowCentricApplication;
class IPersonaPreviewScene;
class ISkeletonTree;

class FPhysicsControlAssetEditor;

/**
 * The application mode for the Physics Control Asset Editor
 * This defines the layout of the UI. It basically spawns all the tabs, such as the viewport, details panels, etc.
 */
class FPhysicsControlAssetApplicationMode : public FApplicationMode
{
public:
	/** The name of this mode. */
	static UE_API FName ModeName;

	UE_API FPhysicsControlAssetApplicationMode(
		TSharedRef<FWorkflowCentricApplication> InHostingApp, 
		TSharedPtr<ISkeletonTree>               SkeletonTree,
		TSharedRef<IPersonaPreviewScene>        InPreviewScene);

	// FApplicationMode overrides.
	UE_API virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	// ~END FApplicationMode overrides.

protected:
	/** The hosting app. */
	TWeakPtr<FPhysicsControlAssetEditor> PhysicsControlAssetEditor = nullptr;

	/** The tab factories we support. */
	FWorkflowAllowedTabSet TabFactories;
};

#undef UE_API
