// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "IHasPersonaToolkit.h"
#include "IPersonaPreviewScene.h"
#include "IPersonaViewport.h"
#include "PersonaAssetEditorToolkit.h"
#include "BodySetupEnums.h"
#include "Preferences/PhysicsAssetEditorOptions.h"

#define UE_API PHYSICSCONTROLEDITOR_API

class FPhysicsControlAssetEditorEditMode;
class FPhysicsControlAssetEditorData;
class UAnimPreviewInstance;
class UPhysicsControlAsset;
class ISkeletonTree;
class FPhysicsControlAssetEditorSkeletonTreeBuilder;
class FUICommandList_Pinnable;

namespace PhysicsControlAssetEditorModes
{
	extern const FName PhysicsControlAssetEditorMode;
}

/**
 * The main toolkit/editor for working with Physics Control Assets
 */
class FPhysicsControlAssetEditor :
	public FPersonaAssetEditorToolkit,
	public IHasPersonaToolkit,
	public FGCObject,
	public FEditorUndoClient,
	public FTickableEditorObject
{
public:
	friend class FPhysicsControlAssetApplicationMode;
	friend class FPhysicsControlAssetEditorEditMode;
	friend class FPhysicsControlAssetProfileDetailsCustomization;
	friend class FPhysicsControlAssetPreviewDetailsCustomization;
	friend class FPhysicsControlAssetSetupDetailsCustomization;
	friend class FPhysicsControlAssetInfoDetailsCustomization;

public:
	/** Initialize the asset editor. This will register the application mode, init the preview scene, etc. */
	UE_API void InitAssetEditor(
		const EToolkitMode::Type        Mode,
		const TSharedPtr<IToolkitHost>& InitToolkitHost,
		UPhysicsControlAsset*    InPhysicsControlAsset);

	/** Shared data accessor */
	UE_API TSharedPtr<FPhysicsControlAssetEditorData> GetEditorData() const;

	// FAssetEditorToolkit overrides.
	UE_API virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	UE_API virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;
	UE_API virtual FText GetToolkitName() const override;
	UE_API virtual FLinearColor GetWorldCentricTabColorScale() const override;
	UE_API virtual FString GetWorldCentricTabPrefix() const override;
	UE_API virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
	// ~END FAssetEditorToolkit overrides.

	// FGCObject overrides.
	virtual FString GetReferencerName() const override { return TEXT("FPhysicsControlAssetEditor"); }
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	// ~END FGCObject overrides.

	// FTickableEditorObject overrides.
	UE_API virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	UE_API virtual TStatId GetStatId() const override;
	// ~END FTickableEditorObject overrides.

	// IHasPersonaToolkit overrides.
	UE_API virtual TSharedRef<IPersonaToolkit> GetPersonaToolkit() const override;
	// ~END IHasPersonaToolkit overrides.

	IPersonaToolkit* GetPersonaToolkitPointer() const { return PersonaToolkit.Get(); }

	/** Repopulates the hierarchy tree view */
	UE_API void RefreshHierarchyTree();

	/** Refreshes the preview viewport */
	UE_API void RefreshPreviewViewport();

	/** Invokes the control profile with the name, assuming simulation is running */
	UE_API void InvokeControlProfile(FName ProfileName);

	/** Invokes the most recently invoked control profile */
	UE_API void ReinvokeControlProfile();

	/** Destroys all existing controls modifiers and then recreates them from the control asset */
	UE_API void RecreateControlsAndModifiers();

protected:
	UE_API FText GetSimulationToolTip() const;
	UE_API FSlateIcon GetSimulationIcon() const;

	/** Preview scene setup. */
	UE_API void HandlePreviewSceneCreated(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene);
	UE_API void HandleViewportCreated(const TSharedRef<IPersonaViewport>& InPersonaViewport);
	UE_API void HandleDetailsCreated(const TSharedRef<class IDetailsView>& InDetailsView);
	UE_API void HandleSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo);
	UE_API void HandleGetFilterLabel(TArray<FText>& InOutItems) const;
	UE_API void HandleExtendFilterMenu(FMenuBuilder& InMenuBuilder);
	UE_API void HandleExtendContextMenu(FMenuBuilder& InMenuBuilder);
	UE_API void OnFinishedChangingDetails(const FPropertyChangedEvent& PropertyChangedEvent);
	UE_API void ShowEmptyDetails() const;

	UE_API void ExtendMenu();
	UE_API void ExtendToolbar();
	UE_API void ExtendViewportMenus();
	UE_API void BindCommands();

	/** Building the context menus */
	UE_API void BuildMenuWidgetSelection(FMenuBuilder& InMenuBuilder);
	UE_API void BuildMenuWidgetBody(FMenuBuilder& InMenuBuilder);

	// Toolbar/menu commands
	UE_API void OnCompile();
	UE_API bool IsCompilationNeeded();
	UE_API void OnToggleSimulation();
	UE_API void OnToggleSimulationNoGravity();
	UE_API bool IsNoGravitySimulationEnabled() const;
	UE_API void OnToggleSimulationFloorCollision();
	UE_API bool IsSimulationFloorCollisionEnabled() const;
	UE_API void OnMeshRenderingMode(EPhysicsAssetEditorMeshViewMode Mode, bool bSimulation);
	UE_API bool IsMeshRenderingMode(EPhysicsAssetEditorMeshViewMode Mode, bool bSimulation) const;
	UE_API void OnCollisionRenderingMode(EPhysicsAssetEditorCollisionViewMode Mode, bool bSimulation);
	UE_API bool IsCollisionRenderingMode(EPhysicsAssetEditorCollisionViewMode Mode, bool bSimulation) const;
	UE_API void OnConstraintRenderingMode(EPhysicsAssetEditorConstraintViewMode Mode, bool bSimulation);
	UE_API bool IsConstraintRenderingMode(EPhysicsAssetEditorConstraintViewMode Mode, bool bSimulation) const;
	UE_API void ToggleDrawViolatedLimits();
	UE_API bool IsDrawingViolatedLimits() const;
	UE_API bool IsRunningSimulation() const;
	UE_API bool IsNotRunningSimulation() const;

	/** Make the constraint scale widget */
	UE_API TSharedRef<SWidget> MakeConstraintScaleWidget();

	/** Make the collision opacity widget */
	UE_API TSharedRef<SWidget> MakeCollisionOpacityWidget();

protected:
	/** The persona toolkit. */
	TSharedPtr<IPersonaToolkit> PersonaToolkit = nullptr;

	// Persona viewport.
	TSharedPtr<IPersonaViewport> PersonaViewport = nullptr;

	/** Data and methods shared across multiple classes */
	TSharedPtr<FPhysicsControlAssetEditorData> EditorData;

	// Asset properties tab 
	TSharedPtr<IDetailsView> DetailsView;

	/** The skeleton tree widget */
	TSharedPtr<ISkeletonTree> SkeletonTree;

	/** The skeleton tree builder */
	TSharedPtr<FPhysicsControlAssetEditorSkeletonTreeBuilder> SkeletonTreeBuilder;

	/** Command list for skeleton tree operations */
	TSharedPtr<FUICommandList_Pinnable> SkeletonTreeCommandList;

	/** Command list for viewport operations */
	TSharedPtr<FUICommandList_Pinnable> ViewportCommandList;

	/** Has the asset editor been initialized? */
	bool bIsInitialized = false;

	/** True if in OnTreeSelectionChanged()... protects against infinite recursion */
	bool bSelecting;

	/** Stored when a control profile is invoked */
	FName PreviouslyInvokedControlProfile;
};

#undef UE_API
