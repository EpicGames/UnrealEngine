// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPersonaEditMode.h"

#define UE_API PHYSICSCONTROLEDITOR_API

class FPhysicsControlAssetEditor;
class FPhysicsControlAssetEditorData;
class FEditorViewportClient;
class UFont;

/**
 * The Physics Control Asset Editor edit mode.
 */
class FPhysicsControlAssetEditorEditMode : public IPersonaEditMode
{
public:
	/** The name of the mode. */
	static UE_API FName ModeName;

	UE_API FPhysicsControlAssetEditorEditMode();

	UE_API void SetEditorData(
		const TSharedRef<FPhysicsControlAssetEditor> InPhysicsControlAssetEditor,
		TSharedPtr<FPhysicsControlAssetEditorData>   InEditorData);

	// IPersonaEditMode overrides.
	UE_API virtual bool GetCameraTarget(FSphere& OutTarget) const override;
	UE_API virtual class IPersonaPreviewScene& GetAnimPreviewScene() const override;
	virtual void GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const override {}
	UE_API virtual void Exit() override;
	// ~END IPersonaEditMode overrides.

	// FEdMode overrides.
	UE_API virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	UE_API virtual bool InputAxis(FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime) override;
	UE_API virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	UE_API virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	UE_API virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override { return true; }
	virtual bool AllowWidgetMove() override { return false; }
	virtual bool ShouldDrawWidget() const override { return false; }
	virtual bool UsesTransformWidget() const override { return false; }
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override { return false; }
	UE_API virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	UE_API virtual bool ReceivedFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport);
	UE_API virtual bool LostFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport);
	// ~END FEdMode overrides.

protected:
	/** Simulation mouse forces */
	UE_API bool SimMousePress(FEditorViewportClient* InViewportClient, FKey Key);
	UE_API void SimMouseMove(FEditorViewportClient* InViewportClient, float DeltaX, float DeltaY);
	UE_API bool SimMouseRelease();
	UE_API bool SimMouseWheelUp(FEditorViewportClient* InViewportClient);
	UE_API bool SimMouseWheelDown(FEditorViewportClient* InViewportClient);

	UE_API void OpenBodyMenu(FEditorViewportClient* InViewportClient);
	UE_API void OpenSelectionMenu(FEditorViewportClient* InViewportClient);

	/** Called when no scene proxy is hit, deselects everything */
	UE_API void HitNothing(FEditorViewportClient* InViewportClient);

protected:
	TWeakPtr<FPhysicsControlAssetEditor> PhysicsControlAssetEditor;

	TSharedPtr<FPhysicsControlAssetEditorData> EditorData;

	/** Font used for drawing debug text to the viewport */
	UFont* PhysicsControlAssetEditorFont;

	/** Misc consts */
	const float	SimHoldDistanceChangeDelta;
	const float	SimMinHoldDistance;
	const float SimGrabMoveSpeed;

	// For mouse forces TODO Danny surely this doesn't need to be here unless its actually shared!
	FIntPoint LastClickPos;
	FVector LastClickOrigin;
	FVector LastClickDirection;
	FVector LastClickHitPos;
	FVector LastClickHitNormal;
	bool bLastClickHit;

	/** Simulation mouse forces */
	float SimGrabPush = 0.0f;
	float SimGrabMinPush = 0.0f;
	FVector SimGrabLocation = FVector::ZeroVector;
	FVector SimGrabX = FVector::ZeroVector;
	FVector SimGrabY = FVector::ZeroVector;
	FVector SimGrabZ = FVector::ZeroVector;

	/** For interacting with the asset while the simulation is running */
	TArray<FTransform> StartManRelConTM;
	TArray<FTransform> StartManParentConTM;
	TArray<FTransform> StartManChildConTM;
	float DragX = 0.0f;
	float DragY = 0.0f;

};

#undef UE_API
