// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"



class FPhysicsControlOperatorViewer;
class FPhysicsControlAssetActions;

/**
 * Module handles the Physics Control Asset editor, and also the Physics Control Visualizer
 */
class FPhysicsControlEditorModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	void StartupModule() override;
	void ShutdownModule() override;

private:

	TArray<FName> VisualizersToUnregisterOnShutdown;
	FPhysicsControlOperatorViewer* EditorInterface = nullptr;
};

