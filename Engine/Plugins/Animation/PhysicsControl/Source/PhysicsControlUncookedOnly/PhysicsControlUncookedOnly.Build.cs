// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
public class PhysicsControlUncookedOnly : ModuleRules
{
	public PhysicsControlUncookedOnly(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AnimationCore",
				"AnimGraphRuntime",
				"Core",
				"CoreUObject",
				"Engine",
				"Persona",
				"PhysicsControl",
				"Slate",
				"SlateCore",
			}
			);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
						"AnimationEditor",
						"AnimGraph",
						"BlueprintGraph",
						"EditorFramework",
						"Kismet",
						"UnrealEd",
				}
			);
		}
	}
}
