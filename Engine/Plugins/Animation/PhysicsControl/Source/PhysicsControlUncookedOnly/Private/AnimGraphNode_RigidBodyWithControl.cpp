// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RigidBodyWithControl.h"
#include "AnimNodeEditModes.h"
#include "Features/IModularFeatures.h"
#include "Kismet2/CompilerResultsLog.h"
#include "AnimNode_RigidBodyWithControl.h"
#include "EditorModeManager.h"
#include "IPhysicsAssetRenderInterface.h"
#include "IPhysicsControlOperatorViewerInterface.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "PhysicsControlOperatorNameGeneration.h"
#include "PhysicsControlAsset.h"

// Details includes
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "PrimitiveDrawingUtils.h"

// UE_DISABLE_OPTIMIZATION;

// Use this CVar to enable/disable the viewer for control/modifier sets. It's not really functional/correct enough
// yet for general use and visibility

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_RigidBodyWithControl)
bool bRBANWithControl_EnableControlSetViewer = false;
FAutoConsoleVariableRef CVarRigidBodyWithControlEnableControlSetViewer(
	TEXT("p.RigidBodyWithControl.EnableControlSetViewer"), 
	bRBANWithControl_EnableControlSetViewer, 
	TEXT("Enable/Disable the simple viewer for control and modifier sets for the RBWC node"), ECVF_Default);


/////////////////////////////////////////////////////
// UAnimGraphNode_RigidBodyWithControl

#define LOCTEXT_NAMESPACE "RigidBodyWithControl"

namespace
{
	// Returns the registered IPhysicsAssetRenderInterface modular feature, or nullptr if it is not
	// available (e.g. during early initialization, or in builds where the providing module is not
	// loaded). GetModularFeature asserts when the feature is missing, so this helper centralizes
	// the IsModularFeatureAvailable guard.
	IPhysicsAssetRenderInterface* GetPhysicsAssetRenderInterface()
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		const FName FeatureName = IPhysicsAssetRenderInterface::GetModularFeatureName();
		if (!ModularFeatures.IsModularFeatureAvailable(FeatureName))
		{
			return nullptr;
		}
		return &ModularFeatures.GetModularFeature<IPhysicsAssetRenderInterface>(FeatureName);
	}
}

UAnimGraphNode_RigidBodyWithControl::UAnimGraphNode_RigidBodyWithControl(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_RigidBodyWithControl::GetControllerDescription() const
{
	return LOCTEXT(
		"AnimGraphNode_RigidBodyWithControl_ControllerDescription", 
		"Rigid body simulation with Control for physics asset");
}

FText UAnimGraphNode_RigidBodyWithControl::GetTooltipText() const
{
	return LOCTEXT(
		"AnimGraphNode_RigidBodyWithControl_Tooltip", 
		"This simulates based on the skeletal mesh component's physics asset with control options");
}

FText UAnimGraphNode_RigidBodyWithControl::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText(LOCTEXT("AnimGraphNode_RigidBodyWithControl_NodeTitle", "RigidBodyWithControl"));
}

FLinearColor UAnimGraphNode_RigidBodyWithControl::GetNodeTitleColor() const
{
	return FLinearColor(1.0f, 0.0f, 1.0f); // <- Magenta - as a warning
}

FString UAnimGraphNode_RigidBodyWithControl::GetNodeCategory() const
{
	return TEXT("Animation|Dynamics");
}

void UAnimGraphNode_RigidBodyWithControl::ValidateAnimNodeDuringCompilation(
	USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

void UAnimGraphNode_RigidBodyWithControl::Draw(
	FPrimitiveDrawInterface* PDI, 
	USkeletalMeshComponent*  PreviewSkelMeshComp, 
	const bool               bIsSelected, 
	const bool               bIsPoseWatchEnabled) const
{
	if (const FAnimNode_RigidBodyWithControl* const RuntimeRigidBodyNode = 
		GetDebuggedAnimNode<FAnimNode_RigidBodyWithControl>())
	{
		if (UPhysicsAsset* const PhysicsAsset = RuntimeRigidBodyNode->GetPhysicsAsset())
		{
			IPhysicsAssetRenderInterface* const PhysicsAssetRenderInterface = GetPhysicsAssetRenderInterface();
			if (!PhysicsAssetRenderInterface)
			{
				return;
			}

			// Draw Bodies.
			if (bIsSelected ||
				(bIsPoseWatchEnabled && PoseWatchElementBodies.IsValid() && PoseWatchElementBodies->GetIsVisible()))
			{
				FColor PrimitiveColorOverride = FColor::Transparent;

				// Get primitive color from pose watch component.
				if (!bIsSelected)
				{
					PrimitiveColorOverride = PoseWatchElementBodies->GetColor();
					PrimitiveColorOverride.A = 255;
				}

				PhysicsAssetRenderInterface->DebugDrawBodies(
					PreviewSkelMeshComp, PhysicsAsset, PDI, PrimitiveColorOverride);
			}

			// Draw Constraints.
			if (bIsSelected ||
				(bIsPoseWatchEnabled && PoseWatchElementConstraints.IsValid() && PoseWatchElementConstraints->GetIsVisible()))
			{
				PhysicsAssetRenderInterface->DebugDrawConstraints(PreviewSkelMeshComp, PhysicsAsset, PDI);
			}
		}
	}
}

void UAnimGraphNode_RigidBodyWithControl::OnPoseWatchChanged(
	const bool             IsPoseWatchEnabled, 
	TObjectPtr<UPoseWatch> InPoseWatch, 
	FEditorModeTools&      InModeTools, 
	FAnimNode_Base*        InRuntimeNode)
{
	Super::OnPoseWatchChanged(IsPoseWatchEnabled, InPoseWatch, InModeTools, InRuntimeNode);

	UPoseWatch* const PoseWatch = InPoseWatch.Get();

	if (PoseWatch)
	{
		// A new pose watch has been created for this node - add node specific pose watch components.
		PoseWatchElementBodies = InPoseWatch.Get()->FindOrAddElement(FText(
			LOCTEXT("PoseWatchElementLabel_RigidBodyWithControl_PhysicsBodies", "Physics Bodies")), 
			TEXT("PhysicsAssetEditor.Tree.Body"));
		PoseWatchElementConstraints = InPoseWatch.Get()->FindOrAddElement(FText(
			LOCTEXT("PoseWatchElementLabel__RigidBodyWithControl_PhysicsConstraints", "Physics Constraints")), 
			TEXT("PhysicsAssetEditor.Tree.Constraint"));
		PoseWatchElementParentSpaceControls = InPoseWatch.Get()->FindOrAddElement(FText(
			LOCTEXT("PoseWatchElementLabel_RigidBodyWithControl_ParentSpaceControls", "Parent Space Controls")), 
			TEXT("PhysicsAssetEditor.Tree.Body"));
		PoseWatchElementWorldSpaceControls = InPoseWatch.Get()->FindOrAddElement(FText(
			LOCTEXT("PoseWatchElementLabel_RigidBodyWithControl_WorldSpaceControls", "World Space Controls")), 
			TEXT("PhysicsAssetEditor.Tree.Body"));

		check(PoseWatchElementConstraints.IsValid()); // Expect to find a valid component;
		if (PoseWatchElementConstraints.IsValid())
		{
			PoseWatchElementConstraints->SetHasColor(false);
		}
	}
}

void UAnimGraphNode_RigidBodyWithControl::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);
	IDetailCategoryBuilder& ViewportCategory = DetailBuilder.EditCategory(TEXT("Debug Visualization"));
	FAnimNode_RigidBodyWithControl* const RigidBodyNode = static_cast<FAnimNode_RigidBodyWithControl*>(GetDebuggedAnimNode());

	if (bRBANWithControl_EnableControlSetViewer)
	{
		FDetailWidgetRow& ControlSetViewerWidgetRow = ViewportCategory.AddCustomRow(
			LOCTEXT("ToggleControlSetViewerWidgetRowButtonRow", "ControlSetViewer"));

		ControlSetViewerWidgetRow
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.OnClicked_Lambda([this]() { this->ToggleControlSetViewerTab(); return FReply::Handled(); })
					.ButtonColorAndOpacity_Lambda([this]() { return (IsControlSetViewerTabOpen()) ? FAppStyle::Get().GetSlateColor("Colors.AccentRed") : FAppStyle::Get().GetSlateColor("Colors.AccentGreen"); })
					.Content()
					[
						SNew(STextBlock)
						.Text_Lambda([this]() { return (IsControlSetViewerTabOpen()) ? LOCTEXT("CloseControlSetViewerTabButtonText", "Close Control Set Viewer") : LOCTEXT("OpenControlSetViewerTabButtonText", "Open Control Set Viewer"); })
						.ToolTipText(LOCTEXT("ToggleControlSetViewerTabButtonToolTip", "Toggle the viewer for control and modifier sets. This lists the controls and modifiers created by each RBWC node, and shows what sets they are in"))
					]
				]
			];
	}

	{
		FDetailWidgetRow& DebugVisualizationWidgetRow = ViewportCategory.AddCustomRow(
			LOCTEXT("ToggleDebugVisualizationButtonRow", "DebugVisualization"));

		DebugVisualizationWidgetRow
			[
				SNew(SHorizontalBox)
				// Show/Hide Bodies button.
				+ SHorizontalBox::Slot()
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.OnClicked_Lambda([this]() { this->ToggleBodyVisibility(); return FReply::Handled(); })
					.ButtonColorAndOpacity_Lambda([this]() { return (AreAnyBodiesHidden()) ? FAppStyle::Get().GetSlateColor("Colors.AccentRed") : FAppStyle::Get().GetSlateColor("Colors.AccentGreen"); })
					.Content()
					[
						SNew(STextBlock)
						.Text_Lambda([this]() { return (AreAnyBodiesHidden()) ? LOCTEXT("ShowAllBodiesButtonText", "Show All Bodies") : LOCTEXT("HideAllBodiesButtonText", "Hide All Bodies"); })
						.ToolTipText(LOCTEXT("ToggleBodyVisibilityButtonToolTip", "Toggle debug visualization of all physics bodies"))
					]
				]
				// Show/Hide Constraints button.
				+ SHorizontalBox::Slot()
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.OnClicked_Lambda([this]() { this->ToggleConstraintVisibility(); return FReply::Handled(); })
					.ButtonColorAndOpacity_Lambda([this]() { return (AreAnyConstraintsHidden()) ? FAppStyle::Get().GetSlateColor("Colors.AccentRed") : FAppStyle::Get().GetSlateColor("Colors.AccentGreen"); })
					.Content()
					[
						SNew(STextBlock)
						.Text_Lambda([this]() { return (AreAnyConstraintsHidden()) ? LOCTEXT("ShowAllConstraintsButtonText", "Show All Constraints") : LOCTEXT("HideAllConstraintsButtonText", "Hide All Constraints"); })
						.ToolTipText(LOCTEXT("ToggleConstraintVisibilityButtonToolTip", "Toggle debug visualization of all physics constriants"))
					]
				]
			];
	}

	// World Collision Enable checkbox and collision type flags.
	{
		IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("CollisionSettings");

		TSharedPtr<IPropertyHandle> NodeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_RigidBodyWithControl, Node), UAnimGraphNode_RigidBodyWithControl::StaticClass());

		if(ensure(NodeHandle.IsValid()))
		{
			TSharedPtr<IPropertyHandle> EnableWorldCollision = NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_RigidBodyWithControl, bEnableWorldGeometry));
																																			
			TSharedPtr<IPropertyHandle> HasPhysics = NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_RigidBodyWithControl, bWorldGeometryCollisionEnabledHasPhysics));
			TSharedPtr<IPropertyHandle> HasQuery = NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_RigidBodyWithControl, bWorldGeometryCollisionEnabledHasQuery));
			TSharedPtr<IPropertyHandle> HasProbe = NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_RigidBodyWithControl, bWorldGeometryCollisionEnabledHasProbe));

			if(ensure(EnableWorldCollision) && ensure(HasPhysics) && ensure(HasQuery) && ensure(HasProbe))
			{
				DetailBuilder.HideProperty(EnableWorldCollision);
				DetailBuilder.HideProperty(HasPhysics);
				DetailBuilder.HideProperty(HasQuery);
				DetailBuilder.HideProperty(HasProbe);

				auto MakeCheckbox = [EnableWorldCollision](const FString& Name, TSharedPtr<IPropertyHandle> Property, const bool IsCollisionType = true) -> TSharedRef<SCheckBox>
				{
					FString ToolTipText = "Enable collision with world geometry";
				
					if(IsCollisionType)
					{
						ToolTipText += FString::Printf(TEXT(" with %s collision type"), *Name);
					}
					
					return SNew(SCheckBox)
						.IsChecked_Lambda([Property]()  -> ECheckBoxState { bool bValue = false; Property->GetValue(bValue); return (bValue)? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
						.OnCheckStateChanged_Lambda([Property](ECheckBoxState InState) { Property->SetValue(InState == ECheckBoxState::Checked); })
						.ToolTipText(FText::FromString(ToolTipText))
						[
							SNew(STextBlock)
								.Text(FText::FromString(Name))
						];
				};

				auto IsCheckboxEnabled = [EnableWorldCollision]() -> bool 
				{ 
					bool bValue = false; 
					EnableWorldCollision->GetValue(bValue); 
					return bValue; 
				};

				const float CheckBoxPadding = 2.0f;

				TSharedRef<SCheckBox> PhysicsWidget = MakeCheckbox("Physics", HasPhysics);
				PhysicsWidget->SetEnabled(TAttribute<bool>::CreateLambda(IsCheckboxEnabled));
				TSharedRef<SCheckBox> QueryWidget = MakeCheckbox("Query", HasQuery);
				QueryWidget->SetEnabled(TAttribute<bool>::CreateLambda(IsCheckboxEnabled));
				TSharedRef<SCheckBox> ProbeWidget = MakeCheckbox("Probe", HasProbe);
				ProbeWidget->SetEnabled(TAttribute<bool>::CreateLambda(IsCheckboxEnabled));

				// Add them to the same row
				Category.AddCustomRow(LOCTEXT("WorldCollisionRow", "WorldCollision"))
					.NameContent()
					[
						MakeCheckbox("World Collision", EnableWorldCollision, false)
					]
					.ValueContent()
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(CheckBoxPadding)
							[
								PhysicsWidget
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(CheckBoxPadding)
							[
								QueryWidget
							]
							+ SHorizontalBox::Slot()							
							.AutoWidth()
							.Padding(CheckBoxPadding)
							[
								ProbeWidget
							]
					];
			}
		}
	}

	uint32 SortOrder = 0;
	DetailBuilder.EditCategory("Debug Visualization").SetSortOrder(SortOrder++);
	DetailBuilder.EditCategory("Functions").SetSortOrder(SortOrder++);
	DetailBuilder.EditCategory("Settings").SetSortOrder(SortOrder++);
	DetailBuilder.EditCategory("CollisionSettings").SetSortOrder(SortOrder++);
	DetailBuilder.EditCategory("Tag").SetSortOrder(SortOrder++);
	DetailBuilder.EditCategory("ControlSetup").SetSortOrder(SortOrder++);
	DetailBuilder.EditCategory("Controls").SetSortOrder(SortOrder++);
	DetailBuilder.EditCategory("PhysicsAssetConditioning").SetSortOrder(SortOrder++);
	DetailBuilder.EditCategory("Performance").SetSortOrder(SortOrder++);
	DetailBuilder.EditCategory("Alpha").SetSortOrder(SortOrder++);
	DetailBuilder.EditCategory("Bindings").SetSortOrder(SortOrder++);
}

void UAnimGraphNode_RigidBodyWithControl::PostChange()
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(
		IPhysicsControlOperatorViewerInterface::GetModularFeatureName()))
	{
		IPhysicsControlOperatorViewerInterface& PhysicsControlViewerInterface =
			IModularFeatures::Get().GetModularFeature<IPhysicsControlOperatorViewerInterface>(
				IPhysicsControlOperatorViewerInterface::GetModularFeatureName());
		PhysicsControlViewerInterface.RequestRefresh();
	}
}

void UAnimGraphNode_RigidBodyWithControl::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (IPhysicsAssetRenderInterface* const PhysicsAssetRenderInterface = GetPhysicsAssetRenderInterface())
	{
		PhysicsAssetRenderInterface->SaveConfig();
	}

	PostChange();
}

void UAnimGraphNode_RigidBodyWithControl::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();
	PostChange();
}

void UAnimGraphNode_RigidBodyWithControl::PostPasteNode()
{
	Super::PostPasteNode();
	PostChange();
}

void UAnimGraphNode_RigidBodyWithControl::DestroyNode()
{
	Super::DestroyNode();
	PostChange();
}

void UAnimGraphNode_RigidBodyWithControl::ToggleBodyVisibility()
{
	FAnimNode_RigidBodyWithControl* const RigidBodyNode =
		static_cast<FAnimNode_RigidBodyWithControl*>(GetDebuggedAnimNode());

	if (RigidBodyNode)
	{
		if (IPhysicsAssetRenderInterface* const PhysicsAssetRenderInterface = GetPhysicsAssetRenderInterface())
		{
			PhysicsAssetRenderInterface->ToggleShowAllBodies(RigidBodyNode->GetPhysicsAsset());
		}
	}
}

void UAnimGraphNode_RigidBodyWithControl::ToggleConstraintVisibility()
{
	FAnimNode_RigidBodyWithControl* const RigidBodyNode =
		static_cast<FAnimNode_RigidBodyWithControl*>(GetDebuggedAnimNode());

	if (RigidBodyNode)
	{
		if (IPhysicsAssetRenderInterface* const PhysicsAssetRenderInterface = GetPhysicsAssetRenderInterface())
		{
			PhysicsAssetRenderInterface->ToggleShowAllConstraints(RigidBodyNode->GetPhysicsAsset());
		}
	}
}

bool UAnimGraphNode_RigidBodyWithControl::AreAnyBodiesHidden() const
{
	FAnimNode_RigidBodyWithControl* const RigidBodyNode =
		static_cast<FAnimNode_RigidBodyWithControl*>(GetDebuggedAnimNode());

	if (RigidBodyNode)
	{
		if (IPhysicsAssetRenderInterface* const PhysicsAssetRenderInterface = GetPhysicsAssetRenderInterface())
		{
			return PhysicsAssetRenderInterface->AreAnyBodiesHidden(RigidBodyNode->GetPhysicsAsset());
		}
	}

	return false;
}

bool UAnimGraphNode_RigidBodyWithControl::AreAnyConstraintsHidden() const
{
	FAnimNode_RigidBodyWithControl* const RigidBodyNode =
		static_cast<FAnimNode_RigidBodyWithControl*>(GetDebuggedAnimNode());

	if (RigidBodyNode)
	{
		if (IPhysicsAssetRenderInterface* const PhysicsAssetRenderInterface = GetPhysicsAssetRenderInterface())
		{
			return PhysicsAssetRenderInterface->AreAnyConstraintsHidden(RigidBodyNode->GetPhysicsAsset());
		}
	}

	return false;
}

void UAnimGraphNode_RigidBodyWithControl::ToggleControlSetViewerTab()
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(
		IPhysicsControlOperatorViewerInterface::GetModularFeatureName()))
	{
		IPhysicsControlOperatorViewerInterface& PhysicsControlViewerInterface =
			IModularFeatures::Get().GetModularFeature<IPhysicsControlOperatorViewerInterface>(
				IPhysicsControlOperatorViewerInterface::GetModularFeatureName());
		PhysicsControlViewerInterface.ToggleOperatorNamesTab();
	}
}

bool UAnimGraphNode_RigidBodyWithControl::IsControlSetViewerTabOpen() const
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(
		IPhysicsControlOperatorViewerInterface::GetModularFeatureName()))
	{
		IPhysicsControlOperatorViewerInterface& PhysicsControlViewerInterface =
		IModularFeatures::Get().GetModularFeature<IPhysicsControlOperatorViewerInterface>(
			IPhysicsControlOperatorViewerInterface::GetModularFeatureName());
		return PhysicsControlViewerInterface.IsOperatorNamesTabOpen();
	}
	return false;
}

TArray<TPair<FName, TArray<FName>>> UAnimGraphNode_RigidBodyWithControl::GenerateControlsAndBodyModifierNames() const
{
	using OperatorNameAndTags = TPair<FName, TArray<FName>>;

	TArray<OperatorNameAndTags> GeneratedOperatorNames;

	if (USkeleton* const Skeleton = GetSkeleton())
	{
		const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

		// These functions will create the base set of controls and modifiers from SetupData
		TMap<FName, FPhysicsControlLimbBones> AllLimbBones =
			UE::PhysicsControl::GetLimbBones(
				Node.CharacterSetupData.LimbSetupData, RefSkeleton, Node.OverridePhysicsAsset.Get());

		TSet<FName> BodyModifierNames;
		TSet<FName> ControlNames;
		FPhysicsControlNameRecords NameRecords;

		// Note that controls can come from the setup data in the node and/or from a profile asset
		FPhysicsControlCharacterSetupData SetupData;
		if (IsValid(Node.PhysicsControlAsset))
		{
			SetupData = Node.PhysicsControlAsset->CharacterSetupData;
		}
		if (Node.bEnableCharacterSetupData)
		{
			SetupData += Node.CharacterSetupData;
		}

		FPhysicsControlAndBodyModifierCreationDatas AdditionalControlAndBodyModifierCreationDatas;
		if (IsValid(Node.PhysicsControlAsset))
		{
			AdditionalControlAndBodyModifierCreationDatas = Node.PhysicsControlAsset->AdditionalControlsAndModifiers;
		}
		AdditionalControlAndBodyModifierCreationDatas += Node.AdditionalControlsAndBodyModifiers;

		// Get the list of modifier and control names, based on the setup data
		UE::PhysicsControl::CollectOperatorNames(
			SetupData, AdditionalControlAndBodyModifierCreationDatas,
			AllLimbBones, RefSkeleton, Node.OverridePhysicsAsset.Get(), BodyModifierNames, ControlNames, NameRecords);

		// Create any additional sets that have been requested
		if (IsValid(Node.PhysicsControlAsset))
		{
			UE::PhysicsControl::CreateAdditionalSets(
				Node.PhysicsControlAsset->AdditionalSets, BodyModifierNames, ControlNames, NameRecords);
		}
		UE::PhysicsControl::CreateAdditionalSets(Node.AdditionalSets, BodyModifierNames, ControlNames, NameRecords);

		auto TransformOperatorNamesAndTags = [&GeneratedOperatorNames](
			const FName TypeTag, const TSet<FName>& Names, const TMap<FName, TArray<FName>>& SetToOperatorNameMap)
		{
			for (const FName OperatorName : Names)
			{
				OperatorNameAndTags NameAndTagsPair;

				NameAndTagsPair.Key = OperatorName;
				NameAndTagsPair.Value.Add(TypeTag);

				for (const TMap<FName, TArray<FName>>::ElementType& Set : SetToOperatorNameMap)
				{
					if (Set.Value.Contains(OperatorName))
					{
						NameAndTagsPair.Value.Add(Set.Key);
					}
				}

				GeneratedOperatorNames.Add(NameAndTagsPair);
			}
		};

		TransformOperatorNamesAndTags(FName("Modifier"), BodyModifierNames, NameRecords.BodyModifierSets);
		TransformOperatorNamesAndTags(FName("Control"), ControlNames, NameRecords.ControlSets);
	}

	return GeneratedOperatorNames;
}

USkeleton* UAnimGraphNode_RigidBodyWithControl::GetSkeleton() const
{
	USkeleton* Skeleton = nullptr;

	if (UAnimBlueprint* const AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(this)))
	{
		Skeleton = AnimBlueprint->TargetSkeleton;
	}

	return Skeleton;
}

#undef LOCTEXT_NAMESPACE
