// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_SkeletalControlBase.h"
#include "AnimNode_RigidBodyWithControl.h"
#include "Engine/PoseWatch.h"
#include "AnimGraphNode_RigidBodyWithControl.generated.h"

#define UE_API PHYSICSCONTROLUNCOOKEDONLY_API

UCLASS(MinimalAPI, meta=(Keywords = "Simulate Rigid Body Physics Ragdoll"))
class UAnimGraphNode_RigidBodyWithControl : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_RigidBodyWithControl Node;

public:
	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FString GetNodeCategory() const override;
	UE_API virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base
	UE_API virtual void Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* PreviewSkelMeshComp, const bool bIsSelected, const bool bIsPoseWatchEnabled) const override;
	UE_API virtual void OnPoseWatchChanged(const bool IsPoseWatchActive, TObjectPtr<UPoseWatch> InPoseWatch, FEditorModeTools& InModeTools, FAnimNode_Base* InRuntimeNode) override;

	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	UE_API virtual void PostPlacedNewNode() override;
	UE_API virtual void PostPasteNode() override;
	UE_API virtual void DestroyNode() override;

	UE_API void ToggleBodyVisibility();
	UE_API void ToggleConstraintVisibility();
	UE_API void ToggleControlSetViewerTab();

	UE_API TArray<TPair<FName, TArray<FName>>> GenerateControlsAndBodyModifierNames() const;

	UE_API USkeleton* GetSkeleton() const;

protected:
	// UAnimGraphNode_SkeletalControlBase interface
	UE_API virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface

	UE_API bool AreAnyBodiesHidden() const;
	UE_API bool AreAnyConstraintsHidden() const;
	UE_API bool IsControlSetViewerTabOpen() const;

	UE_API void PostChange();

private:
	TWeakObjectPtr<UPoseWatchElement> PoseWatchElementBodies;
	TWeakObjectPtr<UPoseWatchElement> PoseWatchElementConstraints;
	TWeakObjectPtr<UPoseWatchElement> PoseWatchElementParentSpaceControls;
	TWeakObjectPtr<UPoseWatchElement> PoseWatchElementWorldSpaceControls;
};

#undef UE_API
