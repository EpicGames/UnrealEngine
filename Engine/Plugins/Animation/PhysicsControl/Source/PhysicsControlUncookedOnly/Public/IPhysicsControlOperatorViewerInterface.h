// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

class IPhysicsControlOperatorViewerInterface : public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("PhysicsControlViewerInterface"));
		return FeatureName;
	}

	virtual void OpenOperatorNamesTab() = 0;
	virtual void CloseOperatorNamesTab() = 0;
	virtual void ToggleOperatorNamesTab() = 0;
	virtual bool IsOperatorNamesTabOpen() = 0;
	virtual void RequestRefresh() = 0;

};